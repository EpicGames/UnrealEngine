// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "RHIShaderPlatform.h"

class FConfigCacheIni;
class FConfigSection;
struct FConfigValue;

// Utility functions primarily for FShaderPlatformConfig, which is located in RenderCore.
// These functions are in RHI/Internal to centralize the config naming logic.
namespace UE::RHIShaderPlatformConfig
{
	RHI_API FConfigCacheIni* GetShaderFormatConfigCacheIni(FName ShaderFormat);

	RHI_API void GetConfigSectionNameForShaderPlatform(FStringBuilderBase& OutSectionName, EShaderPlatform ShaderPlatform);

	RHI_API const FConfigSection* GetConfigSectionForShaderPlatform(FConfigCacheIni* Config, EShaderPlatform ShaderPlatform, FStringBuilderBase& OutSectionName);
	RHI_API const FConfigSection* GetConfigSectionForShaderPlatform(EShaderPlatform ShaderPlatform, FStringBuilderBase& OutSectionName);

	RHI_API const FConfigSection* GetConfigSectionForShaderPlatform(FConfigCacheIni* Config, EShaderPlatform ShaderPlatform);
	RHI_API const FConfigSection* GetConfigSectionForShaderPlatform(EShaderPlatform ShaderPlatform);

	RHI_API const FConfigValue* GetConfigValueForShaderPlatform(FConfigCacheIni* Config, EShaderPlatform ShaderPlatform, FStringView SettingName);
	RHI_API const FConfigValue* GetConfigValueForShaderPlatform(EShaderPlatform ShaderPlatform, FStringView SettingName);
}
