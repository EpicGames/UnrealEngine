// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/App.h"
#include "RHIDefinitions.h"
#include "RHIResources.h"
#include "RHIResourceCollection.h"
#include "RHITextureReference.h"

namespace UE::RHI::Private
{

template<typename TResourceType> struct TResourceTypeStr {};
template<> struct TResourceTypeStr<FRHISamplerState> { static constexpr TCHAR String[] = TEXT("Sampler State"); };
template<> struct TResourceTypeStr<FRHITexture> { static constexpr TCHAR String[] = TEXT("Texture"); };
template<> struct TResourceTypeStr<FRHIShaderResourceView> { static constexpr TCHAR String[] = TEXT("Shader Resource View"); };
template<> struct TResourceTypeStr<FRHIUnorderedAccessView> { static constexpr TCHAR String[] = TEXT("Unordered Access View"); };
template<> struct TResourceTypeStr<FRHIResourceCollection> { static constexpr TCHAR String[] = TEXT("Resource Collection"); };

template<typename TResourceType, typename TCallback>
inline void EnumerateUniformBufferResources(const FRHIUniformBuffer* RESTRICT Buffer, int32 BufferIndex, const uint32* RESTRICT ResourceMap, TCallback && Callback)
{
	const TRefCountPtr<FRHIResource>* RESTRICT Resources = Buffer->GetResourceTable().GetData();

	uint32 BufferOffset = ResourceMap[BufferIndex];
	if (BufferOffset > 0)
	{
		const uint32* RESTRICT ResourceInfos = &ResourceMap[BufferOffset];
		uint32 ResourceInfo = *ResourceInfos++;
		do
		{
			checkSlow(FRHIResourceTableEntry::GetUniformBufferIndex(ResourceInfo) == BufferIndex);

			const uint16 ResourceIndex = FRHIResourceTableEntry::GetResourceIndex(ResourceInfo);
			const uint8 BindIndex = FRHIResourceTableEntry::GetBindIndex    (ResourceInfo);

			TResourceType* Resource = static_cast<TResourceType*>(Resources[ResourceIndex].GetReference());
			checkf(Resource
			       , TEXT("Null %s (resource %d bind %d) on UB Layout %s")
			       , TResourceTypeStr<TResourceType>::String
			       , ResourceIndex
			       , BindIndex
			       , *Buffer->GetLayout().GetDebugName()
			       );

			Callback(Resource, BindIndex);

			ResourceInfo = *ResourceInfos++;
		} while (FRHIResourceTableEntry::GetUniformBufferIndex(ResourceInfo) == BufferIndex);
	}
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
// Helper to call the respective setters for each FRHIResource contained inside the provided uniform buffer.
template<typename TBinder>
void SetAllUniformBufferResourcesForBindless(TBinder&& Binder, const FRHIUniformBuffer* Buffer)
{
	double CurrentTimeForTextureTimes = FAppTime::GetApproximateCurrentTime();

	for (const TRefCountPtr<FRHIResource>& Resource : Buffer->GetResourceTable())
	{
		if (Resource)
		{
			const ERHIResourceType ResourceType = Resource->GetType();
			switch (ResourceType)
			{
			case RRT_Texture:
				static_cast<FRHITexture*>(Resource.GetReference())->SetLastRenderTime(CurrentTimeForTextureTimes);
				Binder.SetTexture(static_cast<FRHITexture*>(Resource.GetReference()), 0);
				break;
			case RRT_TextureReference:
				static_cast<FRHITextureReference*>(Resource.GetReference())->SetLastRenderTime(CurrentTimeForTextureTimes);
				Binder.SetTexture(static_cast<FRHITextureReference*>(Resource.GetReference()), 0);
				break;
			case RRT_ShaderResourceView:
				Binder.SetSRV(static_cast<FRHIShaderResourceView*>(Resource.GetReference()), 0);
				break;
			case RRT_UnorderedAccessView:
				Binder.SetUAV(static_cast<FRHIUnorderedAccessView*>(Resource.GetReference()), 0);
				break;
			case RRT_ResourceCollection:
				Binder.SetResourceCollection(static_cast<FRHIResourceCollection*>(Resource.GetReference()), 0);
				break;
			case RRT_SamplerState:
				Binder.SetSampler(static_cast<FRHISamplerState*>(Resource.GetReference()), 0);
				break;
			}
		}
	}
}
#endif

template<typename TBinder, typename TUniformBufferArrayType, typename TBitMaskType, bool bFullyBindless = false>
void SetUniformBufferResourcesFromTables(TBinder && Binder, FRHIShader const& Shader, TBitMaskType& DirtyUniformBuffers, TUniformBufferArrayType const& BoundUniformBuffers
                                         #if ENABLE_RHI_VALIDATION
                                         , RHIValidation::FTracker* Tracker
                                         #endif
                                         )
{
	double CurrentTimeForTextureTimes = FAppTime::GetApproximateCurrentTime();
	FShaderResourceTable const& SRT = Shader.GetShaderResourceTable();

	// Mask the dirty bits by those buffers from which the shader has bound resources.
	uint32 DirtyBits = SRT.ResourceTableBits & DirtyUniformBuffers;

	#if PLATFORM_SUPPORTS_BINDLESS_RENDERING && !ENABLE_RHI_VALIDATION
	if constexpr (bFullyBindless)
	{
		while (DirtyBits)
		{
			// Scan for the lowest set bit, compute its index, clear it in the set of dirty bits.
			const uint32 LowestBitMask = (DirtyBits) & (-(int32)DirtyBits);
			const int32 BufferIndex = FMath::CountTrailingZeros(LowestBitMask); // todo: This has a branch on zero, we know it could never be zero...
			DirtyBits ^= LowestBitMask;

			EnumerateUniformBufferResources<FRHITexture>(BoundUniformBuffers[BufferIndex], BufferIndex, SRT.TextureMap.GetData(),
				[&](FRHITexture* Texture, uint8 Index)
				{
					Texture->SetLastRenderTime(CurrentTimeForTextureTimes);
				});
		}

		DirtyUniformBuffers = TBitMaskType(0);

		return;
	}
	#endif

	while (DirtyBits)
	{
		// Scan for the lowest set bit, compute its index, clear it in the set of dirty bits.
		const uint32 LowestBitMask = (DirtyBits) & (-(int32)DirtyBits);
		const int32 BufferIndex = FMath::CountTrailingZeros(LowestBitMask); // todo: This has a branch on zero, we know it could never be zero...
		DirtyBits ^= LowestBitMask;

		check(BufferIndex < SRT.ResourceTableLayoutHashes.Num());

		FRHIUniformBuffer* Buffer = BoundUniformBuffers[BufferIndex];

		#if DO_CHECK

		if (!Buffer)
		{
			UE_LOGF(LogRHI, Fatal, "Shader expected a uniform buffer at slot %u but got null instead (Shader='%ls' UB='%ls'). Rendering code needs to set a valid uniform buffer for this slot."
			       , BufferIndex
			       , Shader.GetShaderName()
			       , *Shader.GetUniformBufferName(BufferIndex)
			       );
		}
		else if (Buffer->GetLayout().GetHash() != SRT.ResourceTableLayoutHashes[BufferIndex])
		{
			FRHIUniformBufferLayout const& BufferLayout = Buffer->GetLayout();

			FString ResourcesString;
			for (FRHIUniformBufferResource const& Resource : BufferLayout.Resources)
			{
				ResourcesString += FString::Printf(TEXT("%s%d")
				                                   , ResourcesString.Len() ? TEXT(" ") : TEXT("")
				                                   , Resource.MemberType
				                                   );
			}

			// This might mean you are accessing a data you haven't bound e.g. GBuffer
			UE_LOGF(LogRHI, Fatal,
			       "Uniform buffer bound to slot %u is not what the shader expected:\n" "\tBound                : Uniform Buffer[%ls] with Hash[0x%08x]\n" "\tExpected             : Uniform Buffer[%ls] with Hash[0x%08x]\n" "\tShader Name          : %ls\n" "\tLayout CB Size       : %d\n" "\tLayout Num Resources : %d\n" "\tResource Types       : %ls\n"
			       , BufferIndex
			       , *BufferLayout.GetDebugName(), BufferLayout.GetHash()
			       , *Shader.GetUniformBufferName(BufferIndex), SRT.ResourceTableLayoutHashes[BufferIndex]
			       , Shader.GetShaderName()
			       , BufferLayout.ConstantBufferSize
			       , BufferLayout.Resources.Num()
			       , *ResourcesString
			       );
		}

		#endif // DO_CHECK

		// Textures
		EnumerateUniformBufferResources<FRHITexture>(Buffer, BufferIndex, SRT.TextureMap.GetData(),
			[&](FRHITexture* Texture, uint8 Index)
			{
				#if ENABLE_RHI_VALIDATION
				if (Tracker)
				{
					ERHIAccess Access = IsComputeShaderFrequency(Shader.GetFrequency())
					? ERHIAccess::SRVCompute
					: Shader.GetFrequency() == SF_Pixel
					? ERHIAccess::SRVGraphicsPixel
					: ERHIAccess::SRVGraphicsNonPixel;

					// Textures bound here only have their "common" plane accessible. Stencil etc is ignored.
					// (i.e. only access the color plane of a color texture, or depth plane of a depth texture)
					Tracker->Assert(Texture->GetViewIdentity(0, 0, 0, 0, uint32(RHIValidation::EResourcePlane::Common), 1), Access);
				}
				#endif
				Texture->SetLastRenderTime(CurrentTimeForTextureTimes);
				Binder.SetTexture(Texture, Index);
			});

		// SRVs
		EnumerateUniformBufferResources<FRHIShaderResourceView>(Buffer, BufferIndex, SRT.ShaderResourceViewMap.GetData(),
			[&](FRHIShaderResourceView* SRV, uint8 Index)
			{
				#if ENABLE_RHI_VALIDATION
				if (Tracker)
				{
					ERHIAccess Access = IsComputeShaderFrequency(Shader.GetFrequency())
					? ERHIAccess::SRVCompute
					: Shader.GetFrequency() == SF_Pixel
					? ERHIAccess::SRVGraphicsPixel
					: ERHIAccess::SRVGraphicsNonPixel;

					Tracker->Assert(SRV->GetViewIdentity(), Access);
				}
				if (GRHIValidationEnabled)
				{
					RHIValidation::ValidateShaderResourceView(&Shader, Index, SRV);
				}
				#endif
				Binder.SetSRV(SRV, Index);
			});

		#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		EnumerateUniformBufferResources<FRHIResourceCollection>(Buffer, BufferIndex, SRT.ResourceCollectionMap.GetData(),
			[&](FRHIResourceCollection* ResourceCollection, uint8 Index)
			{
				#if ENABLE_RHI_VALIDATION
				// todo: christopher.waters - ResourceCollection validation
				#endif
				Binder.SetResourceCollection(ResourceCollection, Index);
			});
		#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

		// Samplers
		EnumerateUniformBufferResources<FRHISamplerState>(Buffer, BufferIndex, SRT.SamplerMap.GetData(),
			[&](FRHISamplerState* Sampler, uint8 Index)
			{
				Binder.SetSampler(Sampler, Index);
			});

		// UAVs
		EnumerateUniformBufferResources<FRHIUnorderedAccessView>(Buffer, BufferIndex, SRT.UnorderedAccessViewMap.GetData(),
			[&](FRHIUnorderedAccessView* UAV, uint8 Index)
			{
				#if ENABLE_RHI_VALIDATION
				if (Tracker)
				{
					ERHIAccess Access = IsComputeShaderFrequency(Shader.GetFrequency())
					? ERHIAccess::UAVCompute
					: ERHIAccess::UAVGraphics;

					Tracker->AssertUAV(UAV, Access, Index);
				}
				#endif
				Binder.SetUAV(UAV, Index);
			});
	}

	DirtyUniformBuffers = TBitMaskType(0);
}


template<typename TBinder, typename TUniformBufferArrayType, typename TBitMaskType>
void SetFullyBindlessUniformBufferResourcesFromTables(TBinder && Binder, const FRHIShader& Shader, TBitMaskType& DirtyUniformBuffers, const TUniformBufferArrayType& BoundUniformBuffers
                                                      #if ENABLE_RHI_VALIDATION
                                                      , RHIValidation::FTracker* Tracker
                                                      #endif
                                                      )
{
	SetUniformBufferResourcesFromTables < TBinder, TUniformBufferArrayType, TBitMaskType, true > (
		MoveTemp(Binder)
		, Shader
		, DirtyUniformBuffers
		, BoundUniformBuffers
		#if ENABLE_RHI_VALIDATION
		, Tracker
		#endif
	);
}

} //! UE::RHI::Private