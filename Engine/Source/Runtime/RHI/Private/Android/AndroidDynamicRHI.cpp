// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidDynamicRHI.h"
#include "Modules/ModuleManager.h"
#include "Android/AndroidApplication.h"
#include "Misc/App.h"

namespace UE
{
	namespace FAndroidPlatformDynamicRHI
	{
		using EPSOPrecacheCompileType = FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType;
		static_assert((int)EPSOPrecacheCompileType::MaxPri == 3
			&& (int)EPSOPrecacheCompileType::MinPri == 1
			&& (int)EPSOPrecacheCompileType::NumTypes == 4, "Modifications maybe required if the number precache priorities changes.");

		static bool bEnablePSOSchedulingParams = true;
		static FAutoConsoleVariableRef CVarDisablePSOSchedulingParams(
			TEXT("android.PSOService.EnableSchedulingParams"),
			bEnablePSOSchedulingParams,
			TEXT("Whether to set scheduler values (such as nice and scheduler policy) for each precache process job\n")
			TEXT(" 1 (default)")
			,
			ECVF_RenderThreadSafe
		);
		static int32 PSOPrecacheAffinity[(int)EPSOPrecacheCompileType::NumTypes] = {};
		static FAutoConsoleVariableRef CVarMinPriPSOPrecacheAffinity(
			TEXT("android.PSOService.MinPriPSOPrecacheAffinity"),
			PSOPrecacheAffinity[(int)EPSOPrecacheCompileType::MinPri],
			TEXT("CPU affinity to use when compiling low priority precache PSOs via external compilers\n")
			TEXT(" 0: all cpus (default)\n")
			TEXT(" all other values represent a 32 bit mask of cpu affinity.")
			,
			ECVF_RenderThreadSafe
		);
		static FAutoConsoleVariableRef CVarNormalPriPSOPrecacheAffinity(
			TEXT("android.PSOService.NormalPriPSOPrecacheAffinity"),
			PSOPrecacheAffinity[(int)EPSOPrecacheCompileType::NormalPri],
			TEXT("CPU affinity to use when compiling normal priority precache PSOs via external compilers\n")
			TEXT(" 0: all cpus (default)\n")
			TEXT(" all other values represent a 32 bit mask of cpu affinity.")
			,
			ECVF_RenderThreadSafe
		);
		static FAutoConsoleVariableRef CVarMaxPriPSOPrecacheAffinity(
			TEXT("android.PSOService.MaxPriPSOPrecacheAffinity"),
			PSOPrecacheAffinity[(int)EPSOPrecacheCompileType::MaxPri],
			TEXT("CPU affinity to use when compiling high priority precache PSOs via external compilers\n")
			TEXT(" 0: all cpus (default)\n")
			TEXT(" all other values represent a 32 bit mask of cpu affinity.")
			,
			ECVF_RenderThreadSafe
		);

		static int32 GExternalCompilerFailureThreshold = 5;
		static FAutoConsoleVariableRef GExternalCompilerFailureThresholdCvar(
			TEXT("android.PSOService.ExternalCompilerFailureThreshold"),
			GExternalCompilerFailureThreshold,
			TEXT("Number of external PSO compiler failures to ignore before disabling external PSO compiling altogether.\n")
			TEXT("  default: 5"),
			ECVF_Default | ECVF_RenderThreadSafe
		);

		static bool ArePSOPrecacheAffinitiesSet()
		{
			return PSOPrecacheAffinity[(int)EPSOPrecacheCompileType::MinPri] != 0
				|| PSOPrecacheAffinity[(int)EPSOPrecacheCompileType::NormalPri] != 0
				|| PSOPrecacheAffinity[(int)EPSOPrecacheCompileType::MaxPri] != 0;
		}

		FPSOServicePriInfo::FPSOServicePriInfo(EPSOPrecacheCompileType PSOCompileType)
		{
			if (ArePSOPrecacheAffinitiesSet() && ensure(PSOCompileType >= EPSOPrecacheCompileType::MinPri && PSOCompileType <= EPSOPrecacheCompileType::MaxPri))
			{
				const uint32 CVarValue = PSOPrecacheAffinity[(int)PSOCompileType];
				const uint32 Affinity = CVarValue ? CVarValue : 0xFFFFFFFF;

				SetAffinity(Affinity);

				if (bEnablePSOSchedulingParams)
				{
					static const int32 NiceValuesPerPri[(int)EPSOPrecacheCompileType::NumTypes] = { 0,-19,0,10 };
					SetNice(NiceValuesPerPri[(int)PSOCompileType]);

					static const char SchedPolicy[(int)EPSOPrecacheCompileType::NumTypes] = { SCHED_IDLE, SCHED_IDLE, SCHED_NORMAL, SCHED_NORMAL };
					SetSchedPolicy(SchedPolicy[(int)PSOCompileType], 0); // sched_priority=0 as generally not used with scheduling policies we care about.
				}
			}
		}

		int32 GetPSOServiceFailureThreshold()
		{
			return GExternalCompilerFailureThreshold;
		}

		static FRHIReInitWindowCallbackType OnRHIReleaseWindowCallback;

		FRHIReInitWindowCallbackType& GetRHIOnReInitWindowCallback()
		{
			return OnRHIReleaseWindowCallback;
		}

		void SetRHIOnReInitWindowCallback(FRHIReInitWindowCallbackType&& InRHIOnReInitWindowCallback)
		{
			OnRHIReleaseWindowCallback = MoveTemp(InRHIOnReInitWindowCallback);
		}

		static FRHIReleaseWindowCallbackType OnRHIReleaseWindowCallbackType;

		FRHIReleaseWindowCallbackType& GetRHIOnReleaseWindowCallback()
		{
			return OnRHIReleaseWindowCallbackType;
		}

		void SetRHIOnReleaseWindowCallback(FRHIReleaseWindowCallbackType&& InRHIOnReleaseWindowCallback)
		{
			OnRHIReleaseWindowCallbackType = MoveTemp(InRHIOnReleaseWindowCallback);
		}

		static FRHIOnWindowCreatedCallbackType OnRHIWindowCreatedCallback;

		FRHIOnWindowCreatedCallbackType& GetRHIOnWindowCreatedCallback()
		{
			return OnRHIWindowCreatedCallback;
		}

		void SetRHIOnWindowCreatedCallback(FRHIOnWindowCreatedCallbackType&& InOnWindowCreatedCallback)
		{
			OnRHIWindowCreatedCallback = MoveTemp(InOnWindowCreatedCallback);
		}

	} // namespace FAndroidPlatformDynamicRHI
} // namespace UE

FDynamicRHI* PlatformCreateDynamicRHI()
{
	FDynamicRHI* DynamicRHI = NULL;

	// Load the dynamic RHI module.
	IDynamicRHIModule* DynamicRHIModule = NULL;
	ERHIFeatureLevel::Type RequestedFeatureLevel = ERHIFeatureLevel::Num;
	FString GraphicsRHI;

	if (FPlatformMisc::ShouldUseVulkan() || FPlatformMisc::ShouldUseDesktopVulkan())
	{
		// Vulkan is required, release the EGL created by FAndroidAppEntry::PlatformInit.
		FAndroidAppEntry::ReleaseEGL();

		DynamicRHIModule = &FModuleManager::LoadModuleChecked<IDynamicRHIModule>(TEXT("VulkanRHI"));
		if (!DynamicRHIModule->IsSupported())
		{
			DynamicRHIModule = &FModuleManager::LoadModuleChecked<IDynamicRHIModule>(TEXT("OpenGLDrv"));
			GraphicsRHI = TEXT("OpenGL");
		}
		else
		{
			RequestedFeatureLevel = FPlatformMisc::ShouldUseDesktopVulkan() ? ERHIFeatureLevel::SM5 : ERHIFeatureLevel::ES3_1;
			GraphicsRHI = TEXT("Vulkan");
		}
	}
	else
	{
		DynamicRHIModule = &FModuleManager::LoadModuleChecked<IDynamicRHIModule>(TEXT("OpenGLDrv"));
		GraphicsRHI = TEXT("OpenGL");
	}

	if (!DynamicRHIModule->IsSupported()) 
	{

	//	FMessageDialog::Open(EAppMsgType::Ok, TEXT("OpenGL 3.2 is required to run the engine."));
		FPlatformMisc::RequestExit(true, TEXT("PlatformCreateDynamicRHI"));
		DynamicRHIModule = NULL;
	}

	if (DynamicRHIModule)
	{
		FApp::SetGraphicsRHI(GraphicsRHI);
		// Create the dynamic RHI.
		DynamicRHI = DynamicRHIModule->CreateRHI(RequestedFeatureLevel);
	}

	return DynamicRHI;
}
