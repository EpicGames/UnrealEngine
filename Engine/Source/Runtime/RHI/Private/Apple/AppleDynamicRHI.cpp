// Copyright Epic Games, Inc. All Rights Reserved.


#include "RHI.h"
#include "Modules/ModuleManager.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "DataDrivenShaderPlatformInfo.h"
#include COMPILED_PLATFORM_HEADER_WITH_PREFIX(Apple/Platform, PlatformDynamicRHI.h)

#define LOCTEXT_NAMESPACE "AppleRHI"

//------------------------------------------------------------------------------
// MARK: - FAppleDynamicRHIOptions Union
//

union FAppleDynamicRHIOptions
{
	struct
	{
		uint16 ForceSM5                 : 1;
		uint16 ForceSM6                 : 1;
		uint16 PreferES31               : 1;
		uint16 ForceMTL                 : 1;
		uint16 UnusedReservedBits       : 12;
	};

	uint16 All;
};


//------------------------------------------------------------------------------
// MARK: - Apple Dynamic RHI Support Routines
//

static inline bool ValidateAppleDynamicRHIOptions(FAppleDynamicRHIOptions* Options)
{
	if (0 != (Options->ForceSM5 & Options->ForceSM6))
	{
		UE_LOGF(LogRHI, Fatal, "-sm5 and -sm6 are mutually exclusive options but more than one was specified on the command line.");
		return false;
	}
	if (0 != (Options->ForceMTL & Options->ForceSM6))
	{
		UE_LOGF(LogRHI, Warning, "-mtl and -sm6 are incompatible options, using MetalRHI with SM5.");
		Options->ForceSM5 = 1;
		Options->ForceSM6 = 0;
		Options->ForceMTL = 1;
	}
	if(Options->ForceSM6)
	{
		bool bSupportsSM6 = false;
		if (@available(macOS 15.0, *))
		{
			bSupportsSM6 = true;
		}
		
		if(!bSupportsSM6)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("MetalRHIOptionsError", "-sm6 is selected but Mac requires OS 15 to support SM6"));
			UE_LOGF(LogRHI, Fatal, "-sm6 is selected but Mac requires OS 15 to support SM6");
			return false;
		}
	}
	return true;
}

static bool InitAppleDynamicRHIOptions(FAppleDynamicRHIOptions* Options)
{
	Options->ForceSM5                 = FParse::Param(FCommandLine::Get(), TEXT("sm5"));
	Options->ForceSM6                 = FParse::Param(FCommandLine::Get(), TEXT("sm6"));
	Options->PreferES31               = FPlatformDynamicRHI::ShouldPreferFeatureLevelES31() && !(Options->ForceSM5 || Options->ForceSM6);
	Options->ForceMTL                 = FParse::Param(FCommandLine::Get(), TEXT("mtl"));

	return ValidateAppleDynamicRHIOptions(Options);
}

static inline bool ShouldUseShaderModelPreference(const FAppleDynamicRHIOptions& Options)
{
	return (0 != (Options.ForceSM5 | Options.ForceSM6 | Options.PreferES31));
}

static void ComputeRequestedFeatureLevel(const FAppleDynamicRHIOptions& Options, ERHIFeatureLevel::Type& RequestedFeatureLevel)
{
	if (!ShouldUseShaderModelPreference(Options))
	{
		TArray<FString> TargetedShaderFormats;

		FPlatformDynamicRHI::AddTargetedShaderFormats(TargetedShaderFormats);

		if (TargetedShaderFormats.Num() > 0)
		{
            // Pick the highest targeted shader format
            for(const FString& Format : TargetedShaderFormats)
            {
                FName ShaderFormatName(*Format);
                EShaderPlatform TargetedPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormatName);
                ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel(TargetedPlatform);
                
                if(FeatureLevel > RequestedFeatureLevel || RequestedFeatureLevel == ERHIFeatureLevel::Num)
                {
                    RequestedFeatureLevel = FeatureLevel;
                }
            }
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("MetalMissingTargetError", "No Targeted RHI is set for this project, defaulting to SM5"));
			RequestedFeatureLevel = ERHIFeatureLevel::SM5;
		}
	}
	else
	{
		if (Options.ForceSM6)
		{
			RequestedFeatureLevel = ERHIFeatureLevel::SM6;
		}
		else if (Options.ForceSM5)
		{
			RequestedFeatureLevel = ERHIFeatureLevel::SM5;
		}
		else
		{
			check(Options.PreferES31 != 0);
			RequestedFeatureLevel = ERHIFeatureLevel::ES3_1;
		}
	}

	check(RequestedFeatureLevel != ERHIFeatureLevel::Num);
}

static IDynamicRHIModule* LoadDynamicRHIModule(ERHIFeatureLevel::Type& RequestedFeatureLevel)
{
	IDynamicRHIModule*      DynamicRHIModule = nullptr;
	FAppleDynamicRHIOptions Options          = { .All = 0 };

	if (!InitAppleDynamicRHIOptions(&Options))
	{
		return nullptr;
	}

	ComputeRequestedFeatureLevel(Options, RequestedFeatureLevel);
	
	DynamicRHIModule = &FModuleManager::LoadModuleChecked<IDynamicRHIModule>(TEXT("MetalRHI"));

	return DynamicRHIModule;
}


//------------------------------------------------------------------------------
// MARK: - Dynamic RHI API
//

FDynamicRHI* PlatformCreateDynamicRHI()
{
	FDynamicRHI*           DynamicRHI            = nullptr;
	IDynamicRHIModule*     DynamicRHIModule      = nullptr;
	ERHIFeatureLevel::Type RequestedFeatureLevel = ERHIFeatureLevel::Num;

	if (nullptr != (DynamicRHIModule = LoadDynamicRHIModule(RequestedFeatureLevel)))
	{
		DynamicRHI = DynamicRHIModule->CreateRHI(RequestedFeatureLevel);

		// The Feature Level is known after creating the RHI
		FString FeatureLevelName;
		GetFeatureLevelName(GMaxRHIFeatureLevel, FeatureLevelName);
		FString BindlessStatus = " ";
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		BindlessStatus = GRHIGlobals.bSupportsBindless ? TEXT(" Bindless ") : TEXT(" Bindful ");
#endif
		FApp::SetGraphicsRHI(FString::Printf(TEXT("Metal%s(%s)"), *BindlessStatus, *FeatureLevelName));
	}

	return DynamicRHI;
}

#undef LOCTEXT_NAMESPACE
