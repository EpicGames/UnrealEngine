// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


//------------------------------------------------------------------------------
// MARK: - iOS/tvOS Platform Dynamic RHI Defines
//


//------------------------------------------------------------------------------
// NOTE: Unreal Engine uses the same headers for iOS and tvOS; consequenty, we
//       use PLATFORM defines in this _rare_ instance to differentiate shader
//       platform types.
//------------------------------------------------------------------------------


#define SHADER_PLATFORM_METAL_SM6	SP_METAL_SM6_IOS

#if PLATFORM_TVOS
#define SHADER_PLATFORM_METAL_SM5	SP_METAL_SM5_TVOS
#define SHADER_PLATFORM_METAL_ES3_1	SP_METAL_ES3_1_TVOS
#elif WITH_IOS_SIMULATOR
#define SHADER_PLATFORM_METAL_ES3_1	SP_METAL_SIM
#define SHADER_PLATFORM_METAL_SM5	SP_METAL_SM5_IOS
#else
#define SHADER_PLATFORM_METAL_SM5	SP_METAL_SM5_IOS
#define SHADER_PLATFORM_METAL_ES3_1	SP_METAL_ES3_1_IOS
#endif

//------------------------------------------------------------------------------
// MARK: - iOS/tvOS Platform Dynamic RHI Routines
//

namespace UE
{
namespace FIOSPlatformDynamicRHI
{

bool ShouldPreferFeatureLevelES31()
{
	return FParse::Param(FCommandLine::Get(), TEXT("metal"));
}

FORCEINLINE bool ShouldSupportMetalMobileSM5()
{
	bool bSupportsMetalMobileSM5 = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMobileSM5"), bSupportsMetalMobileSM5, GEngineIni);
	return (bSupportsMetalMobileSM5 || FParse::Param(FCommandLine::Get(), TEXT("metalmobilesm5"))) && !ShouldPreferFeatureLevelES31();
}

FORCEINLINE bool ShouldSupportMetalMobileSM6()
{
	bool bSupportsMetalMobileSM6 = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMobileSM6"), bSupportsMetalMobileSM6, GEngineIni);
	return (bSupportsMetalMobileSM6 || FParse::Param(FCommandLine::Get(), TEXT("metalmobilesm6"))) && !ShouldPreferFeatureLevelES31();
}

void AddTargetedShaderFormats(TArray<FString>& TargetedShaderFormats)
{
	if (ShouldSupportMetalMobileSM6())
	{
		TargetedShaderFormats.Add(LegacyShaderPlatformToShaderFormat(SHADER_PLATFORM_METAL_SM6).ToString());
	}
	if (ShouldSupportMetalMobileSM5())
	{
		TargetedShaderFormats.Add(LegacyShaderPlatformToShaderFormat(SHADER_PLATFORM_METAL_SM5).ToString());
	}
	TargetedShaderFormats.Add(LegacyShaderPlatformToShaderFormat(SHADER_PLATFORM_METAL_ES3_1).ToString());
}

} // namespace FIOSPlatformDynamicRHI
} // namespace UE

namespace FPlatformDynamicRHI = UE::FIOSPlatformDynamicRHI;
