// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


//------------------------------------------------------------------------------
// MARK: - Mac Platform Dynamic RHI Routines
//

namespace UE
{
namespace FMacPlatformDynamicRHI
{

bool ShouldPreferFeatureLevelES31()
{
	// TODO: Force low-spec users into performance mode but respect their choice once they have set a preference.
	return false;
}

void AddTargetedShaderFormats(TArray<FString>& TargetedShaderFormats)
{
	GConfig->GetArray(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("TargetedRHIs"), TargetedShaderFormats, GEngineIni);
}

} // namespace FMacPlatformDynamicRHI
} // namespace UE

namespace FPlatformDynamicRHI = UE::FMacPlatformDynamicRHI;
