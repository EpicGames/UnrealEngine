// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BoundShaderStateCache.cpp: Bound shader state cache implementation.
=============================================================================*/

#include "BoundShaderStateCache.h"
#include "Misc/ScopeLock.h"


typedef TMap<FBoundShaderStateLookupKey,FCachedBoundShaderStateLink*> FBoundShaderStateCache;
typedef TMap<FBoundShaderStateLookupKey,FCachedBoundShaderStateLink_Threadsafe*> FBoundShaderStateCache_Threadsafe;

static FBoundShaderStateCache GBoundShaderStateCache;
static FBoundShaderStateCache_Threadsafe GBoundShaderStateCache_ThreadSafe;

/** Lazily initialized bound shader state cache singleton. */
static FBoundShaderStateCache& GetBoundShaderStateCache()
{
	return GBoundShaderStateCache;
}

/** Lazily initialized bound shader state cache singleton. */
static FBoundShaderStateCache_Threadsafe& GetBoundShaderStateCache_Threadsafe()
{
	return GBoundShaderStateCache_ThreadSafe;
}

static FCriticalSection BoundShaderStateCacheLock;


FCachedBoundShaderStateLink::FCachedBoundShaderStateLink(
	FRHIVertexDeclaration* VertexDeclaration,
	FRHIVertexShader* VertexShader,
	FRHIPixelShader* PixelShader,
	FRHIBoundShaderState* InBoundShaderState,
	bool bAddToSingleThreadedCache)
	: BoundShaderState(InBoundShaderState)
	, Key(VertexDeclaration,VertexShader,PixelShader)
	, bAddedToSingleThreadedCache(bAddToSingleThreadedCache)
{
	if (bAddToSingleThreadedCache)
	{
		GetBoundShaderStateCache().Add(Key,this);
	}
}

FCachedBoundShaderStateLink::FCachedBoundShaderStateLink(
	FRHIVertexDeclaration* VertexDeclaration,
	FRHIVertexShader* VertexShader,
	FRHIPixelShader* PixelShader,
	FRHIGeometryShader* GeometryShader,
	FRHIBoundShaderState* InBoundShaderState,
	bool bAddToSingleThreadedCache)
	: BoundShaderState(InBoundShaderState)
	, Key(VertexDeclaration, VertexShader, PixelShader, GeometryShader)
	, bAddedToSingleThreadedCache(bAddToSingleThreadedCache)
{
	if (bAddToSingleThreadedCache)
	{
		GetBoundShaderStateCache().Add(Key, this);
	}
}

FCachedBoundShaderStateLink::FCachedBoundShaderStateLink(
	FRHIMeshShader* MeshShader,
	FRHIAmplificationShader* AmplificationShader,
	FRHIPixelShader* PixelShader,
	FRHIBoundShaderState* InBoundShaderState,
	bool bAddToSingleThreadedCache)
	: BoundShaderState(InBoundShaderState)
	, Key(MeshShader, AmplificationShader, PixelShader)
	, bAddedToSingleThreadedCache(bAddToSingleThreadedCache)
{
	if (bAddToSingleThreadedCache)
	{
		GetBoundShaderStateCache().Add(Key, this);
	}
}

FCachedBoundShaderStateLink::~FCachedBoundShaderStateLink()
{
	if (bAddedToSingleThreadedCache)
	{
		GetBoundShaderStateCache().Remove(Key);
		bAddedToSingleThreadedCache = false;
	}
}

FCachedBoundShaderStateLink* GetCachedBoundShaderState(
	FRHIVertexDeclaration* VertexDeclaration,
	FRHIVertexShader* VertexShader,
	FRHIPixelShader* PixelShader,
	FRHIGeometryShader* GeometryShader,
	FRHIMeshShader* MeshShader,
	FRHIAmplificationShader* AmplificationShader
	)
{
	if (MeshShader)
	{
		// Find the existing bound shader state in the cache.
		return GetBoundShaderStateCache().FindRef(
			FBoundShaderStateLookupKey(MeshShader, AmplificationShader, PixelShader)
		);
	}
	else
	{
		// Find the existing bound shader state in the cache.
		return GetBoundShaderStateCache().FindRef(
			FBoundShaderStateLookupKey(VertexDeclaration, VertexShader, PixelShader, GeometryShader)
		);
	}
}


void FCachedBoundShaderStateLink_Threadsafe::AddToCache()
{
	FScopeLock Lock(&BoundShaderStateCacheLock);
	GetBoundShaderStateCache_Threadsafe().Add(Key,this);
}
void FCachedBoundShaderStateLink_Threadsafe::RemoveFromCache()
{
	FScopeLock Lock(&BoundShaderStateCacheLock);
	GetBoundShaderStateCache_Threadsafe().Remove(Key);
}


FBoundShaderStateRHIRef GetCachedBoundShaderState_Threadsafe(
	FRHIVertexDeclaration* VertexDeclaration,
	FRHIVertexShader* VertexShader,
	FRHIPixelShader* PixelShader,
	FRHIGeometryShader* GeometryShader,
	FRHIMeshShader* MeshShader,
	FRHIAmplificationShader* AmplificationShader
	)
{
	FScopeLock Lock(&BoundShaderStateCacheLock);

	// Find the existing bound shader state in the cache.
	FCachedBoundShaderStateLink_Threadsafe* CachedBoundShaderStateLink;
	if (MeshShader)
	{
		CachedBoundShaderStateLink = GetBoundShaderStateCache_Threadsafe().FindRef(
			FBoundShaderStateLookupKey(MeshShader, AmplificationShader, PixelShader)
		);
	}
	else
	{
		CachedBoundShaderStateLink = GetBoundShaderStateCache_Threadsafe().FindRef(
			FBoundShaderStateLookupKey(VertexDeclaration, VertexShader, PixelShader, GeometryShader)
		);
	}
	
	if(CachedBoundShaderStateLink && CachedBoundShaderStateLink->BoundShaderState->IsValid())
	{
		// If we've already created a bound shader state with these parameters, reuse it.
		return CachedBoundShaderStateLink->BoundShaderState;
	}
	return FBoundShaderStateRHIRef();
}

void EmptyCachedBoundShaderStates()
{
	GetBoundShaderStateCache().Empty(0);
	GetBoundShaderStateCache_Threadsafe().Empty(0);
}
