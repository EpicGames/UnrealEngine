// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/CommandLine.h"
#include "RHI.h"
#include "RHIStrings.h"

#if WITH_EDITOR
#include "Internationalization/Text.h"
#endif

const FName LANGUAGE_D3D("D3D");
const FName LANGUAGE_Metal("Metal");
const FName LANGUAGE_OpenGL("OpenGL");
const FName LANGUAGE_Vulkan("Vulkan");
const FName LANGUAGE_Sony("Sony");
const FName LANGUAGE_Nintendo("Nintendo");

FGenericDataDrivenShaderPlatformInfo FGenericDataDrivenShaderPlatformInfo::Infos[SP_NumPlatforms];
static TMap<FName, EShaderPlatform> PlatformNameToShaderPlatformMap;

#if WITH_EDITOR
struct FDataDrivenShaderPlatformInfoEditorOnly
{
	TMap<FName,FText> DeviceProfileToFriendlyName;
	EShaderPlatform PreviewShaderPlatformParent = SP_NumPlatforms;
	EShaderPlatform RuntimePlatform = SP_NumPlatforms;
	uint32 bCanUsePreviewPlatformForMaterialValidation : 1;
};
static FDataDrivenShaderPlatformInfoEditorOnly DataDrivenShaderPlatformInfoEditorOnlyInfos[SP_NumPlatforms];

TMap<FString, TFunction<bool(const FStaticShaderPlatform Platform)>> FGenericDataDrivenShaderPlatformInfo::PropertyToShaderPlatformFunctionMap;
#endif

EShaderPlatform ParseShaderPlatform(const TCHAR* String)
{
	for (int i = 0; i < (int)EShaderPlatform::SP_NumPlatforms; ++i)
	{
		if (LexToString((EShaderPlatform)i, false).Equals(String, ESearchCase::IgnoreCase))
		{
			return (EShaderPlatform)i;
		}
	}

	return SP_NumPlatforms;
}

// Gets a string from a section, or empty string if it didn't exist
static inline FString GetSectionString(const FConfigSection& Section, FName Key)
{
	const FConfigValue* Value = Section.Find(Key);
	return Value ? Value->GetValue() : FString();
}

// Gets a string from a section, or default string if it didn't exist
static inline FName GetSectionString(const FConfigSection& Section, FName Key, FName DefaultValue)
{
	const FConfigValue* Value = Section.Find(Key);
	return Value ? *Value->GetValue() : DefaultValue;
}

// Gets a bool from a section.  It returns the original value if the setting does not exist
static inline bool GetSectionBool(const FConfigSection& Section, FName Key, bool OriginalValue)
{
	const FConfigValue* ConfigValue = Section.Find(Key);
	if (ConfigValue != nullptr)
	{
		return FCString::ToBool(*ConfigValue->GetValue());
	}
	else
	{
		return OriginalValue;
	}
}

// Gets an integer from a section.  It returns the original value if the setting does not exist
static inline uint32 GetSectionUint(const FConfigSection& Section, FName Key, uint32 OriginalValue)
{
	const FConfigValue* ConfigValue = Section.Find(Key);
	if (ConfigValue != nullptr)
	{
		return (uint32)FCString::Atoi(*ConfigValue->GetValue());
	}
	else
	{
		return OriginalValue;
	}
}

// Gets an integer from a section.  It returns the original value if the setting does not exist
static inline uint32 GetSectionFeatureSupport(const FConfigSection& Section, FName Key, uint32 OriginalValue)
{
	const FConfigValue* ConfigValue = Section.Find(Key);
	if (ConfigValue != nullptr)
	{
		FString Value = ConfigValue->GetValue();
		if (Value == TEXT("Unsupported"))
		{
			return uint32(ERHIFeatureSupport::Unsupported);
		}
		if (Value == TEXT("RuntimeDependent"))
		{
			return uint32(ERHIFeatureSupport::RuntimeDependent);
		}
		else if (Value == TEXT("RuntimeGuaranteed"))
		{
			return uint32(ERHIFeatureSupport::RuntimeGuaranteed);
		}
		else
		{
			checkf(false, TEXT("Unknown ERHIFeatureSupport value \"%s\" for %s"), *Value, *Key.ToString());
		}
	}

	return OriginalValue;
}

static inline uint32 GetSectionStaticShaderBindingLayoutSupport(const FConfigSection& Section, FName Key, uint32 OriginalValue)
{
	const FConfigValue* ConfigValue = Section.Find(Key);
	if (ConfigValue != nullptr)
	{
		FString Value = ConfigValue->GetValue();
		if (Value == TEXT("Unsupported"))
		{
			return uint32(ERHIStaticShaderBindingLayoutSupport::Unsupported);
		}
		if (Value == TEXT("RayTracingOnly"))
		{
			return uint32(ERHIStaticShaderBindingLayoutSupport::RayTracingOnly);
		}
		else if (Value == TEXT("AllShaderTypes"))
		{
			return uint32(ERHIStaticShaderBindingLayoutSupport::AllShaderTypes);
		}
		else
		{
			checkf(false, TEXT("Unknown ERHIStaticShaderBindingLayoutSupport value \"%s\" for %s"), *Value, *Key.ToString());
		}
	}

	return OriginalValue;
}

void FGenericDataDrivenShaderPlatformInfo::SetDefaultValues()
{
	MaxFeatureLevel = ERHIFeatureLevel::Num;
	bSupportsMSAA = true;
	bSupportsUnrestrictedHalfFloatBuffers = true;
	bSupportsDOFHybridScattering = true;
	bSupportsHZBOcclusion = true;
	bSupportsWaterIndirectDraw = true;
	bSupportsAsyncPipelineCompilation = true;
	bSupportsVertexShaderSRVs = true; // Explicitly overriden to false for ES 3.1 platforms via DDPI ini
	bSupportsVertexShaderUAVs = uint32(ERHIFeatureSupport::Unsupported);
	bSupportsTypedBufferSRVs = true;
	bSupportsManualVertexFetch = true;
	bSupportsVolumeTextureAtomics = true;
	bSupportsClipDistance = true;
	bSupportsShaderPipelines = true;
	MaxSamplers = 16;
}

void FGenericDataDrivenShaderPlatformInfo::ParseDataDrivenShaderInfo(const FConfigSection& Section, uint32 Index)
{
	FGenericDataDrivenShaderPlatformInfo& Info = Infos[Index];

	Info.Language = GetSectionString(Section, "Language", Info.Language);
	Info.ShaderFormat = GetSectionString(Section, "ShaderFormat", Info.ShaderFormat);
	checkf(!Info.ShaderFormat.IsNone(), TEXT("Missing ShaderFormat for ShaderPlatform %s  ShaderFormat %s"), *Info.Name.ToString(), *Info.ShaderFormat.ToString());

	const FConfigValue* MaxFeatureLevelValue = Section.Find("MaxFeatureLevel");
	if (MaxFeatureLevelValue)
	{
		GetFeatureLevelFromName(MaxFeatureLevelValue->GetValue(), Info.MaxFeatureLevel);
	}

	Info.ShaderPropertiesHash = 0;
	FString ShaderPropertiesString = Info.Name.GetPlainNameString();

#define ADD_TO_PROPERTIES_STRING(SettingName, SettingValue) \
	ShaderPropertiesString += TEXT(#SettingName); \
	ShaderPropertiesString += FString::Printf(TEXT("_%d"), SettingValue);

#if WITH_EDITOR
	#define ADD_PROPERTY_TO_SHADERPLATFORM_FUNCTIONMAP(SettingName, FunctionName) \
	FGenericDataDrivenShaderPlatformInfo::PropertyToShaderPlatformFunctionMap.FindOrAdd(#SettingName) = &FunctionName;
#else
	#define ADD_PROPERTY_TO_SHADERPLATFORM_FUNCTIONMAP(SettingName, FunctionName)
#endif

#define GET_SECTION_BOOL_HELPER(SettingName)	\
	Info.SettingName = GetSectionBool(Section, #SettingName, Info.SettingName);	\
	ADD_TO_PROPERTIES_STRING(SettingName, Info.SettingName)

#define GET_SECTION_INT_HELPER(SettingName)	\
	Info.SettingName = GetSectionUint(Section, #SettingName, Info.SettingName); \
	ADD_TO_PROPERTIES_STRING(SettingName, Info.SettingName)

#define GET_SECTION_SUPPORT_HELPER(SettingName)	\
	Info.SettingName = GetSectionFeatureSupport(Section, #SettingName, Info.SettingName); \
	ADD_TO_PROPERTIES_STRING(SettingName, Info.SettingName)

#define GET_SECTION_STATIC_SHADER_BINDING_LAYOUT_SUPPORT_HELPER(SettingName) \
	Info.SettingName = GetSectionStaticShaderBindingLayoutSupport(Section, #SettingName, Info.SettingName); \
	ADD_TO_PROPERTIES_STRING(SettingName, Info.SettingName)

	// These properties will be exposed to the MaterialEditor that use the ShaderPlatformInfo Node
	// If you remove/rename a property be sure to address this in UMaterialExpressionDataDrivenShaderPlatformInfoSwitch serialization
	// If you add a property that you think needs to be exposed to materials, it is enough to just call the macro here with the property name and the function it will call
	ADD_PROPERTY_TO_SHADERPLATFORM_FUNCTIONMAP(IsMobile, GetIsMobile);
	ADD_PROPERTY_TO_SHADERPLATFORM_FUNCTIONMAP(IsPC, GetIsPC);
	ADD_PROPERTY_TO_SHADERPLATFORM_FUNCTIONMAP(IsConsole, GetIsConsole);

	GET_SECTION_BOOL_HELPER(bIsMobile);
	GET_SECTION_BOOL_HELPER(bIsPC);
	GET_SECTION_BOOL_HELPER(bIsConsole);
	GET_SECTION_BOOL_HELPER(bIsAndroidOpenGLES);
	GET_SECTION_BOOL_HELPER(bSupportsDebugViewShaders);
	GET_SECTION_BOOL_HELPER(bSupportsMobileMultiView);
	GET_SECTION_BOOL_HELPER(bSupportsArrayTextureCompression);
	GET_SECTION_BOOL_HELPER(bSupportsDistanceFields);
	GET_SECTION_BOOL_HELPER(bSupportsDiaphragmDOF);
	GET_SECTION_BOOL_HELPER(bSupportsRGBColorBuffer);
	GET_SECTION_BOOL_HELPER(bSupportsPercentageCloserShadows);
	GET_SECTION_BOOL_HELPER(bSupportsIndexBufferUAVs);
	GET_SECTION_BOOL_HELPER(bSupportsInstancedStereo);
	GET_SECTION_SUPPORT_HELPER(SupportsMultiViewport);
	GET_SECTION_BOOL_HELPER(bSupportsMSAA);
	GET_SECTION_BOOL_HELPER(bSupports4ComponentUAVReadWrite);
	GET_SECTION_BOOL_HELPER(bSupportsShaderRootConstants);
	GET_SECTION_BOOL_HELPER(bSupportsShaderBundleDispatch);
	GET_SECTION_BOOL_HELPER(bSupportsRenderTargetWriteMask);
	GET_SECTION_BOOL_HELPER(bSupportsRayTracing);
	GET_SECTION_BOOL_HELPER(bSupportsRayTracingShaders);
	GET_SECTION_BOOL_HELPER(bSupportsInlineRayTracing);
	GET_SECTION_BOOL_HELPER(bInlineRayTracingRequiresBindless);
	GET_SECTION_BOOL_HELPER(bSupportsRayTracingCallableShaders);
	GET_SECTION_BOOL_HELPER(bSupportsRayTracingProceduralPrimitive);
	GET_SECTION_BOOL_HELPER(bSupportsRayTracingLinearSweptSpheres);
	GET_SECTION_BOOL_HELPER(bSupportsRayTracingTraversalStatistics);
	GET_SECTION_BOOL_HELPER(bSupportsRayTracingIndirectInstanceData);
	GET_SECTION_BOOL_HELPER(bSupportsRayTracingClusterOps);
	GET_SECTION_BOOL_HELPER(bSupportsPathTracing);
	GET_SECTION_BOOL_HELPER(bSupportsShaderExecutionReordering);
	GET_SECTION_BOOL_HELPER(bSupportsUnrestrictedHalfFloatBuffers)
	GET_SECTION_BOOL_HELPER(bSupportsGPUScene);
	GET_SECTION_BOOL_HELPER(bSupportsPrimitiveShaders);
	GET_SECTION_BOOL_HELPER(bSupportsUInt64ImageAtomics);
	GET_SECTION_BOOL_HELPER(bRequiresVendorExtensionsForAtomics);
	GET_SECTION_BOOL_HELPER(bSupportsNanite);
	GET_SECTION_BOOL_HELPER(bSupportsLumenGI);
	GET_SECTION_BOOL_HELPER(bSupportsSSDIndirect);
	GET_SECTION_BOOL_HELPER(bSupportsTemporalHistoryUpscale);
	GET_SECTION_BOOL_HELPER(bSupportsRTIndexFromVS);
	GET_SECTION_BOOL_HELPER(bSupportsIntrinsicWaveOnce);
	GET_SECTION_BOOL_HELPER(bSupportsConservativeRasterization);
	GET_SECTION_SUPPORT_HELPER(bSupportsWaveOperations);
	GET_SECTION_BOOL_HELPER(bSupportsWavePermute);
	GET_SECTION_INT_HELPER(MinimumWaveSize);
	GET_SECTION_INT_HELPER(MaximumWaveSize);
	GET_SECTION_BOOL_HELPER(bRequiresExplicit128bitRT);
	GET_SECTION_BOOL_HELPER(bSupportsGen5TemporalAA);
	GET_SECTION_BOOL_HELPER(bTargetsTiledGPU);
	GET_SECTION_BOOL_HELPER(bNeedsOfflineCompiler);
	GET_SECTION_BOOL_HELPER(bSupportsComputeFramework);
	GET_SECTION_BOOL_HELPER(bSupportsAnisotropicMaterials);
	GET_SECTION_BOOL_HELPER(bSupportsDualSourceBlending);
	GET_SECTION_BOOL_HELPER(bRequiresGeneratePrevTransformBuffer);
	GET_SECTION_BOOL_HELPER(bRequiresRenderTargetDuringRaster);
	GET_SECTION_BOOL_HELPER(bRequiresDisableForwardLocalLights);
	GET_SECTION_BOOL_HELPER(bCompileSignalProcessingPipeline);
	GET_SECTION_BOOL_HELPER(bSupportsMeshShadersTier0);
	GET_SECTION_BOOL_HELPER(bSupportsMeshShadersTier1);
	GET_SECTION_BOOL_HELPER(bSupportsMeshShadersWithClipDistance);
	GET_SECTION_INT_HELPER(MaxMeshShaderThreadGroupSize);
	GET_SECTION_BOOL_HELPER(bStripNaniteVertexShaders);
	GET_SECTION_BOOL_HELPER(bRequiresUnwrappedMeshShaderArgs);
	GET_SECTION_BOOL_HELPER(bSupportsPerPixelDBufferMask);
	GET_SECTION_BOOL_HELPER(bSupportsDxc);
	GET_SECTION_BOOL_HELPER(bSupportsVariableRateShading);
	GET_SECTION_BOOL_HELPER(bIsSPIRV);
	GET_SECTION_INT_HELPER(NumberOfComputeThreads);

	GET_SECTION_BOOL_HELPER(bWaterUsesSimpleForwardShading);
	GET_SECTION_BOOL_HELPER(bSupportsHairStrandGeometry);
	GET_SECTION_BOOL_HELPER(bSupportsDOFHybridScattering);
	GET_SECTION_BOOL_HELPER(bNeedsExtraMobileFrames);
	GET_SECTION_BOOL_HELPER(bSupportsHZBOcclusion);
	GET_SECTION_BOOL_HELPER(bSupportsWaterIndirectDraw);
	GET_SECTION_BOOL_HELPER(bSupportsAsyncPipelineCompilation);
	GET_SECTION_BOOL_HELPER(bSupportsVertexShaderSRVs);
	GET_SECTION_SUPPORT_HELPER(bSupportsVertexShaderUAVs);
	GET_SECTION_BOOL_HELPER(bSupportsTypedBufferSRVs);
	GET_SECTION_BOOL_HELPER(bSupportsManualVertexFetch);
	GET_SECTION_BOOL_HELPER(bRequiresReverseCullingOnMobile);
	GET_SECTION_BOOL_HELPER(bOverrideFMaterial_NeedsGBufferEnabled);
	GET_SECTION_BOOL_HELPER(bSupportsFFTBloom);
	GET_SECTION_BOOL_HELPER(bSupportsVertexShaderLayer);
	GET_SECTION_BOOL_HELPER(bSupportsBindless);
	GET_SECTION_BOOL_HELPER(bSupportsDescriptorRange);
	GET_SECTION_STATIC_SHADER_BINDING_LAYOUT_SUPPORT_HELPER(StaticShaderBindingLayoutSupport);
	GET_SECTION_BOOL_HELPER(bSupportsVolumeTextureAtomics);
	GET_SECTION_BOOL_HELPER(bSupportsROV);
	GET_SECTION_BOOL_HELPER(bSupportsOIT);
	GET_SECTION_SUPPORT_HELPER(bSupportsRealTypes);
	GET_SECTION_INT_HELPER(EnablesHLSL2021ByDefault);
	GET_SECTION_BOOL_HELPER(bSupportsSceneDataCompressedTransforms);
	GET_SECTION_BOOL_HELPER(bSupportsSwapchainUAVs);
	GET_SECTION_BOOL_HELPER(bSupportsClipDistance);
	GET_SECTION_BOOL_HELPER(bSupportsNNEShaders);
	GET_SECTION_BOOL_HELPER(bSupportsShaderPipelines);
	GET_SECTION_BOOL_HELPER(bSupportsUniformBufferObjects);
	GET_SECTION_BOOL_HELPER(bRequiresBindfulUtilityShaders);
	GET_SECTION_INT_HELPER(MaxSamplers);
	GET_SECTION_BOOL_HELPER(SupportsBarycentricsIntrinsics);
	GET_SECTION_SUPPORT_HELPER(SupportsBarycentricsSemantic);
	GET_SECTION_BOOL_HELPER(bSupportsWave64);
	GET_SECTION_BOOL_HELPER(bSupportsIndependentSamplers);
	GET_SECTION_BOOL_HELPER(bSupportsWorkGraphs);
	GET_SECTION_BOOL_HELPER(bSupportsWorkGraphsTier1_1);
	GET_SECTION_BOOL_HELPER(bSupportsDLSSShaders);
	GET_SECTION_BOOL_HELPER(bSupportsAdaptiveGBuffer);
	GET_SECTION_BOOL_HELPER(bSupportsSpecializationConstants);
#undef GET_SECTION_BOOL_HELPER
#undef GET_SECTION_INT_HELPER
#undef GET_SECTION_SUPPORT_HELPER
#undef ADD_TO_PROPERTIES_STRING
#undef ADD_PROPERTY_TO_SHADERPLATFORM_FUNCTIONMAP

	Info.ShaderPropertiesHash = GetTypeHash(ShaderPropertiesString);

#if WITH_EDITOR
	FDataDrivenShaderPlatformInfoEditorOnly& EditorInfo = DataDrivenShaderPlatformInfoEditorOnlyInfos[Index];
	FText FriendlyName;
	FTextStringHelper::ReadFromBuffer(*GetSectionString(Section, FName("FriendlyName")), FriendlyName);
	EditorInfo.DeviceProfileToFriendlyName.FindOrAdd(NAME_None) = FriendlyName;

	EditorInfo.bCanUsePreviewPlatformForMaterialValidation = GetSectionBool(Section, "bCanUsePreviewPlatformForMaterialValidation", false);
#endif
}

void FGenericDataDrivenShaderPlatformInfo::Initialize()
{
	static bool bInitialized = false;
	if (bInitialized)
	{
		return;
	}
	PlatformNameToShaderPlatformMap.Empty();

	// look for the standard DataDriven ini files
	int32 NumDDInfoFiles = FDataDrivenPlatformInfoRegistry::GetNumDataDrivenIniFiles();
	int32 CustomShaderPlatform = EShaderPlatform::SP_CUSTOM_PLATFORM_FIRST;

	struct PlatformInfoAndPlatformEnum
	{
		FGenericDataDrivenShaderPlatformInfo Info;
		EShaderPlatform ShaderPlatform;
	};

	for (int32 Index = 0; Index < NumDDInfoFiles; Index++)
	{
		FConfigFile IniFile;
		FString PlatformName;

		FDataDrivenPlatformInfoRegistry::LoadDataDrivenIniFile(Index, IniFile, PlatformName);

		// now walk over the file, looking for ShaderPlatformInfo sections
		for (const TPair<FString, FConfigSection>& Section : AsConst(IniFile))
		{
			if (Section.Key.StartsWith(TEXT("ShaderPlatform ")))
			{
				const FString& SectionName = Section.Key;
				const FConfigSection& SectionSettings = Section.Value;

				// get enum value for the string name
				const EShaderPlatform ShaderPlatform = ParseShaderPlatform(*SectionName.Mid(15));
				if (ShaderPlatform == SP_NumPlatforms)
				{
#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA
					const bool bIsEnabled = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName).bEnabledForUse;
#else
					const bool bIsEnabled = true;
#endif
					UE_CLOGF(bIsEnabled, LogRHI, Warning, "Found an unknown shader platform %ls in a DataDriven ini file", *SectionName.Mid(15));
					continue;
				}

				// at this point, we can start pulling information out
				Infos[ShaderPlatform].Name = *SectionName.Mid(15);
				Infos[ShaderPlatform].PlatformName = FName(*PlatformName);
				PlatformNameToShaderPlatformMap.FindOrAdd(Infos[ShaderPlatform].Name) = ShaderPlatform;
				ParseDataDrivenShaderInfo(SectionSettings, ShaderPlatform);
				Infos[ShaderPlatform].bContainsValidPlatformInfo = true;

#if WITH_EDITOR
				if (!FParse::Param(FCommandLine::Get(), TEXT("NoPreviewPlatforms")))
				{
					const TArray<FPreviewPlatformMenuItem>* Items = FDataDrivenPlatformInfoRegistry::GetMenuItemForShaderPlatformName(Infos[ShaderPlatform].Name);
					if (!Items)
					{
						continue;
					}

					const EShaderPlatform PreviewShaderPlatform = EShaderPlatform(CustomShaderPlatform++);
					ParseDataDrivenShaderInfo(SectionSettings, PreviewShaderPlatform);
					FGenericDataDrivenShaderPlatformInfo& PreviewInfo = Infos[PreviewShaderPlatform];
					FDataDrivenShaderPlatformInfoEditorOnly& PreviewEditorInfo = DataDrivenShaderPlatformInfoEditorOnlyInfos[PreviewShaderPlatform];
					PreviewEditorInfo.PreviewShaderPlatformParent = ShaderPlatform;

					for (bool bFirst = true; const FPreviewPlatformMenuItem& Item : *Items)
					{
						if (bFirst)
						{
							PreviewInfo.Name = Item.PreviewShaderPlatformName;
							PreviewInfo.PlatformName = Infos[ShaderPlatform].PlatformName;
							PreviewInfo.bIsPreviewPlatform = true;
							PreviewInfo.bContainsValidPlatformInfo = true;
							ERHIFeatureLevel::Type PreviewFeatureLevel = ERHIFeatureLevel::Num;
							if (GetFeatureLevelFromName(Item.PreviewFeatureLevelName, PreviewFeatureLevel))
							{
								PreviewInfo.MaxFeatureLevel = PreviewFeatureLevel;
							}

							PlatformNameToShaderPlatformMap.FindOrAdd(PreviewInfo.Name) = PreviewShaderPlatform;
							bFirst = false;
						}
						
						if (!Item.OptionalFriendlyNameOverride.IsEmpty())
						{
							PreviewEditorInfo.DeviceProfileToFriendlyName.FindOrAdd(Item.DeviceProfileName) = Item.OptionalFriendlyNameOverride;
						}
						else
						{
							PreviewEditorInfo.DeviceProfileToFriendlyName.FindOrAdd(Item.DeviceProfileName) = GetFriendlyName(PreviewShaderPlatform);
						}
					}
				}
#endif
			}
#if WITH_EDITOR
			else if (Section.Key.StartsWith(TEXT("PreviewShaderPlatform ")))
			{
				const FString& SectionName = Section.Key;
				const FConfigSection& SectionSettings = Section.Value;

				// get enum value for the string name
				const EShaderPlatform ShaderPlatform = EShaderPlatform(CustomShaderPlatform++);
				FGenericDataDrivenShaderPlatformInfo& Info = Infos[ShaderPlatform];

				const FString ParentShaderPlatformName = GetSectionString(SectionSettings, "ParentShaderPlatform");
				if (!ParentShaderPlatformName.IsEmpty())
				{
					const EShaderPlatform ParentShaderPlatform = ParseShaderPlatform(*ParentShaderPlatformName);
					
					if (IsValid(ParentShaderPlatform))
					{
						const FGenericDataDrivenShaderPlatformInfo& ParentInfo = Infos[ParentShaderPlatform];
						Info = ParentInfo;
					}
				}

				Info.Name = *SectionName.Mid(22);

				ParseDataDrivenShaderInfo(SectionSettings, ShaderPlatform);
				Info.bContainsValidPlatformInfo = true;
				Info.bIsPreviewPlatform = true;

				ERHIFeatureLevel::Type PreviewFeatureLevel = ERHIFeatureLevel::Num;
				if (GetFeatureLevelFromName(GetSectionString(SectionSettings, "PreviewFeatureLevel"), PreviewFeatureLevel))
				{
					Info.MaxFeatureLevel = PreviewFeatureLevel;
				}

				PlatformNameToShaderPlatformMap.FindOrAdd(Info.Name) = ShaderPlatform;

				DataDrivenShaderPlatformInfoEditorOnlyInfos[ShaderPlatform].PreviewShaderPlatformParent = ShaderPlatform;

				const TArray<FPreviewPlatformMenuItem>* Items = FDataDrivenPlatformInfoRegistry::GetMenuItemForShaderPlatformName(Info.Name);
				if (!Items)
				{
					continue;
				}

				for (const FPreviewPlatformMenuItem & Item : *Items)
				{
					FDataDrivenShaderPlatformInfoEditorOnly& PreviewEditorInfo = DataDrivenShaderPlatformInfoEditorOnlyInfos[ShaderPlatform];
					if (!Item.OptionalFriendlyNameOverride.IsEmpty())
					{
						PreviewEditorInfo.DeviceProfileToFriendlyName.FindOrAdd(Item.DeviceProfileName) = Item.OptionalFriendlyNameOverride;
					}
					else
					{
						PreviewEditorInfo.DeviceProfileToFriendlyName.FindOrAdd(Item.DeviceProfileName) = GetFriendlyName(ShaderPlatform);
					}
				}
			}
#endif
		}
	}
	bInitialized = true;
}

#if WITH_EDITOR
void FGenericDataDrivenShaderPlatformInfo::UpdatePreviewPlatforms()
{
	for (int32 PlatformIndex=0; PlatformIndex < SP_NumPlatforms; PlatformIndex++)
	{
		const EShaderPlatform PreviewPlatform = EShaderPlatform(PlatformIndex);
		if (IsValid(PreviewPlatform) && GetIsPreviewPlatform(PreviewPlatform))
		{
			const ERHIFeatureLevel::Type PreviewFeatureLevel = Infos[PreviewPlatform].MaxFeatureLevel;
			const EShaderPlatform RuntimePlatform = GRHIGlobals.ShaderPlatformForFeatureLevel[PreviewFeatureLevel];

			if (RuntimePlatform < SP_NumPlatforms)
			{
				FGenericDataDrivenShaderPlatformInfo& PreviewInfo = Infos[PreviewPlatform];
				const FGenericDataDrivenShaderPlatformInfo& RuntimeInfo = Infos[RuntimePlatform];

				FDataDrivenShaderPlatformInfoEditorOnly& PreviewEditorInfo = DataDrivenShaderPlatformInfoEditorOnlyInfos[PreviewPlatform];
				PreviewEditorInfo.RuntimePlatform = RuntimePlatform;

#define PREVIEW_USE_RUNTIME_VALUE(SettingName) \
	PreviewInfo.SettingName = RuntimeInfo.SettingName

#define PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(SettingName) \
	PreviewInfo.SettingName &= RuntimeInfo.SettingName

#define PREVIEW_FORCE_SETTING(SettingName, Value) \
	PreviewInfo.SettingName = (Value)

#define PREVIEW_FORCE_DISABLE(SettingName) \
	PREVIEW_FORCE_SETTING(SettingName, false)

				// Always inherit these core settings from the preview
				PREVIEW_USE_RUNTIME_VALUE(ShaderFormat);
				PREVIEW_USE_RUNTIME_VALUE(Language);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsDxc);
				PREVIEW_USE_RUNTIME_VALUE(bIsSPIRV);

				// Editor is always PC, never console and always supports debug view shaders
				PREVIEW_FORCE_SETTING(bIsPC, true);
				PREVIEW_FORCE_SETTING(bSupportsDebugViewShaders, true);
				PREVIEW_FORCE_SETTING(bIsConsole, false);

				// Support for stereo features requires extra consideration. The editor may not use the same technique as the preview platform,
				// particularly MobileMultiView may be substituted by a fallback path. In order to avoid inundating real mobile platforms
				// with the properties needed for the desktop MMV fallback path, override them here with the editor ones to make MMV preview possible
				if (PreviewInfo.bSupportsMobileMultiView && !RuntimeInfo.bSupportsMobileMultiView)
				{
					PREVIEW_USE_RUNTIME_VALUE(bSupportsInstancedStereo);
					PREVIEW_USE_RUNTIME_VALUE(bSupportsVertexShaderLayer);
				}
				else
				{
					PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bSupportsInstancedStereo);
					PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bSupportsVertexShaderLayer);
				}

				// Settings that should be kept true if the runtime also supports it.
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bSupportsNanite);
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bSupportsLumenGI);
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bSupportsPrimitiveShaders);
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bSupportsUInt64ImageAtomics);
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bSupportsGen5TemporalAA);
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bSupportsMeshShadersTier0);
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bSupportsMeshShadersTier1);
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bStripNaniteVertexShaders);
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(bSupportsMobileMultiView);
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(SupportsBarycentricsIntrinsics);
				PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(SupportsBarycentricsSemantic);

				// Settings that need to match the runtime
				PREVIEW_USE_RUNTIME_VALUE(bSupportsSpecializationConstants);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsGPUScene);
				PREVIEW_USE_RUNTIME_VALUE(MaxMeshShaderThreadGroupSize);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsSceneDataCompressedTransforms);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsVertexShaderSRVs);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsVertexShaderUAVs);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsTypedBufferSRVs);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsManualVertexFetch);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsRealTypes);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsUniformBufferObjects);
				PREVIEW_USE_RUNTIME_VALUE(bRequiresBindfulUtilityShaders);
				PREVIEW_USE_RUNTIME_VALUE(EnablesHLSL2021ByDefault);

				PREVIEW_USE_RUNTIME_VALUE(bSupportsInlineRayTracing);
				PREVIEW_USE_RUNTIME_VALUE(bInlineRayTracingRequiresBindless);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsRayTracingShaders);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsBindless);
				PREVIEW_USE_RUNTIME_VALUE(bSupportsDescriptorRange);
				PREVIEW_USE_RUNTIME_VALUE(StaticShaderBindingLayoutSupport);

				PREVIEW_USE_RUNTIME_VALUE(bSupportsRayTracingClusterOps);
				
				// Settings that will never be supported in preview
				PREVIEW_FORCE_DISABLE(bSupportsShaderRootConstants);
				PREVIEW_FORCE_DISABLE(bSupportsShaderBundleDispatch);
				PREVIEW_FORCE_DISABLE(bSupportsRenderTargetWriteMask);
				PREVIEW_FORCE_DISABLE(bSupportsIntrinsicWaveOnce);
				PREVIEW_FORCE_DISABLE(bSupportsDOFHybridScattering);
				PREVIEW_FORCE_DISABLE(bSupports4ComponentUAVReadWrite);

				// Metals bindless and bSupportsSceneDataCompressedTransforms need to match the max shader platform
				if (IsMetalPlatform(GMaxRHIShaderPlatform))
				{
					PREVIEW_FORCE_SETTING(bSupportsBindless, Infos[GMaxRHIShaderPlatform].bSupportsBindless);
					PREVIEW_FORCE_SETTING(bSupportsSceneDataCompressedTransforms, Infos[GMaxRHIShaderPlatform].bSupportsSceneDataCompressedTransforms);
				}
				
				// Make sure we're marked valid
				PreviewInfo.bContainsValidPlatformInfo = true;

				// Seeing as we are merging the two shader platforms merge the hash key as well, this way
				// any changes in the editor feature level shader platform will dirty the preview key.
				PreviewInfo.ShaderPropertiesHash = HashCombine(PreviewInfo.ShaderPropertiesHash, RuntimeInfo.ShaderPropertiesHash);

#undef PREVIEW_FORCE_DISABLE
#undef PREVIEW_FORCE_SETTING
#undef PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED
#undef PREVIEW_USE_RUNTIME_VALUE
			}
		}
	}
}

FText FGenericDataDrivenShaderPlatformInfo::GetFriendlyName(EShaderPlatform Platform, FName DeviceProfileName)
{
	if (IsRunningCommandlet() || GUsingNullRHI)
	{
		return FText();
	}
	check(IsValid(Platform));
	
	TMap<FName, FText>& DeviceProfileToFriendlyName = DataDrivenShaderPlatformInfoEditorOnlyInfos[Platform].DeviceProfileToFriendlyName;
	if (FText* FriendlyName = DeviceProfileToFriendlyName.Find(DeviceProfileName))
	{
		return *FriendlyName;
	}

	return FText::FromName(DeviceProfileName);
}

const EShaderPlatform FGenericDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(EShaderPlatform Platform)
{
	check(IsValid(Platform));
	return DataDrivenShaderPlatformInfoEditorOnlyInfos[Platform].PreviewShaderPlatformParent;
}

const EShaderPlatform FGenericDataDrivenShaderPlatformInfo::GetPreviewRuntimePlatform(EShaderPlatform Platform)
{
	check(IsValid(Platform));
	check(GetIsPreviewPlatform(Platform));
	return DataDrivenShaderPlatformInfoEditorOnlyInfos[Platform].RuntimePlatform;
}

const bool FGenericDataDrivenShaderPlatformInfo::CanUseForMaterialValidation(EShaderPlatform Platform)
{
	check(IsValid(Platform));
	return DataDrivenShaderPlatformInfoEditorOnlyInfos[Platform].bCanUsePreviewPlatformForMaterialValidation || !Infos[Platform].bIsPreviewPlatform;
}

FName FGenericDataDrivenShaderPlatformInfo::GetPreferredShaderCodeFormat(EShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(Platform) ? GRHIGlobals.PreferredPreviewShaderCodeFormat : NAME_None;
}

#endif  // WITH_EDITOR

const EShaderPlatform FGenericDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(const FName ShaderPlatformName)
{
	if (EShaderPlatform* ShaderPlatform = PlatformNameToShaderPlatformMap.Find(ShaderPlatformName))
	{
		return *ShaderPlatform;
	}
	return SP_NumPlatforms;
}
