// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DynamicRHI.cpp: Dynamically bound Render Hardware Interface implementation.
=============================================================================*/

#include "DynamicRHI.h"
#include "Misc/MessageDialog.h"
#include "Experimental/Containers/HazardPointer.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "ProfilingDebugging/BootProfiling.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "PipelineStateCache.h"
#include "RHI.h"
#include "RHIFwd.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHICommandList.h"
#include "RHIStrings.h"
#include "RHITextureReference.h"
#include "RHIAllocators.h"
#include "RenderTracing.h"

#ifndef PLATFORM_ALLOW_NULL_RHI
	#define PLATFORM_ALLOW_NULL_RHI		0
#endif

// Globals.
FDynamicRHI* GDynamicRHI = NULL;
RHIGetGPUUsageType RHIGetGPUUsage = nullptr;
bool bDriverDenylistMessageShown = false;
bool GDynamicRHIIgnoredSM5Preference = false;
FString GDynamicRHIIgnoredPreferredRHI;

static int32 GWarnOfBadDrivers = true;
static FAutoConsoleVariableRef CVarWarnOfBadDrivers(
	TEXT("r.WarnOfBadDrivers"),
	GWarnOfBadDrivers,
	TEXT("Check the current GPU driver on engine startup, warn the user about issues and suggest a specific version.\n")
	TEXT("The driver denylist is used to check for bad drivers according to their release date and/or driver versions.\n")
	TEXT(" 0: off\n")
	TEXT(" 1: check the driver and display a pop-up message if the driver is denylisted (default)"),
	ECVF_RenderThreadSafe
	);

static bool GBadDriverWarningIsFatal = false;
static FAutoConsoleVariableRef CVarBadDriverWarningIsFatal(
	TEXT("r.BadDriverWarningIsFatal"),
	GBadDriverWarningIsFatal,
	TEXT("If non-zero, trigger a fatal error if a denylisted driver is detected.\n")
	TEXT("For the fatal error to occur, r.WarnOfBadDrivers must be non-zero.\n")
	TEXT(" 0: off (default)\n")
	TEXT(" 1: a fatal error occurs after the out-of-date driver message is dismissed (non-Shipping only)\n"),
	ECVF_RenderThreadSafe);

void InitNullRHI()
{
	// Use the null RHI if it was specified on the command line, or if a commandlet is running.
	IDynamicRHIModule* DynamicRHIModule = &FModuleManager::LoadModuleChecked<IDynamicRHIModule>(TEXT("NullDrv"));
	// Create the dynamic RHI.
	if ((DynamicRHIModule == 0) || !DynamicRHIModule->IsSupported())
	{
		FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("DynamicRHI", "NullDrvFailure", "NullDrv failure?"));
		FPlatformMisc::RequestExit(true, TEXT("InitNullRHI"));
	}

	GDynamicRHI = DynamicRHIModule->CreateRHI();
	GDynamicRHI->Init();

	GUsingNullRHI = true;
	GRHISupportsTextureStreaming = false;

	// Update the crash context analytics
	FGenericCrashContext::SetEngineData(TEXT("RHI.RHIName"), TEXT("NullRHI"));
}

#if PLATFORM_WINDOWS || PLATFORM_UNIX
void RHIDetectAndWarnOfBadDrivers(bool bHasEditorToken)
{
	// Don't show another prompt if we already did during this session.
	if (bDriverDenylistMessageShown)
	{
		return;
	}

	if (GRHIVendorId == 0)
	{
		UE_LOGF(LogRHI, Log, "Skipping driver check, no vendor ID set.");
		return;
	}

	UE_LOGF(LogRHI, Log, "Performing driver check for denylisted drivers.");

	FGPUDriverInfo DriverInfo;
	DriverInfo.VendorId = GRHIVendorId;
	DriverInfo.AdapterGeneration = GRHIGlobals.GpuInfo.AdapterGeneration;
	DriverInfo.DeviceDescription = GRHIAdapterName;
	DriverInfo.ProviderName = TEXT("Unknown");
	DriverInfo.InternalDriverVersion = GRHIAdapterInternalDriverVersion;
	DriverInfo.UserDriverVersion = GRHIAdapterUserDriverVersion;
	DriverInfo.DriverDate = GRHIAdapterDriverDate;
	DriverInfo.RHIName = GDynamicRHI ? GDynamicRHI->GetName() : FString();

	// Pre-GCN GPUs usually don't support updating to latest driver
	// But it is unclear what is the latest version supported as it varies from card to card
	// So just don't complain if pre-gcn
	if (DriverInfo.IsValid() && !GRHIDeviceIsAMDPreGCNArchitecture)
	{
		TArray<FString> DeviceCanUpdateDriverList;
		GConfig->GetArray(TEXT("Devices"), TEXT("DeviceCanUpdateDriverList"), DeviceCanUpdateDriverList, GHardwareIni);

		bool bVendorHasEntries = false;
		bool bDeviceCanUpdateDriver = false;
		for (const FString& DeviceCanUpdateDriverString : DeviceCanUpdateDriverList)
		{
			const TCHAR* Line = *DeviceCanUpdateDriverString;

			FString VendorId;
			FParse::Value(Line + 1, TEXT("VendorId="), VendorId);
			uint32 VendorIdInt = FParse::HexNumber(*VendorId);

			FString DeviceId;
			FParse::Value(Line + 1, TEXT("DeviceId="), DeviceId);
			uint32 DeviceIdInt = FParse::HexNumber(*DeviceId);

			bVendorHasEntries |= DriverInfo.VendorId && DriverInfo.VendorId == VendorIdInt;

			if (DriverInfo.VendorId && GRHIDeviceId &&
				DriverInfo.VendorId == VendorIdInt && GRHIDeviceId == DeviceIdInt)
			{
				int32 DriverGeneration = 0;
				if (FParse::Value(Line + 1, TEXT("DriverGeneration="), DriverGeneration) && (DriverGeneration > 0))
				{
					// Allow config to override vendor API queries.
					DriverInfo.AdapterGeneration = DriverGeneration;
					UE_LOGF(LogRHI, Log, "Overriding AdapterGeneration %u with DriverGeneration %d", GRHIGlobals.GpuInfo.AdapterGeneration, DriverGeneration);
					GRHIGlobals.GpuInfo.AdapterGeneration = DriverGeneration;
				}

				bDeviceCanUpdateDriver = true;
				break;
			}
		}

		FGPUDriverHelper DetectedGPUHardware(DriverInfo);
		TOptional<FDriverDenyListEntry> DenyListEntry = DetectedGPUHardware.FindDriverDenyListEntry();

		GRHIAdapterDriverOnDenyList = DenyListEntry.IsSet() && DenyListEntry->IsValid();
		FGenericCrashContext::SetEngineData(TEXT("RHI.DriverDenylisted"), GRHIAdapterDriverOnDenyList ? TEXT("true") : TEXT("false"));

		if(!GRHIAdapterDriverOnDenyList)
		{
			return;
		}

		const FString SuggestedDriverString = DetectedGPUHardware.FindSuggestedDriverVersion();
		UE_LOGF(LogRHI, Warning, "Out of date driver found. Using: '%ls' Suggested: '%ls'", *DriverInfo.UserDriverVersion, (SuggestedDriverString.IsEmpty() ? TEXT("Unknown") : *SuggestedDriverString));

		// Only alert users who are capable of updating their driver. Assume vendors with an empty list can always update.
		// The warning message can also be suppressed with r.WarnOfBadDrivers=0.
		bool bShowPrompt = bDeviceCanUpdateDriver || !bVendorHasEntries;
		bShowPrompt = bShowPrompt && !FApp::IsUnattended() && GWarnOfBadDrivers != 0;

		if (bShowPrompt)
		{
			UE_LOGF(LogRHI, Log, "Current driver is denylisted.");

			// Note: we don't localize the vendor's name.
			FString VendorString = DriverInfo.ProviderName;
			FText HyperlinkText;
			if (DriverInfo.IsNVIDIA())
			{
				VendorString = TEXT("NVIDIA");
				HyperlinkText = NSLOCTEXT("MessageDialog", "DriverDownloadLinkNVIDIA", "https://www.nvidia.com/download/index.aspx");
			}
			else if (DriverInfo.IsAMD())
			{
				VendorString = TEXT("AMD");
				HyperlinkText = NSLOCTEXT("MessageDialog", "DriverDownloadLinkAMD", "https://www.amd.com/en/support");
			}
			else if (DriverInfo.IsIntel())
			{
				VendorString = TEXT("Intel");
				HyperlinkText = NSLOCTEXT("MessageDialog", "DriverDownloadLinkIntel", "https://downloadcenter.intel.com/product/80939/Graphics");
			}
			else if (DriverInfo.IsQualcomm())
			{
				VendorString = TEXT("Qualcomm");
				HyperlinkText = NSLOCTEXT("MessageDialog", "DriverDownloadLinkQualcomm", "https://softwarecenter.qualcomm.com/catalog/item/Windows_Graphics_Driver");
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("AdapterName"), FText::FromString(DriverInfo.DeviceDescription));
			Args.Add(TEXT("Vendor"), FText::FromString(VendorString));
			Args.Add(TEXT("Hyperlink"), HyperlinkText);
			Args.Add(TEXT("InstalledVer"), FText::FromString(DriverInfo.UserDriverVersion));

			// Find the best driver version to recommend.
			if (!SuggestedDriverString.IsEmpty())
			{
				// Suggest the latest too, if not denylisted.
				if (!DenyListEntry->AppliesToLatestDrivers())
				{
					Args.Add(TEXT("RecommendedVer"), FText::Format(NSLOCTEXT("MessageDialog", "SuggestedDriverOrLatest", "{0} or latest driver available"), FText::FromString(SuggestedDriverString)));
				}
				else
				{
					Args.Add(TEXT("RecommendedVer"), FText::FromString(SuggestedDriverString));
				}
			}
			else
			{
				ensureMsgf(!DenyListEntry->AppliesToLatestDrivers(), TEXT("Latest drivers are denylisted but no recommended driver driver has been provided"));
				Args.Add(TEXT("RecommendedVer"), NSLOCTEXT("MessageDialog", "LatestDriver", "latest driver available"));
			}

			FText LocalizedMsg;
			if (DenyListEntry->RHINameConstraint)
			{
				Args.Add(TEXT("RHI"), FText::FromString(*DenyListEntry->RHINameConstraint));
				LocalizedMsg = FText::Format(NSLOCTEXT("MessageDialog", "VideoCardDriverRHIIssueReport", "The installed version of the {Vendor} graphics driver has known issues in {RHI}.\nPlease install the recommended driver version or switch to a different rendering API.\n\nWould you like to visit the following URL to download the driver?\n\n{Hyperlink}\n\n{AdapterName}\nInstalled: {InstalledVer}\nRecommended: {RecommendedVer}"), Args);
			}
			else
			{
				LocalizedMsg = FText::Format(NSLOCTEXT("MessageDialog", "VideoCardDriverIssueReport", "The installed version of the {Vendor} graphics driver has known issues.\nPlease install the recommended driver version.\n\nWould you like to visit the following URL to download the driver?\n\n{Hyperlink}\n\n{AdapterName}\nInstalled: {InstalledVer}\nRecommended: {RecommendedVer}"), Args);
			}


			FText Title = NSLOCTEXT("MessageDialog", "TitleVideoCardDriverIssue", "WARNING: Known issues with graphics driver");
			const double DialogStartTime = FPlatformTime::Seconds();
			EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo, LocalizedMsg, Title);
			FBootProfiling::AddBlockingDialogDuration(FPlatformTime::Seconds() - DialogStartTime);
			if (Response == EAppReturnType::Yes)
			{
				FPlatformProcess::LaunchURL(*HyperlinkText.ToString(), nullptr, nullptr);
			}
#if !UE_BUILD_SHIPPING
			if (GBadDriverWarningIsFatal)
			{
				// Force a fatal error depending on CVar.
				UE_LOGF(LogRHI, Fatal, "Fatal crash requested when graphics drivers are out of date.\n"
					"To prevent this crash, please update drivers.");
			}
#endif
			bDriverDenylistMessageShown = true;
		}
		else
		{
			UE_LOGF(LogRHI, Warning, "Running with bad GPU drivers but warning dialog will not be shown: bDeviceCanUpdateDriver=%d, VendorHasEntries=%d, IsUnattended=%d, r.WarnOfBadDrivers=%d", bDeviceCanUpdateDriver, bVendorHasEntries, FApp::IsUnattended(), GWarnOfBadDrivers);
		}
	}
}
#elif PLATFORM_MAC
void RHIDetectAndWarnOfBadDrivers(bool bHasEditorToken)
{
	// Don't show another prompt if we already did during this session.
	if (bDriverDenylistMessageShown)
	{
		return;
	}

	if (!GWarnOfBadDrivers || GRHIVendorId == 0 || bHasEditorToken || FApp::IsUnattended())
	{
		return;
	}

	if (FPlatformMisc::MacOSXVersionCompare(12, 0, 0) < 0)
	{
		// this message can be suppressed with r.WarnOfBadDrivers=0
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok,
								 *NSLOCTEXT("MessageDialog", "UpdateMacOSX_Body", "Please update to the latest version of macOS for best performance and stability.").ToString(),
								 *NSLOCTEXT("MessageDialog", "UpdateMacOSX_Title", "Update macOS").ToString());
		
#if !UE_BUILD_SHIPPING
		if (GBadDriverWarningIsFatal)
		{
			// Force a fatal error depending on CVar
		UE_LOGF(LogRHI, Fatal, "Fatal crash requested when graphics drivers are out of date.\n"
			"To prevent this crash, please update macOS.");
		}
#endif
		bDriverDenylistMessageShown = true;
	}
}
#endif // PLATFORM_WINDOWS

void RHIInit(bool bHasEditorToken)
{
	if (!GDynamicRHI)
	{
#if RHI_ENABLE_RESOURCE_INFO
		FRHIResource::StartTrackingAllResources();
#endif

		// read in any data driven shader platform info structures we can find
		FGenericDataDrivenShaderPlatformInfo::Initialize();

		GRHICommandList.LatchBypass(); // read commandline for bypass flag

		if (!FApp::CanEverRender())
		{
			InitNullRHI();
		}
		else
		{
			LLM_SCOPE(ELLMTag::RHIMisc);

			GDynamicRHI = PlatformCreateDynamicRHI();
			if (GDynamicRHI)
			{
#if PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_UNIX
				// Get driver version. Creating GDynamicRHI is expected to set GRHIAdapterName.
				FGPUDriverInfo GPUDriverInfo = FPlatformMisc::GetGPUDriverInfo(GRHIAdapterName);
				if (GPUDriverInfo.IsValid())
				{
					// GetGPUDriverInfo is not implemented on Linux, so it returns an invalid driver info object. However, the FVulkanDynamicRHI constructor
					// sets these values on that platform, so we'll still have data we can log.
					GRHIAdapterUserDriverVersion = GPUDriverInfo.UserDriverVersion;
					GRHIAdapterInternalDriverVersion = GPUDriverInfo.InternalDriverVersion;
					GRHIAdapterDriverDate = GPUDriverInfo.DriverDate;
				}

				UE_LOGF(LogRHI, Log, "RHI Adapter Info:");
				UE_LOGF(LogRHI, Log, "            Name: %ls", *GRHIAdapterName);
				UE_LOGF(LogRHI, Log, "  Driver Version: %ls (internal:%ls, unified:%ls)", *GRHIAdapterUserDriverVersion, *GRHIAdapterInternalDriverVersion, *GPUDriverInfo.GetUnifiedDriverVersion());
				UE_LOGF(LogRHI, Log, "     Driver Date: %ls", *GRHIAdapterDriverDate);

				RHIDetectAndWarnOfBadDrivers(bHasEditorToken);
#endif

				GDynamicRHI->Init();

				// Validation of contexts.
				FRHICommandListImmediate::Get().GetContext();
				check(GIsRHIInitialized);

				// Set default GPU mask to all GPUs. This is necessary to ensure that any commands
				// that create and initialize resources are executed on all GPUs. Scene rendering
				// will restrict itself to a subset of GPUs as needed.
				FRHICommandListImmediate::Get().SetGPUMask(FRHIGPUMask::All());

				FString FeatureLevelString;
				GetFeatureLevelName(GMaxRHIFeatureLevel, FeatureLevelString);

				if (bHasEditorToken && GMaxRHIFeatureLevel < ERHIFeatureLevel::SM5)
				{
					FString ShaderPlatformString = LegacyShaderPlatformToShaderFormat(GetFeatureLevelShaderPlatform(GMaxRHIFeatureLevel)).ToString();
					FString Error = FString::Printf(TEXT("A Feature Level 5 video card is required to run the editor.\nAvailableFeatureLevel = %s, ShaderPlatform = %s"), *FeatureLevelString, *ShaderPlatformString);
					FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(Error));
					FPlatformMisc::RequestExit(true, TEXT("RHIInit"));
				}

				// Update the crash context analytics
				FGenericCrashContext::SetEngineData(TEXT("RHI.RHIName"), GDynamicRHI ? (GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1 ? FString(GDynamicRHI->GetName()) + TEXT("_ES31") : GDynamicRHI->GetName()) : TEXT("Unknown"));
				FGenericCrashContext::SetEngineData(TEXT("RHI.AdapterName"), GRHIAdapterName);
				FGenericCrashContext::SetEngineData(TEXT("RHI.UserDriverVersion"), GRHIAdapterUserDriverVersion);
				FGenericCrashContext::SetEngineData(TEXT("RHI.InternalDriverVersion"), GRHIAdapterInternalDriverVersion);
				FGenericCrashContext::SetEngineData(TEXT("RHI.DriverDate"), GRHIAdapterDriverDate);
				FGenericCrashContext::SetEngineData(TEXT("RHI.FeatureLevel"), FeatureLevelString);
				FGenericCrashContext::SetEngineData(TEXT("RHI.IgnoredSM5Preference"), GDynamicRHIIgnoredSM5Preference ? TEXT("True") : TEXT("False"));
				FGenericCrashContext::SetEngineData(TEXT("RHI.IgnoredPreferredRHI"), GDynamicRHIIgnoredPreferredRHI);
				FGenericCrashContext::SetEngineData(TEXT("RHI.GPUVendor"), RHIVendorIdToString());
				FGenericCrashContext::SetEngineData(TEXT("RHI.DeviceId"), FString::Printf(TEXT("%04X"), GRHIDeviceId));
			}
#if PLATFORM_ALLOW_NULL_RHI
			else
			{
				// If the platform supports doing so, fall back to the NULL RHI on failure
				InitNullRHI();
			}
#endif
		}

		check(GDynamicRHI);
		IF_RENDER_TRACING_ENABLED(RenderTracing::Initialize());
	}

#if WITH_EDITOR
	FGenericDataDrivenShaderPlatformInfo::UpdatePreviewPlatforms();

	// add an ability to override the shader platform, intended for preview platforms only
	FString ShaderPlatformName;
	if (FParse::Value(FCommandLine::Get(), TEXT("OverrideSP="), ShaderPlatformName) && !ShaderPlatformName.IsEmpty())
	{
		EShaderPlatform OverrideShaderPlatform = FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(*ShaderPlatformName);
		if (OverrideShaderPlatform != SP_NumPlatforms)
		{
			// only allow to override to preview shader platform (to avoid complications), and make sure the SP matches current feature level
			if (FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(OverrideShaderPlatform))
			{
				if (GetMaxSupportedFeatureLevel(OverrideShaderPlatform) == GMaxRHIFeatureLevel)
				{
					UE_LOGF(LogRHI, Log, "Overriding shader platform to use from %ls to %ls",
						*LexToString(GMaxRHIShaderPlatform, false),
						*LexToString(OverrideShaderPlatform, false));

					GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel] = OverrideShaderPlatform;
					GMaxRHIShaderPlatform = OverrideShaderPlatform;
				}
				else
				{
					UE_LOGF(LogRHI, Log, "Cannot override to use shaderplatform %ls, its max feature level %ls does not match current max feature level %ls",
						*LexToString(OverrideShaderPlatform, false),
						*LexToString(GetMaxSupportedFeatureLevel(OverrideShaderPlatform)),
						*LexToString(GMaxRHIFeatureLevel)
						);
				}
			}
			else
			{
				UE_LOGF(LogRHI, Log, "Cannot override to use shader platform %ls, it is not a preview platform", *LexToString(OverrideShaderPlatform, false));
			}
		}
		else
		{
			UE_LOGF(LogRHI, Log, "Shader platform %ls is not a valid platform, cannot use it.", *LexToString(OverrideShaderPlatform, false));
		}
	}
#endif

	// Set CSV profiler metadata so every capture can see this
	// GDynamicRHI is checked above
	CSV_METADATA(TEXT("VerbatimRHIName"), GDynamicRHI->GetName());
	// there's already an unfortunate precedent to concoct RHIName for PCD3D_ES31 and such platforms
	CSV_METADATA(TEXT("RHIName"), GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1 ? *(FString(GDynamicRHI->GetName()) + TEXT("_ES31")) : GDynamicRHI->GetName());
	CSV_METADATA(TEXT("RHIFeatureLevel"), *LexToString(GMaxRHIFeatureLevel));
	CSV_METADATA(TEXT("ShaderPlatform"), *LexToString(GetFeatureLevelShaderPlatform(GMaxRHIFeatureLevel)));
	CSV_METADATA(TEXT("GPU"), *GRHIGlobals.GpuInfo.AdapterName.TrimStartAndEnd());
	CSV_METADATA(TEXT("GPUDriver"), *GRHIGlobals.GpuInfo.AdapterUserDriverVersion.TrimStartAndEnd());
}

void RHIPostInit(const TArray<uint32>& InPixelFormatByteWidth)
{
	check(GDynamicRHI);
	GDynamicRHI->InitPixelFormatInfo(InPixelFormatByteWidth);
	GDynamicRHI->PostInit();

#if PLATFORM_ANDROID
	// The Android HW window is locked during init to prevent it being destroyed asynch during app backgrounding
	// It is unlocked after the RHI is initialized as it is able to handle backgrounding.
	FAndroidMisc::UnlockAndroidWindow();
#endif
}

void RHIExit()
{
	if (!GUsingNullRHI && GDynamicRHI != NULL)
	{
		// Clean up all cached pipelines
		PipelineStateCache::Shutdown();

		// Flush any potential commands queued before we shut things down.
		FRHICommandListImmediate::Get().ImmediateFlush(EImmediateFlushType::FlushRHIThread);

		// Destruct the dynamic RHI.
		GDynamicRHI->Shutdown();
		delete GDynamicRHI;
		GDynamicRHI = NULL;

#if RHI_ENABLE_RESOURCE_INFO
		FRHIResource::StopTrackingAllResources();
#endif
	}
	else if (GUsingNullRHI)
	{
		// If we are using NullRHI flush the command list here in case somethings has been added to the command list during exit calls
		FRHICommandListImmediate::Get().ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
		FRHICommandListImmediate::Get().ImmediateFlush(EImmediateFlushType::FlushRHIThread);
	}

	GRHICommandList.CleanupGraphEvents();
}

FDynamicRHI::~FDynamicRHI() = default;

void FDynamicRHI::RHIEndFrame_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	ERHISubmitFlags Flags = ERHISubmitFlags::EndFrame;

	if (UE::RHI::GPUProfiler::ShouldProfileNextFrame())
	{
		EnumAddFlags(Flags, ERHISubmitFlags::ProfileNextFrame);
	}

	RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread, Flags);
}

// Default fallback; will not work for non-8-bit surfaces and it's extremely slow.
void FDynamicRHI::RHIReadSurfaceData(FRHITexture* Texture, FIntRect Rect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	TArray<FColor> TempData;
	RHIReadSurfaceData(Texture, Rect, TempData, InFlags);
	OutData.SetNumUninitialized(TempData.Num());
	for (int32 Index = 0; Index < TempData.Num(); ++Index)
	{
		OutData[Index] = TempData[Index].ReinterpretAsLinear();
	}
}

void FDynamicRHI::RHIReadSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags InFlags)
{
#if WITH_MGPU
	if (InFlags.GetGPUIndex() != 0)
	{
		unimplemented();
	}
	else
#endif
	{
		RHIReadSurfaceFloatData(Texture, Rect, OutData, InFlags.GetCubeFace(), InFlags.GetArrayIndex(), InFlags.GetMip());
	}
}

void FDynamicRHI::RHIRead3DSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags InFlags)
{
#if WITH_MGPU
	if (InFlags.GetGPUIndex() != 0)
	{
		unimplemented();
	}
	else
#endif
	{
		RHIRead3DSurfaceFloatData(Texture, Rect, ZMinMax, OutData);
	}
}

void FDynamicRHI::EnableIdealGPUCaptureOptions(bool bEnabled)
{
	UE_LOGF(LogRHI, Display, "Setting GPU Capture Options: %i", bEnabled ? 1 : 0);

	// Draw Events
	if (bEnabled != (GetEmitDrawEvents() != 0))
	{
		UE_LOGF(LogRHI, Display, "Toggling draw events: %i", bEnabled ? 1 : 0);
		SetEmitDrawEvents(bEnabled);
	}

	// Material Draw Events
	static IConsoleVariable* ShowMaterialDrawEventVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ShowMaterialDrawEvents"));
	if (ShowMaterialDrawEventVar)
	{
		if (bEnabled != (ShowMaterialDrawEventVar->GetInt() != 0))
		{
			UE_LOGF(LogRHI, Display, "Toggling showmaterialdrawevents: %i", bEnabled ? 1 : 0);
			ShowMaterialDrawEventVar->Set(bEnabled ? -1 : 0);
		}
	}

	// Full pass names in RDG
	static IConsoleVariable* RDGEvents = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RDG.Events"));
	if (RDGEvents)
	{
		if (bEnabled)
		{
			UE_LOGF(LogRHI, Display, "Enabling full RDG events (r.RDG.Events 3)");
			RDGEvents->Set(3, ECVF_SetByCode);
		}
		else
		{
			UE_LOGF(LogRHI, Display, "Resetting RDG events to default (r.RDG.Events)");
			RDGEvents->Unset(ECVF_SetByCode);
		}
	}
}

FTextureReferenceRHIRef FDynamicRHI::RHICreateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* InReferencedTexture)
{
	FRHITexture* ReferencedTexture = InReferencedTexture ? InReferencedTexture : FRHITextureReference::GetDefaultTexture();
	return new FRHITextureReference(ReferencedTexture);
}

void FDynamicRHI::RHIUpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITextureReference* TextureRef, FRHITexture* InReferencedTexture)
{
	// Workaround for a crash bug where FRHITextureReferences are deleted before this command is executed on the RHI thread.
	// Take a reference on the FRHITextureReference object to keep it alive.
	// @todo dev-pr - This should be refactored out when we eventually remove FRHITextureReference.
	TRefCountPtr<FRHITextureReference> Ref = TextureRef;

	RHICmdList.EnqueueLambda(TEXT("FDynamicRHI::RHIUpdateTextureReference"), [TextureRef = MoveTemp(Ref), InReferencedTexture](FRHICommandListBase&)
	{
		FRHITexture* ReferencedTexture = InReferencedTexture ? InReferencedTexture : FRHITextureReference::GetDefaultTexture();
		TextureRef->SetReferencedTexture(ReferencedTexture);
	});
	
	RHICmdList.RHIThreadFence(true);
}

void FDynamicRHI::RHIVirtualTextureSetFirstMipInMemory(FRHICommandListImmediate& RHICmdList, FRHITexture* TextureRHI, uint32 FirstMip)
{
	UE_LOGF(LogRHI, Fatal, "The current RHI does not implement support for virtually allocated textures.");
}

void FDynamicRHI::RHIVirtualTextureSetFirstMipVisible(FRHICommandListImmediate& RHICmdList, FRHITexture* TextureRHI, uint32 FirstMip)
{
	UE_LOGF(LogRHI, Fatal, "The current RHI does not implement support for virtually allocated textures.");
}

uint64 FDynamicRHI::RHIGetMinimumAlignmentForBufferBackedSRV(EPixelFormat Format)
{
	return GPixelFormats[Format].BlockBytes;
}

uint64 FDynamicRHI::RHIComputeStatePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	struct FHashKey
	{
		uint32 VertexDeclaration;
		uint32 VertexShader;
		uint32 VertexShaderSpecConsts;
		uint32 PixelShader;
		uint32 PixelShaderSpecConsts;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		uint32 GeometryShader;
		uint32 GeometryShaderSpecConsts;
#endif // PLATFORM_SUPPORTS_GEOMETRY_SHADERS
#if PLATFORM_SUPPORTS_MESH_SHADERS
		uint32 MeshShader;
		uint32 MeshShaderSpecConsts;
#endif // PLATFORM_SUPPORTS_MESH_SHADERS
		uint32 BlendState;
		uint32 RasterizerState;
		uint32 DepthStencilState;

		uint32 DrawShadingRate : 8;
		uint32 PrimitiveType : 8;
		uint32 bDepthBounds : 1;
		uint32 Unused : 15;
	} HashKey;
	 
	FMemory::Memzero(&HashKey, sizeof(FHashKey));

	HashKey.VertexDeclaration = Initializer.BoundShaderState.VertexDeclarationRHI ? Initializer.BoundShaderState.VertexDeclarationRHI->GetPrecachePSOHash() : 0;
	HashKey.VertexShader = Initializer.BoundShaderState.GetVertexShader() ? GetTypeHash(Initializer.BoundShaderState.GetVertexShader()->GetHash()) : 0;
	HashKey.VertexShaderSpecConsts = Initializer.BoundShaderState.VertexShaderSpecializationConstants.Num() ? GetArrayHash(Initializer.BoundShaderState.VertexShaderSpecializationConstants.GetData(), Initializer.BoundShaderState.VertexShaderSpecializationConstants.NumBytes()) : 0;
	HashKey.PixelShader = Initializer.BoundShaderState.GetPixelShader() ? GetTypeHash(Initializer.BoundShaderState.GetPixelShader()->GetHash()) : 0;
	HashKey.PixelShaderSpecConsts = Initializer.BoundShaderState.PixelShaderSpecializationConstants.Num() ? GetArrayHash(Initializer.BoundShaderState.PixelShaderSpecializationConstants.GetData(), Initializer.BoundShaderState.PixelShaderSpecializationConstants.NumBytes()) : 0;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	HashKey.GeometryShader = Initializer.BoundShaderState.GetGeometryShader() ? GetTypeHash(Initializer.BoundShaderState.GetGeometryShader()->GetHash()) : 0;
	HashKey.GeometryShaderSpecConsts = Initializer.BoundShaderState.GeometryShaderSpecializationConstants.Num() ? GetArrayHash(Initializer.BoundShaderState.GeometryShaderSpecializationConstants.GetData(), Initializer.BoundShaderState.GeometryShaderSpecializationConstants.NumBytes()) : 0;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
	HashKey.MeshShader = Initializer.BoundShaderState.GetMeshShader() ? GetTypeHash(Initializer.BoundShaderState.GetMeshShader()->GetHash()) : 0;
	HashKey.MeshShaderSpecConsts = Initializer.BoundShaderState.MeshShaderSpecializationConstants.Num() ? GetArrayHash(Initializer.BoundShaderState.MeshShaderSpecializationConstants.GetData(), Initializer.BoundShaderState.MeshShaderSpecializationConstants.NumBytes()) : 0;
#endif

	FBlendStateInitializerRHI BlendStateInitializerRHI;
	if (Initializer.BlendState && Initializer.BlendState->GetInitializer(BlendStateInitializerRHI))
	{
		HashKey.BlendState = GetTypeHash(BlendStateInitializerRHI);
	}
	FRasterizerStateInitializerRHI RasterizerStateInitializerRHI;
	if (Initializer.RasterizerState && Initializer.RasterizerState->GetInitializer(RasterizerStateInitializerRHI))
	{
		HashKey.RasterizerState = GetTypeHash(RasterizerStateInitializerRHI);
	}
	FDepthStencilStateInitializerRHI DepthStencilStateInitializerRHI;
	if (Initializer.DepthStencilState && Initializer.DepthStencilState->GetInitializer(DepthStencilStateInitializerRHI))
	{
		HashKey.DepthStencilState = GetTypeHash(DepthStencilStateInitializerRHI);
	}

	HashKey.DrawShadingRate = Initializer.ShadingRate;
	HashKey.PrimitiveType = Initializer.PrimitiveType;
	HashKey.bDepthBounds = Initializer.bDepthBounds;

	uint64 PrecachePSOHash = CityHash64((const char*)&HashKey, sizeof(FHashKey));

	return PrecachePSOHash;
}

uint64 FDynamicRHI::RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	uint64 StatePrecachePSOHash = Initializer.StatePrecachePSOHash;
	if (StatePrecachePSOHash == 0)
	{
		StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(Initializer);
	}

	checkf(StatePrecachePSOHash != 0, TEXT("Initializer should have a valid state precache PSO hash set when computing the full initializer PSO hash"));
	
	// All members which are not part of the state objects
	struct FNonStateHashKey
	{
		uint64							StatePrecachePSOHash;

		EPrimitiveType					PrimitiveType;
		uint32							RenderTargetsEnabled;
		FGraphicsPipelineStateInitializer::TRenderTargetFormats	RenderTargetFormats;
		FGraphicsPipelineStateInitializer::TRenderTargetFlags RenderTargetFlags;
		EPixelFormat					DepthStencilTargetFormat;
		ETextureCreateFlags				DepthStencilTargetFlag;
		ERenderTargetLoadAction			DepthTargetLoadAction;
		ERenderTargetStoreAction		DepthTargetStoreAction;
		ERenderTargetLoadAction			StencilTargetLoadAction;
		ERenderTargetStoreAction		StencilTargetStoreAction;
		FExclusiveDepthStencil			DepthStencilAccess;
		uint16							NumSamples;
		ESubpassHint					SubpassHint;
		uint8							SubpassIndex;
		EConservativeRasterization		ConservativeRasterization;
		bool							bDepthBounds;
		uint8							MultiViewCount;
		bool							bHasFragmentDensityAttachment;
		EVRSShadingRate					ShadingRate;
	} HashKey;

	FMemory::Memzero(&HashKey, sizeof(FNonStateHashKey));

	HashKey.StatePrecachePSOHash		= StatePrecachePSOHash;

	HashKey.PrimitiveType				= Initializer.PrimitiveType;
	HashKey.RenderTargetsEnabled		= Initializer.RenderTargetsEnabled;
	HashKey.RenderTargetFormats			= Initializer.RenderTargetFormats;
	HashKey.RenderTargetFlags			= Initializer.RenderTargetFlags;
	HashKey.DepthStencilTargetFormat	= Initializer.DepthStencilTargetFormat;
	HashKey.DepthStencilTargetFlag		= Initializer.DepthStencilTargetFlag;
	HashKey.DepthTargetLoadAction		= Initializer.DepthTargetLoadAction;
	HashKey.DepthTargetStoreAction		= Initializer.DepthTargetStoreAction;	
	HashKey.StencilTargetLoadAction		= Initializer.StencilTargetLoadAction;
	HashKey.StencilTargetStoreAction	= Initializer.StencilTargetStoreAction;
	HashKey.DepthStencilAccess			= Initializer.DepthStencilAccess;
	HashKey.NumSamples					= Initializer.NumSamples;
	HashKey.SubpassHint					= Initializer.SubpassHint;
	HashKey.SubpassIndex				= Initializer.SubpassIndex;
	HashKey.ConservativeRasterization	= Initializer.ConservativeRasterization;
	HashKey.bDepthBounds				= Initializer.bDepthBounds;
	HashKey.MultiViewCount				= Initializer.MultiViewCount;
	HashKey.bHasFragmentDensityAttachment = Initializer.bHasFragmentDensityAttachment;
	HashKey.ShadingRate					= Initializer.ShadingRate;

	for (ETextureCreateFlags& Flags : HashKey.RenderTargetFlags)
	{
		Flags = Flags & FGraphicsPipelineStateInitializer::RelevantRenderTargetFlagMask;
	}

	return CityHash64((const char*)&HashKey, sizeof(FNonStateHashKey));
}

bool FDynamicRHI::RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS)
{
	// first check non pointer objects
	if (LHS.PrimitiveType != RHS.PrimitiveType ||
		LHS.bDepthBounds != RHS.bDepthBounds ||
		LHS.MultiViewCount != RHS.MultiViewCount ||
		LHS.ShadingRate != RHS.ShadingRate ||
		LHS.bHasFragmentDensityAttachment != RHS.bHasFragmentDensityAttachment ||
		LHS.RenderTargetsEnabled != RHS.RenderTargetsEnabled ||
		LHS.RenderTargetFormats != RHS.RenderTargetFormats ||
		!FGraphicsPipelineStateInitializer::RelevantRenderTargetFlagsEqual(LHS.RenderTargetFlags, RHS.RenderTargetFlags) ||
		LHS.DepthStencilTargetFormat != RHS.DepthStencilTargetFormat ||
		!FGraphicsPipelineStateInitializer::RelevantDepthStencilFlagsEqual(LHS.DepthStencilTargetFlag, RHS.DepthStencilTargetFlag) ||
		LHS.DepthTargetLoadAction != RHS.DepthTargetLoadAction ||
		LHS.DepthTargetStoreAction != RHS.DepthTargetStoreAction ||
		LHS.StencilTargetLoadAction != RHS.StencilTargetLoadAction ||
		LHS.StencilTargetStoreAction != RHS.StencilTargetStoreAction ||
		LHS.DepthStencilAccess != RHS.DepthStencilAccess ||
		LHS.NumSamples != RHS.NumSamples ||
		LHS.SubpassHint != RHS.SubpassHint ||
		LHS.SubpassIndex != RHS.SubpassIndex ||
		LHS.ConservativeRasterization != RHS.ConservativeRasterization)
	{
		return false;
	}

	// check the RHI shaders (pointer check for shaders should be fine)
	if (LHS.BoundShaderState.VertexShaderRHI != RHS.BoundShaderState.VertexShaderRHI ||
		LHS.BoundShaderState.PixelShaderRHI != RHS.BoundShaderState.PixelShaderRHI ||
		LHS.BoundShaderState.GetMeshShader() != RHS.BoundShaderState.GetMeshShader() ||
		LHS.BoundShaderState.GetAmplificationShader() != RHS.BoundShaderState.GetAmplificationShader() ||
		LHS.BoundShaderState.GetGeometryShader() != RHS.BoundShaderState.GetGeometryShader())
	{
		return false;
	}

	// check the spec consts
	if (LHS.BoundShaderState.VertexShaderSpecializationConstants != RHS.BoundShaderState.VertexShaderSpecializationConstants
		|| LHS.BoundShaderState.PixelShaderSpecializationConstants != RHS.BoundShaderState.PixelShaderSpecializationConstants
#if PLATFORM_SUPPORTS_MESH_SHADERS
		|| LHS.BoundShaderState.MeshShaderSpecializationConstants != RHS.BoundShaderState.MeshShaderSpecializationConstants
		|| LHS.BoundShaderState.AmplificationShaderSpecializationConstants != RHS.BoundShaderState.AmplificationShaderSpecializationConstants
#endif
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		|| LHS.BoundShaderState.GeometryShaderSpecializationConstants != RHS.BoundShaderState.GeometryShaderSpecializationConstants
#endif
		)
	{
		return false;
	}

	// Full compare the of the vertex declaration
	if (!MatchRHIState<FRHIVertexDeclaration, FVertexDeclarationElementList>(LHS.BoundShaderState.VertexDeclarationRHI, RHS.BoundShaderState.VertexDeclarationRHI))
	{
		return false;
	}

	// Check actual state content (each initializer can have it's own state and not going through a factory)
	if (!MatchRHIState<FRHIBlendState, FBlendStateInitializerRHI>(LHS.BlendState, RHS.BlendState) ||
		!MatchRHIState<FRHIRasterizerState, FRasterizerStateInitializerRHI>(LHS.RasterizerState, RHS.RasterizerState) ||
		!MatchRHIState<FRHIDepthStencilState, FDepthStencilStateInitializerRHI>(LHS.DepthStencilState, RHS.DepthStencilState))
	{
		return false;
	}

	return true;
}

FDefaultRHIRenderQueryPool::~FDefaultRHIRenderQueryPool()
{
	check(IsInRHIThread() || IsInRenderingThread());
	checkf(AllocatedQueries == Queries.Num(), TEXT("Querypool deleted before all Queries have been released"));
}

FRHIPooledRenderQuery FDefaultRHIRenderQueryPool::AllocateQuery()
{
	check(IsInParallelRenderingThread());
	UE::TScopeLock Lock(QueriesMutex);
	if (Queries.Num() > 0)
	{
		return FRHIPooledRenderQuery(this, Queries.Pop());
	}
	else
	{
		FRHIPooledRenderQuery Query = FRHIPooledRenderQuery(this, RHICreateRenderQuery(QueryType));
		if (Query.IsValid())
		{
			++AllocatedQueries;
		}

		return Query;
	}
}

void FDefaultRHIRenderQueryPool::ReleaseQuery(TRefCountPtr<FRHIRenderQuery>&& Query)
{
	check(IsInParallelRenderingThread());
	checkf(Query.IsValid(), TEXT("Only release valid queries"));
	
	UE::TScopeLock Lock(QueriesMutex);
	Queries.Push(MoveTemp(Query));
	check(!Query.IsValid());
}

FRenderQueryPoolRHIRef RHICreateRenderQueryPool(ERenderQueryType QueryType, uint32 /* unused NumQueries */)
{
	return new FDefaultRHIRenderQueryPool(QueryType);
}

FRHITransition* RHICreateTransition(const FRHITransitionCreateInfo& CreateInfo)
{
	// Placement create the transition on the heap.
	FRHITransition* Transition = new (FRHICmdListBaseLinearAllocator::Malloc(FRHITransition::GetTotalAllocationSize(), (uint32)FRHITransition::GetAlignment())) FRHITransition(CreateInfo.SrcPipelines, CreateInfo.DstPipelines, CreateInfo.Flags);
	GDynamicRHI->RHICreateTransition(Transition, CreateInfo);
	return Transition;
}

void* FDynamicRHI::RHILockBufferMGPU(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 GPUIndex, uint32 Offset, uint32 Size, EResourceLockMode LockMode)
{
	// Fall through to single GPU case
	check(GPUIndex == 0);
	return RHILockBuffer(RHICmdList, Buffer, Offset, Size, LockMode);
}

void FDynamicRHI::RHIUnlockBufferMGPU(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 GPUIndex)
{
	// Fall through to single GPU case
	check(GPUIndex == 0);
	RHIUnlockBuffer(RHICmdList, Buffer);
}

// Provided for back-compat.
RHI_API FShaderResourceViewInitializer::FShaderResourceViewInitializer(FRHIBuffer* InBuffer, EPixelFormat InFormat, uint32 InStartOffsetBytes, uint32 InNumElements)
	: FRHIViewDesc::FBufferSRV::FInitializer()
	, Buffer(InBuffer)
{
	if (EnumHasAnyFlags(Buffer->GetUsage(), BUF_ByteAddressBuffer))
	{
		SetType(FRHIViewDesc::EBufferType::Raw);
	}
	else
	{
		SetType(FRHIViewDesc::EBufferType::Typed);
		SetFormat(InFormat);
	}

	SetOffsetInBytes(InStartOffsetBytes);
	SetNumElements(InNumElements);
}

// Provided for back-compat.
RHI_API FShaderResourceViewInitializer::FShaderResourceViewInitializer(FRHIBuffer* InBuffer, EPixelFormat InFormat)
	: FRHIViewDesc::FBufferSRV::FInitializer()
	, Buffer(InBuffer)
{
	if (EnumHasAnyFlags(Buffer->GetUsage(), BUF_ByteAddressBuffer))
	{
		SetType(FRHIViewDesc::EBufferType::Raw);
	}
	else
	{
		SetType(FRHIViewDesc::EBufferType::Typed);
		SetFormat(InFormat);
	}
}

// Provided for back-compat.
RHI_API FShaderResourceViewInitializer::FShaderResourceViewInitializer(FRHIBuffer* InBuffer, uint32 InStartOffsetBytes, uint32 InNumElements)
	: FRHIViewDesc::FBufferSRV::FInitializer()
	, Buffer(InBuffer)
{
	SetTypeFromBuffer(Buffer);
	SetOffsetInBytes(InStartOffsetBytes);
	SetNumElements(InNumElements);
}

// Provided for back-compat.
RHI_API FShaderResourceViewInitializer::FShaderResourceViewInitializer(FRHIBuffer* InBuffer, FRHIRayTracingScene* InRayTracingScene, uint32 InStartOffsetBytes)
	: FRHIViewDesc::FBufferSRV::FInitializer()
	, Buffer(InBuffer)
{
	SetTypeFromBuffer(Buffer);
	SetRayTracingScene(InRayTracingScene);
	SetOffsetInBytes(InStartOffsetBytes);
}

// Provided for back-compat.
RHI_API FShaderResourceViewInitializer::FShaderResourceViewInitializer(FRHIBuffer* InBuffer)
	: FRHIViewDesc::FBufferSRV::FInitializer()
	, Buffer(InBuffer)
{
	SetTypeFromBuffer(InBuffer);
}

void FDynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	Texture->SetName(Name);
#endif
}

void FDynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	Buffer->SetName(Name);
#endif
}

void FDynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIUnorderedAccessView* UnorderedAccessViewRHI, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	if (UnorderedAccessViewRHI->IsTexture())
	{
		RHICmdList.BindDebugLabelName(UnorderedAccessViewRHI->GetTexture(), Name);
	}
	else
	{
		RHICmdList.BindDebugLabelName(UnorderedAccessViewRHI->GetBuffer(), Name);
	}
#endif
}
