// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPUDefragAllocator.h"
#include "HAL/IConsoleManager.h"
#include "Stats/StatsMisc.h"
#include "RHI.h"
#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("TexturePool"), STATGROUP_TexturePool, STATCAT_ADVANCED);

DECLARE_CYCLE_STAT(TEXT("Defragmentation"), STAT_TexturePool_DefragTime, STATGROUP_TexturePool);
DECLARE_CYCLE_STAT(TEXT("Blocked By GPU Relocation"), STAT_TexturePool_Blocked, STATGROUP_TexturePool);
DECLARE_MEMORY_STAT_POOL(TEXT("Allocated"), STAT_TexturePool_Allocated, STATGROUP_TexturePool, FPlatformMemory::MCR_GPUDefragPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Free"), STAT_TexturePool_Free, STATGROUP_TexturePool, FPlatformMemory::MCR_GPUDefragPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Largest Hole"), STAT_TexturePool_LargestHole, STATGROUP_TexturePool, FPlatformMemory::MCR_GPUDefragPool);
DECLARE_MEMORY_STAT_POOL(TEXT("Relocating Memory"), STAT_TexturePool_RelocatedSize, STATGROUP_TexturePool, FPlatformMemory::MCR_GPUDefragPool);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Relocations"), STAT_TexturePool_NumRelocations, STATGROUP_TexturePool);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Holes"), STAT_TexturePool_NumHoles, STATGROUP_TexturePool);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total Async Reallocs"), STAT_TexturePool_TotalAsyncReallocations, STATGROUP_TexturePool);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total Async Allocs"), STAT_TexturePool_TotalAsyncAllocations, STATGROUP_TexturePool);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total Async Cancels"), STAT_TexturePool_TotalAsyncCancellations, STATGROUP_TexturePool);

#ifdef UE_BUILD_DEBUG
#define PARTIALDEFRAG_TIMELIMIT		(4.0/1000.0)	// 4 ms
#else
#define PARTIALDEFRAG_TIMELIMIT		(1.0/1000.0)	// 1 ms
#endif

// Number of bytes when a chunk is considered "small", for defragmentation retry purposes.
#define DEFRAG_SMALL_CHUNK_SIZE			(16*1024-1)		// ~16 KB
// Number of defrags before trying a small chunk again (about 5-10 seconds at 30 fps). Must fit in FMemoryChunk::DefragCounter.
#define DEFRAG_SMALL_CHUNK_COUNTER_MIN	(5*30)
#define DEFRAG_SMALL_CHUNK_COUNTER_MAX	(10*30)
// Number of defrags before trying a chunk again (about 1-2 seconds at 30 fps).  Must fit in FMemoryChunk::DefragCounter.
#define DEFRAG_CHUNK_COUNTER_MIN		(20)
#define DEFRAG_CHUNK_COUNTER_MAX		(80)

#if TRACK_RELOCATIONS
int32 FGPUDefragAllocator::GGPUDefragDumpRelocationsToTTY = 0;
FAutoConsoleVariableRef FGPUDefragAllocator::CVarGPUDefragDumpRelocationsToTTY(
	TEXT("r.GPUDefrag.DumpRelocationsToTTY"),
	GGPUDefragDumpRelocationsToTTY,
	TEXT("Dumps logging information for every relocation.\n"),
	ECVF_Default
	);

FGPUDefragAllocator::FRelocationEntry::FRelocationEntry(const uint8* InOldBase, const uint8* InNewBase, uint64 InSize, uint64 InSyncIndex)
: OldBase(InOldBase)
, NewBase(InNewBase)
, Size(InSize)
, SyncIndex(InSyncIndex)
{
	if (GGPUDefragDumpRelocationsToTTY)
	{
		printf("Relocation: 0x%p to 0x%p, %i, %i\n", OldBase, NewBase, (int32)Size, (int32)SyncIndex);
	}
}
#endif

/*-----------------------------------------------------------------------------
FGPUDefragAlllocator::FMemoryChunk implementation.
-----------------------------------------------------------------------------*/

#define FGPUDEFRAGALLOCATOR_FMEMORYCHUNK_MAINTAIN_SORT_ORDER 1

/**
* Inserts this chunk at the head of the free chunk list.
* If bMaintainSortOrder is true, insert-sort this chunk into the free chunk list.
*/
void FGPUDefragAllocator::FMemoryChunk::LinkFree(FMemoryChunk* FirstFreeChunkToSearch)
{
	check(!bIsAvailable);
	bIsAvailable = true;
	DefragCounter = 0;
	UserPayload = 0;
	bTail = false;

	#if !FGPUDEFRAGALLOCATOR_FMEMORYCHUNK_MAINTAIN_SORT_ORDER

		if (BestFitAllocator.FirstFreeChunk)
		{
			NextFreeChunk = BestFitAllocator.FirstFreeChunk;
			PreviousFreeChunk = nullptr;
			BestFitAllocator.FirstFreeChunk->PreviousFreeChunk = this;
			BestFitAllocator.FirstFreeChunk = this;
		}
		else
		{
			PreviousFreeChunk = nullptr;
			NextFreeChunk = nullptr;
			BestFitAllocator.FirstFreeChunk = this;
		}

	#else

		if (BestFitAllocator.FirstFreeChunk)
		{
			FMemoryChunk* InsertBefore = (FirstFreeChunkToSearch && FirstFreeChunkToSearch->bIsAvailable) ? FirstFreeChunkToSearch : BestFitAllocator.FirstFreeChunk;
			while (Base > InsertBefore->Base && InsertBefore->NextFreeChunk)
			{
				InsertBefore = InsertBefore->NextFreeChunk;
			}
			NextFreeChunk = InsertBefore;
			PreviousFreeChunk = InsertBefore->PreviousFreeChunk;
			if (InsertBefore->PreviousFreeChunk)
			{
				InsertBefore->PreviousFreeChunk->NextFreeChunk = this;
			}
			else
			{
				BestFitAllocator.FirstFreeChunk = this;
			}
			InsertBefore->PreviousFreeChunk = this;
		}
		else
		{
			PreviousFreeChunk = nullptr;
			NextFreeChunk = nullptr;
			BestFitAllocator.FirstFreeChunk = this;
		}

	#endif
}

/*-----------------------------------------------------------------------------
FBestFitAllocator implementation.
-----------------------------------------------------------------------------*/

#define GPU_DEFRAG_SANITYCHECK 0
/**
* Allocate physical memory.
*
* @param	AllocationSize	Size of allocation
* @param	bAllowFailure	Whether to allow allocation failure or not
* @return	Pointer to allocated memory
*/
void* FGPUDefragAllocator::Allocate(int64 AllocationSize, int32 Alignment, TStatId InStat, bool bAllowFailure)
{
	SCOPE_SECONDS_COUNTER(TimeSpentInAllocator);
	FScopeLock Lock(&SynchronizationObject);
	check(FirstChunk);
	check(Alignment <= AllocationAlignment);
	const int64 OrigSize = AllocationSize;
	// Make sure everything is appropriately aligned.
	AllocationSize = Align(AllocationSize, AllocationAlignment);

	// Perform a "best fit" search, returning first perfect fit if there is one.
	FMemoryChunk* CurrentChunk = FirstFreeChunk;
	FMemoryChunk* BestChunk = nullptr;
	int64 BestSize = MAX_int64;
	int32 Iteration = 0;
	do
	{
		while (CurrentChunk)
		{
			// Check whether chunk is available and large enough to hold allocation.
			check(CurrentChunk->bIsAvailable);
			int64 AvailableSize = CurrentChunk->GetAvailableSize();
			if (AvailableSize >= AllocationSize)
			{
				// Tighter fits are preferred.
				if (AvailableSize < BestSize)
				{
					BestSize = AvailableSize;
					BestChunk = CurrentChunk;
				}

				// We have a perfect fit, no need to iterate further.
				if (AvailableSize == AllocationSize)
				{
					break;
				}
			}
			CurrentChunk = CurrentChunk->NextFreeChunk;
		}

		// If we didn't find any chunk to allocate, and we're currently doing some async defragmentation...
		if (!BestChunk && NumRelocationsInProgress > 0 && !bAllowFailure)
		{
			// Wait for all relocations to finish and try again.
			FinishAllRelocations();
			CurrentChunk = FirstFreeChunk;
		}
	} while (!BestChunk && CurrentChunk);

	// Dump allocation info and return nullptr if we weren't able to satisfy allocation request.
	if (!BestChunk)
	{
		if (!bAllowFailure)
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			DumpAllocs();
			UE_LOG(LogRHI, Fatal, TEXT("Ran out of memory for allocation in best-fit allocator of size %" INT64_FMT " KByte"), AllocationSize / 1024);
#endif
		}
		return nullptr;
	}

#if GPU_DEFRAG_SANITYCHECK
	for (auto Iter = PointerToChunkMap.CreateConstIterator(); Iter; ++Iter)
	{
		const uint8* UsedMemAddr = (uint8*)Iter.Key();
		const FMemoryChunk* UsedMemChunk = Iter.Value();
		check(UsedMemAddr == UsedMemChunk->Base);
		bool bBefore = (BestChunk->Base + BestChunk->Size) <= UsedMemAddr;
		bool bAfter = BestChunk->Base >= (UsedMemAddr + UsedMemChunk->Size);

		checkf(bBefore || bAfter, TEXT("%i, %i"), (int32)bBefore, (int32)bAfter);
		check(UsedMemAddr != BestChunk->Base);
	}
#endif

	FMemoryChunk* AllocatedChunk = AllocateChunk(BestChunk, AllocationSize, false);

	AllocatedChunk->OrigSize = OrigSize;
	FPlatformAtomics::InterlockedAdd(&PaddingWasteSize, AllocatedChunk->Size - OrigSize);
	//todo: fix stats
	//ensureMsgf(AllocatedChunk->Stat.IsNone(), TEXT("FreeChunk already has a stat."));
	AllocatedChunk->Stat = InStat;

	check(IsAligned(AllocatedChunk->Base, Alignment));

	MemoryTrace_Alloc((uint64)AllocatedChunk->Base, AllocationSize, Alignment);
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, AllocatedChunk->Base, AllocationSize));

	return AllocatedChunk->Base;
}

/**
* Marks the specified chunk as 'allocated' and updates tracking variables.
* Splits the chunk if only a portion of it is allocated.
*
* @param FreeChunk			Chunk to allocate
* @param AllocationSize	Number of bytes to allocate
* @param bAsync			If true, allows allocating from relocating chunks and maintains the free-list sort order.
* @return					The memory chunk that was allocated (the original chunk could've been split).
*/
FGPUDefragAllocator::FMemoryChunk* FGPUDefragAllocator::AllocateChunk(FMemoryChunk* FreeChunk, int64 AllocationSize, bool bAsync, bool bDoValidation)
{
	check(FreeChunk);
	check(FreeChunk->GetAvailableSize() >= AllocationSize);
	check(!FreeChunk->IsLocked());

#if TRACK_RELOCATIONS
	if (GGPUDefragDumpRelocationsToTTY)
	{
		printf("allocate before splits: 0x%p, %i, %i, %i, SyncSize: %i\n", FreeChunk->Base, (int32)FreeChunk->Size, (int32)AllocationSize, (int32)GetCurrentSyncIndex(), (int32)FreeChunk->SyncSize);
	}
#endif

	// If this is an immediate allocation (i.e. the CPU will start accessing the memory right away)
	// and the beginning of the chunk is currently being relocated by the GPU, split that part off and allocate from the rest.
	if (!bAsync && FreeChunk->IsRelocating() && FreeChunk->SyncSize > 0 && FreeChunk->SyncSize < FreeChunk->Size)
	{
#if TRACK_RELOCATIONS
		if (GGPUDefragDumpRelocationsToTTY)
		{
			printf("splitting:\n");
		}
#endif
		Split(FreeChunk, FreeChunk->SyncSize);
		FreeChunk = FreeChunk->NextChunk;
	}

	// Mark as being in use.
	FreeChunk->UnlinkFree();

	// Split chunk to avoid waste.
	if (FreeChunk->Size > AllocationSize)
	{
#if TRACK_RELOCATIONS
		if (GGPUDefragDumpRelocationsToTTY)
		{
			printf("splitting again:\n");
		}
#endif
		Split(FreeChunk, AllocationSize);
	}

	// Ensure that everything's in range.
	check((FreeChunk->Base + FreeChunk->Size) <= (MemoryBase + MemorySize));
	check(FreeChunk->Base >= MemoryBase);

	// Update usage stats in a thread safe way.
	FPlatformAtomics::InterlockedAdd(&AllocatedMemorySize, FreeChunk->Size);
	FPlatformAtomics::InterlockedAdd(&AvailableMemorySize, -FreeChunk->Size);

#if TRACK_RELOCATIONS
	if (GGPUDefragDumpRelocationsToTTY)
	{
		printf("allocate: 0x%p, %i, %i\n", FreeChunk->Base, (int32)AllocationSize, (int32)GetCurrentSyncIndex());
	}
#endif

#if VALIDATE_SYNC_SIZE
	if (bDoValidation)
	{
		ValidateRelocations(FreeChunk->Base, AllocationSize);
	}	
#endif

#if VALIDATE_MEMORY_PROTECTION
	PlatformSetStandardMemoryPrivileges(FMemProtectTracker(FreeChunk->Base, nullptr, FreeChunk->Size, 0));
#endif

	// Keep track of mapping and return pointer.
	PointerToChunkMap.Add(FreeChunk->Base, FreeChunk);
	return FreeChunk;
}

/**
* Marks the specified chunk as 'free' and updates tracking variables.
* Calls LinkFreeChunk() to coalesce adjacent free memory.
*
* @param Chunk						Chunk to free
* @param bMaintainSortedFreelist	If true, maintains the free-list sort order
*/
void FGPUDefragAllocator::FreeChunk(FMemoryChunk* Chunk)
{
#if TRACK_RELOCATIONS
	if (GGPUDefragDumpRelocationsToTTY)
	{
		printf("FreeChunk: 0x%p, %i, %i, %i\n", Chunk->Base, (int32)Chunk->Size, (int32)Chunk->SyncSize, (int32)Chunk->SyncIndex);
	}
#endif

	// Remove the entry
	PointerToChunkMap.Remove(Chunk->Base);
	Chunk->Stat = TStatId();

	// Update usage stats in a thread safe way.
	FPlatformAtomics::InterlockedAdd(&AllocatedMemorySize, -Chunk->Size);
	FPlatformAtomics::InterlockedAdd(&AvailableMemorySize, Chunk->Size);

#if VALIDATE_MEMORY_PROTECTION
	PlatformSetNoMemoryPrivileges(FMemProtectTracker(Chunk->Base, nullptr, Chunk->Size, 0));
#endif

	// Free the chunk.
	LinkFreeChunk(Chunk);
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
/** For debugging minidumps and release builds. */
void* GBestFitAllocatorFreePointer = nullptr;
#endif

/**
* Frees allocation associated with passed in pointer.
*
* @param	Pointer		Pointer to free.
*/
void FGPUDefragAllocator::Free(void* Pointer)
{
	MemoryTrace_Free((uint64)Pointer);
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	if (Pointer)
	{
		FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, Pointer);
	}
#endif

	SCOPE_SECONDS_COUNTER(TimeSpentInAllocator);
	FScopeLock Lock(&SynchronizationObject);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	GBestFitAllocatorFreePointer = Pointer;
#endif

	// Look up pointer in TMap.
	FMemoryChunk* MatchingChunk = PointerToChunkMap.FindRef(Pointer);
	check(MatchingChunk);
	check(MatchingChunk->Base == Pointer);
	checkf(MatchingChunk->LockCount == 0, TEXT("Chunk with base address: 0x%p is being freed with %i outstanding locks.  This is a data corruption hazard."), Pointer, MatchingChunk->LockCount);

	int64 PaddingWaste = MatchingChunk->Size - MatchingChunk->OrigSize;
	FPlatformAtomics::InterlockedAdd(&PaddingWasteSize, -PaddingWaste);

	// Is this chunk is currently being relocated asynchronously (by the GPU)?
	if (MatchingChunk->IsRelocating())
	{
		PendingFreeChunks.AddTail(MatchingChunk);
	}
	else
	{
		// Free the chunk.
		FreeChunk(MatchingChunk);
	}
}

/**
* Locks an FMemoryChunk
*
* @param Pointer		Pointer indicating which chunk to lock
*/
void FGPUDefragAllocator::Lock(const void* Pointer)
{
	FScopeLock Lock(&SynchronizationObject);
	FMemoryChunk* MatchingChunk = PointerToChunkMap.FindRef(Pointer);
	checkf(MatchingChunk, TEXT("Couldn't find chunk for address: 0x%p"), Pointer);

	// Is this chunk is currently being relocated asynchronously (by the GPU)?
	if (MatchingChunk->IsRelocating())
	{
		// Wait for it to finish.
		FinishAllRelocations();
	}

	check(MatchingChunk->LockCount >= 0);
	++MatchingChunk->LockCount;
	++NumLockedChunks;
}

/**
* Unlocks an FMemoryChunk
*
* @param Pointer		Pointer indicating which chunk to unlock
*/
void FGPUDefragAllocator::Unlock(const void* Pointer)
{
	FScopeLock Lock(&SynchronizationObject);
	FMemoryChunk* MatchingChunk = PointerToChunkMap.FindRef(Pointer);
	check(MatchingChunk && MatchingChunk->IsRelocating() == false);
	checkf(MatchingChunk->LockCount > 0, TEXT("Chunk: 0x%p has bad lockcount: %i"), Pointer, MatchingChunk->LockCount);
	--MatchingChunk->LockCount;
	--NumLockedChunks;
}

/**
* Sets the user payload for an FMemoryChunk
*
* @param Pointer		Pointer indicating a chunk
* @param UserPayload	User payload to set
*/
void FGPUDefragAllocator::SetUserPayload(const void* Pointer, void* UserPayload)
{
	FScopeLock Lock(&SynchronizationObject);
	FMemoryChunk* MatchingChunk = PointerToChunkMap.FindRef(Pointer);
	checkf(MatchingChunk, TEXT("Couldn't find matching chunk for %p"), Pointer);
	if (MatchingChunk)
	{
		MatchingChunk->UserPayload = UserPayload;
	}

#if VALIDATE_MEMORY_PROTECTION
	PlatformSetStaticMemoryPrivileges(FMemProtectTracker(MatchingChunk->Base, UserPayload, MatchingChunk->Size, 0));
#endif
}

/**
* Returns the user payload for an FMemoryChunk
*
* @param Pointer		Pointer indicating a chunk
* return				The chunk's user payload
*/
void* FGPUDefragAllocator::GetUserPayload(const void* Pointer)
{
	FScopeLock Lock(&SynchronizationObject);
	FMemoryChunk* MatchingChunk = PointerToChunkMap.FindRef(Pointer);
	check(MatchingChunk);
	if (MatchingChunk)
	{
		return MatchingChunk->UserPayload;
	}
	return 0;
}

/**
* Returns the amount of memory allocated for the specified address.
*
* @param	Pointer		Pointer to check.
* @return				Number of bytes allocated
*/
int64 FGPUDefragAllocator::GetAllocatedSize(void* Pointer)
{
	FScopeLock Lock(&SynchronizationObject);
	FMemoryChunk* MatchingChunk = PointerToChunkMap.FindRef(Pointer);
	return MatchingChunk ? MatchingChunk->Size : 0;
}

/**
* Tries to reallocate texture memory in-place (without relocating),
* by adjusting the base address of the allocation but keeping the end address the same.
*
* @param	OldBaseAddress	Pointer to the original allocation
* @returns	New base address if it succeeded, otherwise nullptr
**/
void* FGPUDefragAllocator::Reallocate(void* OldBaseAddress, int64 NewSize)
{
	FScopeLock Lock(&SynchronizationObject);
	SCOPE_SECONDS_COUNTER(TimeSpentInAllocator);

	// Look up pointer in TMap.
	FMemoryChunk* MatchingChunk = PointerToChunkMap.FindRef(OldBaseAddress);
	check(MatchingChunk && (OldBaseAddress == MatchingChunk->Base));

	// Can't modify a chunk that is currently being relocated.
	// Actually, yes we can, since immediate reallocation doesn't move any memory.
	// 	if ( MatchingChunk->IsRelocating() )
	// 	{
	// 		return nullptr;
	// 	}

	int64 AlignedNewSize = Align(NewSize, AllocationAlignment);
	FMemoryChunk* NewChunk = nullptr;
	int64 MemoryAdjustment = FMath::Abs(AlignedNewSize - MatchingChunk->Size);

	// Are we growing the allocation?
	if (AlignedNewSize > MatchingChunk->Size)
	{
		NewChunk = Grow(MatchingChunk, MemoryAdjustment);
	}
	else
	{
		NewChunk = Shrink(MatchingChunk, MemoryAdjustment);
	}
	return NewChunk ? NewChunk->Base : nullptr;
}

/**
* Tries to immediately grow a memory chunk by moving the base address, without relocating any memory.
*
* @param Chunk			Chunk to grow
* @param GrowAmount	Number of bytes to grow by
* @return				nullptr if it failed, otherwise the new grown chunk
*/
FGPUDefragAllocator::FMemoryChunk* FGPUDefragAllocator::Grow(FMemoryChunk* Chunk, int64 GrowAmount)
{
	// Is there enough free memory immediately before this chunk?
	FMemoryChunk* PrevChunk = Chunk->PreviousChunk;
	if (PrevChunk && PrevChunk->bIsAvailable && PrevChunk->Size >= GrowAmount)
	{
		void* OldBaseAddress = Chunk->Base;
		void* UserPayload = Chunk->UserPayload;
		PointerToChunkMap.Remove(OldBaseAddress);

		// Shrink the previous and grow the current chunk.
		PrevChunk->Size -= GrowAmount;
		Chunk->Base -= GrowAmount;
		Chunk->Size += GrowAmount;

		PointerToChunkMap.Add(Chunk->Base, Chunk);

		if (PrevChunk->Size == 0)
		{
			delete PrevChunk;
		}

		Chunk->UserPayload = UserPayload;

		// Update usage stats in a thread safe way.
		FPlatformAtomics::InterlockedAdd(&AllocatedMemorySize, GrowAmount);
		FPlatformAtomics::InterlockedAdd(&AvailableMemorySize, -GrowAmount);
		return Chunk;
	}
	return nullptr;
}

/**
* Immediately shrinks a memory chunk by moving the base address, without relocating any memory.
* Always succeeds.
*
* @param Chunk			Chunk to shrink
* @param ShrinkAmount	Number of bytes to shrink by
* @return				The new shrunken chunk
*/
FGPUDefragAllocator::FMemoryChunk* FGPUDefragAllocator::Shrink(FMemoryChunk* Chunk, int64 ShrinkAmount)
{
	// We're shrinking the allocation.
	check(ShrinkAmount <= Chunk->Size);
	void* OldBaseAddress = Chunk->Base;
	void* UserPayload = Chunk->UserPayload;

	FMemoryChunk* NewFreeChunk = Chunk->PreviousChunk;
	if (NewFreeChunk)
	{
		// Shrink the current chunk.
		Chunk->Base += ShrinkAmount;
		Chunk->Size -= ShrinkAmount;

		// Grow the previous chunk.
		int64 OriginalPrevSize = NewFreeChunk->Size;
		NewFreeChunk->Size += ShrinkAmount;

		// If the previous chunk was "in use", split it and insert a 2nd free chunk.
		if (!NewFreeChunk->bIsAvailable)
		{
			Split(NewFreeChunk, OriginalPrevSize);
			NewFreeChunk = NewFreeChunk->NextChunk;
		}
	}
	else
	{
		// This was the first chunk, split it.
		Split(Chunk, ShrinkAmount);

		// We're going to use the new chunk. Mark it as "used memory".
		Chunk = Chunk->NextChunk;
		Chunk->UnlinkFree();

		// Make the original chunk "free memory".
		NewFreeChunk = Chunk->PreviousChunk;
#if TRACK_RELOCATIONS
		if (GGPUDefragDumpRelocationsToTTY)
		{
			printf("shrink free chunk\n");
		}
#endif
		LinkFreeChunk(NewFreeChunk);
	}

	// Mark the newly freed memory as "being relocated" and require GPU sync.
	// (The GPU may still be rendering with the old, larger size.)
	NewFreeChunk->SetSyncIndex(GetCurrentSyncIndex(), NewFreeChunk->Size);

	PointerToChunkMap.Remove(OldBaseAddress);
	PointerToChunkMap.Add(Chunk->Base, Chunk);
	Chunk->UserPayload = UserPayload;

	// Update usage stats in a thread safe way.
	FPlatformAtomics::InterlockedAdd(&AllocatedMemorySize, -ShrinkAmount);
	FPlatformAtomics::InterlockedAdd(&AvailableMemorySize, ShrinkAmount);
	return Chunk;
}

#if 0
/**
* Requests an async allocation or reallocation.
* The caller must hold on to the request until it has been completed or canceled.
*
* @param ReallocationRequest	The request
* @param bForceRequest			If true, the request will be accepted even if there's currently not enough free space
* @return						true if the request was accepted
*/
bool FBestFitAllocator::AsyncReallocate(FAsyncReallocationRequest* Request, bool bForceRequest)
{
	// Make sure everything is appropriately aligned.
	Request->NewSize = Align(Request->NewSize, AllocationAlignment);

	if (Request->IsReallocation())
	{
		// Look up pointer in TMap.
		Request->MemoryChunk = PointerToChunkMap.FindRef(PTRint32(Request->OldAddress));
		check(Request->MemoryChunk);
		Request->OldSize = Request->MemoryChunk->Size;
	}

	// OOM?
	int32 MemoryAdjustment = Request->NewSize - Request->OldSize;
	if (!bForceRequest && MemoryAdjustment > 0 && MemoryAdjustment > AvailableMemorySize)
	{
		return false;
	}

	if (Request->IsReallocation())
	{
		// Already being reallocated?
		if (Request->MemoryChunk->HasReallocationRequest())
		{
			//@TODO: Perhaps flush or cancel previous reallocation and try again
			return false;
		}

		// Try an immediate in-place reallocation (just changing the base address of the existing chunk).
		Request->NewAddress = Reallocate(Request->OldAddress, Request->NewSize);
		if (Request->NewAddress)
		{
			// Note: The caller should call PlatformNotifyReallocationFinished().
			Request->MarkCompleted();
			return true;
		}
	}

#if 0
	if (Settings.bEnableAsyncDefrag && Settings.bEnableAsyncReallocation)
	{
		FPlatformAtomics::InterlockedAdd(&PendingMemoryAdjustment, +MemoryAdjustment);

		if (Request->IsReallocation())
		{
			ReallocationRequests.AddTail(Request);
			Request->MemoryChunk->ReallocationRequestNode = ReallocationRequests.GetTail();
		}
		else
		{
			// Allocations takes priority over reallocations.
			ReallocationRequests.AddHead(Request);
		}
		return true;
	}
	else
#endif
	{
		return false;
	}
}
#endif

/**
* Sorts the freelist based on increasing base address.
*
* @param NumFreeChunks		[out] Number of free chunks
* @param LargestFreeChunk	[out] Size of the largest free chunk, in bytes
*/
void FGPUDefragAllocator::SortFreeList(int32& NumFreeChunks, int64& LargestFreeChunk)
{
	NumFreeChunks = 0;
	LargestFreeChunk = 0;

	if (FirstFreeChunk)
	{
		NumFreeChunks++;
		LargestFreeChunk = FirstFreeChunk->Size;
		FMemoryChunk* LastSortedChunk = FirstFreeChunk;
		FMemoryChunk* ChunkToSort = FirstFreeChunk->NextFreeChunk;
		while (ChunkToSort)
		{
			//chunk to sort will be properly sorted.  Thus the next correct chunk to sort is the next one we haven't sorted.
			//this ensures we get the proper chunk count also.
			FMemoryChunk* NextChunkToSort = ChunkToSort->NextFreeChunk;
			LargestFreeChunk = FMath::Max(LargestFreeChunk, ChunkToSort->Size);

			// Out of order?
			if (ChunkToSort->Base < LastSortedChunk->Base)
			{
				FMemoryChunk* InsertBefore = FirstFreeChunk;
				while (ChunkToSort->Base > InsertBefore->Base)
				{
					InsertBefore = InsertBefore->NextFreeChunk;
				}
				ChunkToSort->UnlinkFree();
				ChunkToSort->bIsAvailable = true;	// Set it back to 'free'
				ChunkToSort->PreviousFreeChunk = InsertBefore->PreviousFreeChunk;
				ChunkToSort->NextFreeChunk = InsertBefore;
				if (InsertBefore->PreviousFreeChunk)
				{
					InsertBefore->PreviousFreeChunk->NextFreeChunk = ChunkToSort;
				}
				InsertBefore->PreviousFreeChunk = ChunkToSort;
				if (InsertBefore == FirstFreeChunk)
				{
					FirstFreeChunk = ChunkToSort;
				}
			}
			LastSortedChunk = ChunkToSort;
			ChunkToSort = NextChunkToSort;
			NumFreeChunks++;
		}
	}
}

FGPUDefragAllocator::FMemoryChunk* FGPUDefragAllocator::FindAdjacent(FMemoryChunk* UsedChunk, bool bAnyChunkType)
{
	if (UsedChunk && (bAnyChunkType))
	{
		FMemoryChunk* FreeChunkLeft = UsedChunk->PreviousChunk;
		FMemoryChunk* FreeChunkRight = UsedChunk->NextChunk;

		int64 AvailableSize = UsedChunk->Size;
		if (FreeChunkLeft && FreeChunkLeft->bIsAvailable)
		{
			AvailableSize += FreeChunkLeft->Size;
		}
		if (FreeChunkRight && FreeChunkRight->bIsAvailable)
		{
			AvailableSize += FreeChunkRight->Size;
		}

		// Does it fit?
		int64 FinalSize = UsedChunk->GetFinalSize();
		if (FinalSize <= AvailableSize && CanRelocate(UsedChunk))
		{
			return UsedChunk;
		}
	}
	return nullptr;
}

/**
* Searches for an allocated chunk that would fit within the specified free chunk.
* The allocated chunk must be adjacent to a free chunk and have a larger
* base address than 'FreeChunk'.
* Starts searching from the end of the texture pool.
*
* @param FreeChunk		Free chunk we're trying to fill up
* @return				Pointer to a suitable chunk, or nullptr
*/
FGPUDefragAllocator::FMemoryChunk* FGPUDefragAllocator::FindAdjacentToHole(FMemoryChunk* FreeChunk)
{
	//@TODO: Maintain LastFreeChunk for speed
	FMemoryChunk* LastFreeChunk = LastChunk;
	while (LastFreeChunk && !LastFreeChunk->bIsAvailable)
	{
		LastFreeChunk = LastFreeChunk->PreviousChunk;
	}

	FMemoryChunk* Chunk = LastFreeChunk;
	while (Chunk && Chunk->Base > FreeChunk->Base)
	{
		// Check Right
		const FMemoryChunk* Right = Chunk->NextChunk;
		if (Right && !Right->bIsAvailable && Right->GetFinalSize() < FreeChunk->Size && CanRelocate(Right))
		{
			return Chunk->NextChunk;
		}
		// Check Left
		const FMemoryChunk* Left = Chunk->PreviousChunk;
		if (Left && !Left->bIsAvailable && Left->GetFinalSize() < FreeChunk->Size && CanRelocate(Left))
		{
			return Chunk->PreviousChunk;
		}
		Chunk = Chunk->PreviousFreeChunk;
	}
	return nullptr;
}

/**
* Searches for an allocated chunk that would fit within the specified free chunk.
* Any chunk that fits and has a larger base address than 'FreeChunk' is accepted.
* Starts searching from the end of the texture pool.
*
* @param FreeChunk		Free chunk we're trying to fill up
* @return				Pointer to a suitable chunk, or nullptr
*/
FGPUDefragAllocator::FMemoryChunk* FGPUDefragAllocator::FindAny(FMemoryChunk* FreeChunk)
{
	//@TODO: Stop the search at some reasonable threshold (FreeChunk->Size < THRESHOLD || NumChunksSearched > THRESHOLD).
	FMemoryChunk* BestChunk = nullptr;
	int64 BestFit = MAX_int64;
	FMemoryChunk* CurrentChunk = LastChunk;
	while (CurrentChunk && CurrentChunk->Base > FreeChunk->Base)
	{
		if (!CurrentChunk->bIsAvailable)
		{
			int64 CurrentFit = FreeChunk->Size - CurrentChunk->GetFinalSize();

			// Better fit than previously?
			if (CurrentFit >= 0 && CurrentFit < BestFit && CanRelocate(CurrentChunk))
			{
				if (RelocateAllowed(FreeChunk, CurrentChunk))
				{
					BestFit = CurrentFit;
					BestChunk = CurrentChunk;

					// Perfect fit?
					if (CurrentFit == 0)
					{
						break;
					}
				}
			}
		}
		CurrentChunk = CurrentChunk->PreviousChunk;
	}

	return BestChunk;
}

/**
* Checks the internal state for errors. (Slow)
*
* @param bCheckSortedFreeList	If true, also checks that the freelist is sorted
*/
void FGPUDefragAllocator::CheckForErrors(bool bCheckSortedFreeList)
{
	if (FirstFreeChunk == nullptr)
	{
		return;
	}

	if (bCheckSortedFreeList)
	{
		FMemoryChunk* Chunk = FirstFreeChunk;
		int64 TotalFreeMem = Chunk->Size;
		while (Chunk->NextFreeChunk)
		{
			check(Chunk->bIsAvailable);
			check(Chunk->Base < Chunk->NextFreeChunk->Base);
			check(!Chunk->NextChunk->bIsAvailable);
			check(Chunk->PreviousChunk == nullptr || !Chunk->PreviousChunk->bIsAvailable);
			Chunk = Chunk->NextFreeChunk;
			TotalFreeMem += Chunk->Size;
		}
		check(TotalFreeMem == AvailableMemorySize);
	}

	int64 TotalUsedMem = 0;
	int64 TotalFreeMem = 0;
	FMemoryChunk* Chunk = FirstChunk;
	while (Chunk)
	{
		if (Chunk->bIsAvailable)
		{
			TotalFreeMem += Chunk->Size;
		}
		else
		{
			TotalUsedMem += Chunk->Size;
		}
		Chunk = Chunk->NextChunk;
	}
	check(TotalUsedMem == AllocatedMemorySize);
	check(TotalFreeMem == AvailableMemorySize);
}

#if VALIDATE_SYNC_SIZE
void FGPUDefragAllocator::ValidateRelocations(uint8* UsedBaseAddress, uint64 Size)
{	
	FScopeLock SyncLock(&SynchronizationObject);
	for (int32 i = 0; i < Relocations.Num(); ++i)
	{
		const FRelocationEntry& Relocation = Relocations[i];

		bool bBeforeOrig = (UsedBaseAddress + Size) <= Relocation.OldBase;
		bool bAfterOrig = (UsedBaseAddress) >= (Relocation.OldBase + Relocation.Size);

		bool bBeforeNew = (UsedBaseAddress + Size) <= Relocation.NewBase;
		bool bAfterNew = (UsedBaseAddress) >= (Relocation.NewBase + Relocation.Size);

		bool bAfterSync = Relocation.SyncIndex <= CompletedSyncIndex;
		checkf(((bBeforeOrig || bAfterOrig) && (bBeforeNew || bAfterNew)) || bAfterSync, TEXT("Corruption Hazard, Allocation not protected by sync size."));
	}	
}
#endif

/**
* Initiates an async relocation of an allocated chunk into a free chunk.
* Takes potential reallocation request into account.
*
* @param Stats			[out] Stats
* @param FreeChunk		Destination chunk (free memory)
* @param SourceChunk	Source chunk (allocated memory)
* @return				Next Free chunk to try to fill up
*/
FGPUDefragAllocator::FMemoryChunk* FGPUDefragAllocator::RelocateIntoFreeChunk(FRelocationStats& Stats, FMemoryChunk* FreeChunk, FMemoryChunk* SourceChunk)
{
	check(!FreeChunk->IsLocked());
	check(!SourceChunk->IsLocked());
	check(FreeChunk->bIsAvailable);
	check(!SourceChunk->bIsAvailable);

	// Save off important data from 'SourceChunk', since it will get modified by the call to LinkFreeChunk().	
	void* UserPayload = SourceChunk->UserPayload;
	const int64 OrigSize = SourceChunk->OrigSize;
	const int64 OldSize = SourceChunk->Size;
	const uint8* SourceOldBase = SourceChunk->Base;
	const uint8* DestNewBase = FreeChunk->Base;
	const int64 NewSize = SourceChunk->GetFinalSize();
	const int64 UsedSize = FMath::Min(NewSize, OldSize);
	TStatId	SourceStat = SourceChunk->Stat;

	// Are we relocating into adjacent free chunk?
	const bool bFreeChunkPreviousAdjacent = SourceChunk->PreviousChunk == FreeChunk;
	const bool bFreeChunkNextAdjacent = SourceChunk->NextChunk == FreeChunk;
	const bool bAdjacentRelocation = bFreeChunkPreviousAdjacent || bFreeChunkNextAdjacent;

	// Enable for debugging:
	// CheckForErrors( true );

	// Merge adjacent free chunks into SourceChunk to make a single free chunk.
#if TRACK_RELOCATIONS
	if (GGPUDefragDumpRelocationsToTTY)
	{
		printf("relocate link free chunk\n");
	}
#endif
	LinkFreeChunk(SourceChunk);

	FMemoryChunk* DestinationChunk;
	if (bAdjacentRelocation)
	{
		DestinationChunk = SourceChunk;
	}
	else
	{
		DestinationChunk = FreeChunk;
	}
	// 'FreeChunk' was deleted if it was adjacent to SourceChunk. Set to nullptr to avoid using it by mistake.
	FreeChunk = nullptr;

	// Leave room for new mips to stream in.
	int64 DestinationOffset = FMath::Max((NewSize - OldSize), (int64)0);

	// Relocate the memory if needed
	bool bRelocated = false;
	uint8* NewBaseAddress = DestinationChunk->Base;
	if (SourceOldBase != (NewBaseAddress + DestinationOffset))
	{
		Relocate(Stats, DestinationChunk, DestinationOffset, SourceOldBase, UsedSize, UserPayload);
		bRelocated = true;
	}
	// Make sure the destination chunk keeps the UserPayload, no matter what. :)
	DestinationChunk->UserPayload = UserPayload;

	SourceChunk->OrigSize = 0;
	DestinationChunk->OrigSize = OrigSize;

	// Update our book-keeping.
	PointerToChunkMap.Remove(SourceOldBase);
	PointerToChunkMap.Add(NewBaseAddress, DestinationChunk);
	SourceChunk->Stat = TStatId();
	DestinationChunk->Stat = SourceStat;	

	// Is there free space left over at the end of DestinationChunk?
	FMemoryChunk* NextFreeChunk;
	if (DestinationChunk->Size > NewSize)
	{
		// Split the DestinationChunk into a used chunk and a free chunk.
		Split(DestinationChunk, NewSize);
		NextFreeChunk = DestinationChunk->NextChunk;
		NextFreeChunk->bTail = bAdjacentRelocation;

		if (bAdjacentRelocation)
		{
			//if the free chunk was on the right
			if (bFreeChunkNextAdjacent)
			{
				ensureMsgf(!bRelocated, TEXT("free chunk was on the right, why did we relocate?"));
			}
			else
			{
				//if the free chunk was adjacent on the left then we need to make sure the tail where the source chunk used to reside is not allocated and written to before the GPU
				//finishes moving it to its new location.
				//Original Layout                   |xxxxxxxxxxxxx|uuuu|xxxx -> 
				//Layout after move                 |uuuu|xxxxxxxxxxxxxxxxx| ->

				//Layout after move                 |uuuu|ssssssssstttt| ->
				//									|uuuu|sssssssss|tttt|


				//Layout after tail safety split    |uuuu|xxxxxxxx|tttt|xxxx

				//									|xxxx|uuuuuuuuuuuuu| ->
				//									|uuuuuuuuuuuuu|xxxx| ->											

				//to compute the relocatable size properly we need to account for any righthand coalescing that may have made the chunk bigger.
				//right hand coalesce will be added into the tail.
				int64 LeftShiftSize = (SourceOldBase - SourceChunk->Base);
				int64 RightAddSize = (NextFreeChunk->Size - LeftShiftSize);
				int64 RelocatableSize = FMath::Max((int64)((NextFreeChunk->Size - NewSize - RightAddSize)), (int64)0);				
				if ((RelocatableSize > 0) && NextFreeChunk->GetAvailableSize())
				{
					Split(NextFreeChunk, RelocatableSize);
					FMemoryChunk* InFlightTailChunk = NextFreeChunk->NextFreeChunk;					

					//if the tail has right coalesced memory afterwards, split the right coalesce back off into its own chunk for re-use.
					if (InFlightTailChunk->Size > OldSize)
					{
						Split(InFlightTailChunk, OldSize);
					}
					//defragger code assumes free chunks are not adjacent (they would have been coalesced)
					//so mark this as 'used' so it can't be allocated while the tail is being moved by the GPU and put it on the deferred free list
					//hack to avoid internal checks on the hack allocate
					InFlightTailChunk->SetSyncIndex(CompletedSyncIndex, 0);
					AllocateChunk(InFlightTailChunk, InFlightTailChunk->Size, false, false);
					InFlightTailChunk->SetSyncIndex(GetCurrentSyncIndex(), InFlightTailChunk->Size);
					++InFlightTailChunk->LockCount;
					PendingFreeChunks.AddTail(InFlightTailChunk);
				}
				else
				{
					//in this case there was no tail and we need to protect the existing sync size, or just the old size.
					//maxing with the existing sync size keeps any sync-size from the right-coalesce from going away.
					//should be a case like this:
					// |xxx|uuuuuuuuuuuuu|fffffff|
					// |uuuuuuuuuuuuu|sssffffffff|
					NextFreeChunk->SetSyncIndex(GetCurrentSyncIndex(), FMath::Max(NextFreeChunk->SyncSize, OldSize));
				}
			}
		}
	}
	else
	{
		// The whole DestinationChunk is now allocated.
		check(DestinationChunk->Size == NewSize);
		NextFreeChunk = DestinationChunk->NextFreeChunk;
	}
	DestinationChunk->UnlinkFree();

	//if source chunk did not coalesce with another free chunk when it was 'freed' or if it coalesced with a freechunk on its right side then we only need
	//to protect up to the OldSize of the SourceChunk.
	int64 SourceChunkSyncSize = OldSize;

	const bool bSourceCoalescedOnLeft = SourceChunk->Base < SourceOldBase;
	if (bSourceCoalescedOnLeft)
	{
		//if source chunk coalesced with a free chunk on the left, then protecting with OldSize does not suffice
		//Orig. uuuu is being relocated by the GPU    |xx|uuuu|sssxxxxxxxxxxxx ->
		//                                            |xx|rrrr| ->
		//We must protect up to the full relocation.  |pp|pppp|ssssxxxxxxxxxxxxxx		
		SourceChunkSyncSize += (SourceOldBase - SourceChunk->Base);		
	}

	//if the source chunk coalesced with a right chunk that HAD a sync size, then the coalense function would have
	//set an appropriate syncsize to cover up to the right side's required protection.  However we are about to override that so it needs to be taken into account here.		
	SourceChunkSyncSize = FMath::Max(SourceChunkSyncSize, SourceChunk->SyncSize);

	// Mark both chunks as "in use" during the current sync step.
	// Note: This sync index will propagate if these chunks are involved in any merge/split in the future.	
	SourceChunk->SetSyncIndex(GetCurrentSyncIndex(), SourceChunkSyncSize);

	//if the destination is already relocation a larger size than we are moving, we still need to protect the larger size so allocations don't split off an end that's still moving.
	int64 DestinationSyncSize = NewSize;
	if (DestinationChunk->IsRelocating())
	{
		DestinationSyncSize = FMath::Max(DestinationChunk->SyncSize, NewSize);
	}
	DestinationChunk->SetSyncIndex(GetCurrentSyncIndex(), DestinationSyncSize);

	if (NewSize != OldSize)
	{
		int64 MemoryAdjustment = NewSize - OldSize;
		FPlatformAtomics::InterlockedAdd(&AllocatedMemorySize, +MemoryAdjustment);
		FPlatformAtomics::InterlockedAdd(&AvailableMemorySize, -MemoryAdjustment);
		FPlatformAtomics::InterlockedAdd(&PendingMemoryAdjustment, -MemoryAdjustment);
	}

#if VALIDATE_MOVES
	for (const auto& Relocation : Relocations)
	{
		bool bBeforeReloc = (DestNewBase + OldSize) <= Relocation.NewBase;
		bool bAfterReloc = (DestNewBase) >= (Relocation.NewBase + Relocation.Size);
				
		checkf((bBeforeReloc || bAfterReloc), TEXT("Corruption Hazard, Destinations overlap within same frame."));
	}
#endif
#if TRACK_RELOCATIONS
	Relocations.Add(FRelocationEntry(SourceOldBase, DestNewBase, OldSize, GetCurrentSyncIndex()));

	if (GGPUDefragDumpRelocationsToTTY)
	{
		printf("bFreeAdjOnLeft %i, bFreeAdjOnRight %i\n", bFreeChunkPreviousAdjacent, bFreeChunkNextAdjacent);
	}
#endif

	// Enable for debugging:
	// CheckForErrors( true );

	// Did we free up a chunk to the left of NextFreeChunk?
	if (!bAdjacentRelocation && SourceOldBase < NewBaseAddress)
	{
		// Use that one for the next defrag step!
		return SourceChunk;
	}
	return NextFreeChunk;
}

/**
* Blocks the calling thread until all relocations and reallocations that were initiated by Tick() have completed.
*
* @return		true if there were any relocations in progress before this call
*/
bool FGPUDefragAllocator::FinishAllRelocations()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_DEFRAG_FinishRelocations);
	bool bWasAnyRelocationsInProgress = NumRelocationsInProgress > 0;

	if (bWasAnyRelocationsInProgress)
	{		
		BlockOnFence();
	}	

	// Take the opportunity to free all chunks that couldn't be freed immediately before.
	for (TDoubleLinkedList<FMemoryChunk*>::TIterator It(PendingFreeChunks.GetHead()); It; ++It)
	{
		FMemoryChunk* Chunk = *It;
		--Chunk->LockCount;
		FreeChunk(Chunk);
	}
	PendingFreeChunks.Empty();

	NumRelocationsInProgress = 0;

	return bWasAnyRelocationsInProgress;
}

/**
* Inserts a platform fence and updates the allocator sync index to match.
*/
void FGPUDefragAllocator::InsertFence()
{
	if (!bBenchmarkMode)
	{
		PlatformSyncFence = PlatformInsertFence();
	}
	CurrentSyncIndex++;
}

#if VALIDATE_MEMORY_PROTECTION
void FGPUDefragAllocator::SetStaticMemoryPrivileges()
{
	FScopedDurationTimer Timer(TimeInMemProtect);
	const int32 Count = BlocksToUnProtect.Num();
	for (int32 i = Count - 1; i >= 0; --i)
	{
		const FMemProtectTracker& Block = BlocksToUnProtect[i];
		if (Block.SyncIndex >= CompletedSyncIndex)
		{
			PlatformSetStaticMemoryPrivileges(Block);
			BlocksToUnProtect.RemoveAtSwap(i, EAllowShrinking::No);
		}
	}	
}
#endif

/**
* Blocks the calling thread until the current sync fence has been completed.
*/
void FGPUDefragAllocator::BlockOnFence()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_DEFRAG_BlockOnFence);
	if (CompletedSyncIndex < (CurrentSyncIndex - 1))
	{
		uint32 StartTime = FPlatformTime::Cycles();
		if (!bBenchmarkMode)
		{
			PlatformBlockOnFence(PlatformSyncFence);
		}
		CompletedSyncIndex = CurrentSyncIndex - 1;
		BlockedCycles += FPlatformTime::Cycles() - StartTime;

#if VALIDATE_MEMORY_PROTECTION
		SetStaticMemoryPrivileges();
#endif
	}
}

/**
* Blocks the calling thread until the specified sync index has been completed.
*
* @param SyncIndex		Sync index to wait for
*/
void FGPUDefragAllocator::BlockOnSyncIndex(uint32 SyncIndex)
{
	// Not completed yet?
	if (SyncIndex > CompletedSyncIndex)
	{
		FinishAllRelocations();

		// Still not completed?
		if (SyncIndex > CompletedSyncIndex)
		{
			InsertFence();
			BlockOnFence();
			FinishAllRelocations();
		}
	}
}

static int32 GGPUDefragEnableTimeLimits = 1;
static FAutoConsoleVariableRef CVarGPUDefragEnableTimeLimits(
	TEXT("r.GPUDefrag.EnableTimeLimits"),
	GGPUDefragEnableTimeLimits,
	TEXT("Limits CPU time spent doing GPU defragmentation.\n"),
	ECVF_Default
	);

static int32 GGPUDefragMaxRelocations = 10;
static FAutoConsoleVariableRef CVarGPUDefragMaxRelocations(
	TEXT("r.GPUDefrag.MaxRelocations"),
	GGPUDefragMaxRelocations,
	TEXT("Limits the number of total relocations in a frame regardless of number of bytes moved..\n"),
	ECVF_Default
	);

static int32 GGPUDefragAllowOverlappedMoves = 1;
static FAutoConsoleVariableRef CVarGPUDefragAllowOverlappedMoves(
	TEXT("r.GPUDefrag.AllowOverlappedMoves"),
	GGPUDefragAllowOverlappedMoves,
	TEXT("Allows defrag relocations that partially overlap themselves.\n"),
	ECVF_Default
	);

/**
* Performs a partial defrag while trying to process any pending async reallocation requests.
*
* @param Stats			[out] Stats
* @param StartTime		Start time, used for limiting the Tick() time
*/

FGPUDefragAllocator::FMemoryChunk* FGPUDefragAllocator::RelocateAllowed(FMemoryChunk* FreeChunk, FMemoryChunk* UsedChunk)
{
	FMemoryChunk* AllowedChunk = nullptr;
	int64 MemDist = 0;
	if (UsedChunk)
	{
		MemDist = FMath::Abs(FreeChunk->Base - UsedChunk->Base);
	}
	if (UsedChunk && (GGPUDefragAllowOverlappedMoves != 0 || (MemDist >= UsedChunk->Size)))
	{
		AllowedChunk = UsedChunk;
	}
	return AllowedChunk;
}

void FGPUDefragAllocator::PartialDefragmentationFast(FRelocationStats& Stats, double StartTime)
{
	FMemoryChunk* FreeChunk = FirstFreeChunk;
	while (FreeChunk /*&& ReallocationRequests.Num() > 0*/ && Stats.NumBytesRelocated < Settings.MaxDefragRelocations && Stats.NumRelocations < GGPUDefragMaxRelocations)
	{
		FRequestNode* BestRequestNode = nullptr;
		FMemoryChunk* BestChunk = nullptr;

		if (FreeChunk->DefragCounter)
		{
			FreeChunk->DefragCounter--;
		}
		else
		{
			//Not much point merging to the used chunk on the left.  We should have already done any merges on the left with the ordered free chunk walk.
			//BestChunk = RelocateAllowed(FreeChunk, FindAdjacent(FreeChunk->PreviousChunk, true));
			
			// 1. Merge with Right,
			BestChunk = RelocateAllowed(FreeChunk, FindAdjacent(FreeChunk->NextChunk, true));

			if (!BestChunk)
			{					
				// 2. Merge with a used chunk adjacent to hole (to make that hole larger).
				BestChunk = RelocateAllowed(FreeChunk, FindAdjacentToHole(FreeChunk));				
			}
		}

		if (BestChunk)
		{
			FreeChunk = RelocateIntoFreeChunk(Stats, FreeChunk, BestChunk);			
		}		
		else
		{
			// Did the free chunk fail to defrag?
			if (FreeChunk->DefragCounter == 0 && (FreeChunk->NextFreeChunk))
			{
				// Don't try it again for a while.
				if (FreeChunk->Size < DEFRAG_SMALL_CHUNK_SIZE)
				{
					FreeChunk->DefragCounter = DEFRAG_SMALL_CHUNK_COUNTER_MIN + FMath::RandHelper(DEFRAG_SMALL_CHUNK_COUNTER_MAX - DEFRAG_SMALL_CHUNK_COUNTER_MIN);
				}
				else
				{
					FreeChunk->DefragCounter = DEFRAG_CHUNK_COUNTER_MIN + FMath::RandHelper(DEFRAG_CHUNK_COUNTER_MAX - DEFRAG_CHUNK_COUNTER_MIN);
				}
			}

			FreeChunk = FreeChunk->NextFreeChunk;
		}

		// Limit time spent
		double TimeSpent = FPlatformTime::Seconds() - StartTime;
		if ((GGPUDefragEnableTimeLimits != 0) && (TimeSpent > PARTIALDEFRAG_TIMELIMIT))
		{
			break;
		}
	}
}

void FGPUDefragAllocator::PartialDefragmentationSlow(FRelocationStats& Stats, double StartTime)
{
	if (Stats.NumBytesRelocated > 0)
	{
		return;
	}

	FMemoryChunk* FreeChunk = FirstFreeChunk;
	while (FreeChunk /*&& ReallocationRequests.Num() > 0*/ && Stats.NumBytesRelocated < Settings.MaxDefragRelocations && Stats.NumRelocations < GGPUDefragMaxRelocations)
	{
		FRequestNode* BestRequestNode = nullptr;
		FMemoryChunk* BestChunk = nullptr;

		if (FreeChunk->DefragCounter)
		{
			FreeChunk->DefragCounter--;
		}
		else
		{
			//1. Merge with chunk from the end of the pool (well-fitting)
			BestChunk = FindAny(FreeChunk);
		}

		if (BestChunk)
		{
			FreeChunk = RelocateIntoFreeChunk(Stats, FreeChunk, BestChunk);
		}
		else
		{
			// Did the free chunk fail to defrag?
			if (FreeChunk->DefragCounter == 0 && (FreeChunk->NextFreeChunk))
			{
				// Don't try it again for a while.
				if (FreeChunk->Size < DEFRAG_SMALL_CHUNK_SIZE)
				{
					FreeChunk->DefragCounter = DEFRAG_SMALL_CHUNK_COUNTER_MIN + FMath::RandHelper(DEFRAG_SMALL_CHUNK_COUNTER_MAX - DEFRAG_SMALL_CHUNK_COUNTER_MIN);
				}
				else
				{
					FreeChunk->DefragCounter = DEFRAG_CHUNK_COUNTER_MIN + FMath::RandHelper(DEFRAG_CHUNK_COUNTER_MAX - DEFRAG_CHUNK_COUNTER_MIN);
				}
			}

			FreeChunk = FreeChunk->NextFreeChunk;
		}

		// Limit time spent
		double TimeSpent = FPlatformTime::Seconds() - StartTime;
		if ((GGPUDefragEnableTimeLimits != 0) && (TimeSpent > PARTIALDEFRAG_TIMELIMIT))
		{
			break;
		}
	}
}

/**
* Performs a partial defrag by shifting down memory to fill holes, in a brute-force manner.
* Takes consideration to async reallocations, but processes the all memory in order.
*
* @param Stats			[out] Stats
* @param StartTime		Start time, used for limiting the Tick() time
*/
void FGPUDefragAllocator::PartialDefragmentationDownshift(FRelocationStats& Stats, double StartTime)
{
	FMemoryChunk* FreeChunk = FirstFreeChunk;
	while (FreeChunk && Stats.NumBytesRelocated < Settings.MaxDefragRelocations && Stats.NumBytesDownShifted < Settings.MaxDefragDownShift && Stats.NumRelocations < GGPUDefragMaxRelocations)
	{
		FMemoryChunk* BestChunk = nullptr;

		// 6. Merge with Right, if it fits
		BestChunk = FindAdjacent(FreeChunk->NextChunk, true);

		int64 MemDist = 0;
		if (BestChunk)
		{
			MemDist = FMath::Abs(FreeChunk->Base - BestChunk->Base);
		}

		if (BestChunk && (GGPUDefragAllowOverlappedMoves != 0 || (MemDist >= BestChunk->Size)))
		{
			Stats.NumBytesDownShifted += BestChunk->Size;
			FreeChunk = RelocateIntoFreeChunk(Stats, FreeChunk, BestChunk);			
		}
		else
		{
			FreeChunk = FreeChunk->NextFreeChunk;
		}

		// Limit time spent
		double TimeSpent = FPlatformTime::Seconds() - StartTime;
		if ((GGPUDefragEnableTimeLimits != 0) && (TimeSpent > PARTIALDEFRAG_TIMELIMIT))
		{
			break;
		}
	}
}

/**
* Performs a full defrag and ignores all reallocation requests.
*
* @param Stats			[out] Stats
*/
void FGPUDefragAllocator::FullDefragmentation(FRelocationStats& Stats)
{
	FMemoryChunk* FreeChunk = FirstFreeChunk;
	while (FreeChunk)
	{
		// Try merging with a used chunk adjacent to hole (to make that hole larger).
		FMemoryChunk* BestChunk = FindAdjacentToHole(FreeChunk);

		if (!BestChunk)
		{
			// Try merging with chunk from the end of the pool (well-fitting)
			BestChunk = FindAny(FreeChunk);

			if (!BestChunk)
			{
				// Try merging with Right, if it fits (brute-force downshifting)
				BestChunk = FindAdjacent(FreeChunk->NextChunk, true);
				if (BestChunk)
				{
					Stats.NumBytesDownShifted += BestChunk->Size;
				}
			}
		}
		if (BestChunk)
		{
			FreeChunk = RelocateIntoFreeChunk(Stats, FreeChunk, BestChunk);
		}
		else
		{
			FreeChunk = FreeChunk->NextFreeChunk;
		}
	}
}

/**
* Partially defragments the memory and tries to process all async reallocation requests at the same time.
* Call this once per frame.
*
* @param Stats			[out] Stats
* @param bPanicDefrag	If true, performs a full defrag and ignores all reallocation requests
*/

int32 FGPUDefragAllocator::Tick(FRelocationStats& Stats, bool bPanicDefrag)
{
	FScopeLock SyncLock(&SynchronizationObject);

#if VALIDATE_MEMORY_PROTECTION
	TimeInMemProtect = 0;
#endif

	SET_CYCLE_COUNTER(STAT_TexturePool_Blocked, BlockedCycles);
	double StartTime = FPlatformTime::Seconds();
	BlockedCycles = 0;

	// Block until all relocations that were kicked of from last call have been completed.
	// There may still be chunks being flagged as 'IsRelocating' due to immediate shrinks between calls.
	FinishAllRelocations();

#if VALIDATE_SYNC_SIZE || VALIDATE_MOVES
	Relocations.Empty();	
#endif

	// Sort the Free chunks.
	SortFreeList(Stats.NumHoles, Stats.LargestHoleSize);

	//if (ReallocationRequests.Num() || ReallocationRequestsInProgress.Num() || bPanicDefrag)
	{
		if (!bPanicDefrag)
		{
			// Smart defrag
			PartialDefragmentationFast(Stats, StartTime);

			PartialDefragmentationSlow(Stats, StartTime);

			// Brute-force defrag
			//PartialDefragmentationDownshift(Stats, StartTime);			
		}
		else
		{
			FullDefragmentation(Stats);
		}
	}

#if VALIDATE_MEMORY_PROTECTION
	{
		FScopedDurationTimer MemProtectTimer(TimeInMemProtect);
		PlatformSetRelocationMemoryPrivileges(BlocksToProtect);
		BlocksToUnProtect.Append(BlocksToProtect);
		BlocksToProtect.Reset();
	}
#endif
	NumRelocationsInProgress = Stats.NumRelocations;

	// Start a new sync point.
	InsertFence();

	TotalNumRelocations += Stats.NumRelocations;
	TotalNumBytesRelocated += Stats.NumBytesRelocated;
	MaxNumHoles = FMath::Max(MaxNumHoles, Stats.NumHoles);
	MinLargestHole = FMath::Min(MinLargestHole, Stats.LargestHoleSize);
	CurrentLargestHole = Stats.LargestHoleSize;
	CurrentNumHoles = Stats.NumHoles;

#if GPU_DEFRAG_SANITYCHECK
	FMemoryChunk* TestChunk = FirstChunk;
	while (TestChunk)
	{		
		FMemoryChunk* TestAgainstChunk = TestChunk->NextChunk;
		while (TestAgainstChunk)
		{			
			bool bBefore = (TestChunk->Base + TestChunk->Size) <= TestAgainstChunk->Base;
			bool bAfter = TestChunk->Base >= (TestAgainstChunk->Base + TestAgainstChunk->Size);

			checkf(bBefore || bAfter, TEXT("%i, %i"), (int32)bBefore, (int32)bAfter);
			check(TestChunk->Base != TestAgainstChunk->Base);
			TestAgainstChunk = TestAgainstChunk->NextChunk;
		}
		TestChunk = TestChunk->NextChunk;
	}	
#endif

	return Stats.NumBytesRelocated;
}

/**
* Dump allocation information.
*/
void FGPUDefragAllocator::DumpAllocs(FOutputDevice& Ar/*=*GLog*/)
{
	// Memory usage stats.
	int64				UsedSize = 0;
	int64				FreeSize = 0;
	int64				NumUsedChunks = 0;
	int64				NumFreeChunks = 0;

	// Fragmentation and allocation size visualization.
	int64				NumBlocks = MemorySize / AllocationAlignment;
	int64				Dimension = 1 + NumBlocks / FMath::TruncToInt(FMath::Sqrt(static_cast<float>(NumBlocks)));
	TArray<FColor>	AllocationVisualization;
	AllocationVisualization.AddZeroed(Dimension * Dimension);
	int64				VisIndex = 0;

	// Traverse linked list and gather allocation information.
	FMemoryChunk*	CurrentChunk = FirstChunk;
	while (CurrentChunk)
	{
		FColor VisColor;
		// Free chunk.
		if (CurrentChunk->bIsAvailable)
		{
			NumFreeChunks++;
			FreeSize += CurrentChunk->Size;
			VisColor = FColor(0, 255, 0);
		}
		// Allocated chunk.
		else
		{
			NumUsedChunks++;
			UsedSize += CurrentChunk->Size;

			// Slightly alternate coloration to also visualize allocation sizes.
			if (NumUsedChunks % 2 == 0)
			{
				VisColor = FColor(255, 0, 0);
			}
			else
			{
				VisColor = FColor(192, 0, 0);
			}
		}

		for (int32 i = 0; i < (CurrentChunk->Size / AllocationAlignment); i++)
		{
			AllocationVisualization[VisIndex++] = VisColor;
		}

		CurrentChunk = CurrentChunk->NextChunk;
	}

	check(UsedSize == AllocatedMemorySize);
	check(FreeSize == AvailableMemorySize);

	// Write out bitmap for visualization of fragmentation and allocation patterns.
	//appCreateBitmap(TEXT("..\\..\\Binaries\\TextureMemory"), Dimension, Dimension, AllocationVisualization.GetTypedData());
	Ar.Logf(TEXT("BestFitAllocator: Allocated %i KByte in %i chunks, leaving %i KByte in %i chunks."), UsedSize / 1024, NumUsedChunks, FreeSize / 1024, NumFreeChunks);
	Ar.Logf(TEXT("BestFitAllocator: %5.2f ms in allocator"), TimeSpentInAllocator * 1000);
}


/**
* Helper function to fill in one gradient bar in the texture, for memory visualization purposes.
*/
void FillVizualizeData(FColor* TextureData, int32& X, int32& Y, int32& NumBytes, const FColor& Color1, const FColor& Color2, const int32 SizeX, const int32 SizeY, const int32 Pitch, const int32 PixelSize)
{
	// Fill pixels with a color gradient that represents the current allocation type.
	int32 MaxPixelIndex = FMath::Max((NumBytes - 1) / PixelSize, 1);
	int32 PixelIndex = 0;
	while (NumBytes > 0)
	{
		FColor& PixelColor = TextureData[Y*Pitch + X];
		PixelColor.R = (int32(Color1.R) * PixelIndex + int32(Color2.R) * (MaxPixelIndex - PixelIndex)) / MaxPixelIndex;
		PixelColor.G = (int32(Color1.G) * PixelIndex + int32(Color2.G) * (MaxPixelIndex - PixelIndex)) / MaxPixelIndex;
		PixelColor.B = (int32(Color1.B) * PixelIndex + int32(Color2.B) * (MaxPixelIndex - PixelIndex)) / MaxPixelIndex;
		PixelColor.A = 255;
		if (++X >= SizeX)
		{
			X = 0;
			if (++Y >= SizeY)
			{
				break;
			}
		}
		PixelIndex++;
		NumBytes -= PixelSize;
	}
}

/**
* Fills a texture with to visualize the texture pool memory.
*
* @param	TextureData		Start address
* @param	SizeX			Number of pixels along X
* @param	SizeY			Number of pixels along Y
* @param	Pitch			Number of bytes between each row
* @param	PixelSize		Number of bytes each pixel represents
*
* @return true if successful, false otherwise
*/
bool FGPUDefragAllocator::GetTextureMemoryVisualizeData(FColor* TextureData, int32 SizeX, int32 SizeY, int32 Pitch, const int32 PixelSize)
{
	check(Align(Pitch, sizeof(FColor)) == Pitch);
	Pitch /= sizeof(FColor);
	FColor TypeColor[2][6] =
		//  Allocated:           Free:             Locked:          Relocating:        Resizing:		Resized:
	{ { FColor(220, 220, 220), FColor(50, 50, 50), FColor(220, 0, 0), FColor(220, 220, 0), FColor(0, 220, 0), FColor(0, 140, 0) },
	{ FColor(180, 180, 180), FColor(50, 50, 50), FColor(180, 0, 0), FColor(180, 180, 0), FColor(0, 180, 0), FColor(0, 50, 0) } };
	int32 X = 0;
	int32 Y = 0;
	int32 NumBytes = 0;
	EMemoryElementType CurrentType = MET_Allocated;
	FMemoryChunk* Chunk = FirstChunk;
	FMemoryChunk* CurrentChunk = nullptr;
	bool bIsColoringUsableRelocatingMemory = false;
	while (Chunk && Y < SizeY)
	{
		EMemoryElementType ChunkType = GetChunkType(Chunk);

		// Fill pixels with a color gradient that represents the current allocation type.
		FColor Color1 = TypeColor[0][CurrentType];
		FColor Color2 = TypeColor[1][CurrentType];

		// Special case for relocating chunks, to show it in two color gradient bars.
		if (CurrentType == MET_Relocating)
		{
			// First, color the FMemoryChunk::SyncSize part of the chunk.
			int32 UsableMemorySize = CurrentChunk->Size - CurrentChunk->SyncSize;
			NumBytes -= UsableMemorySize;
			FillVizualizeData(TextureData, X, Y, NumBytes, Color1, Color2, SizeX, SizeY, Pitch, PixelSize);

			// Second, color the rest (immediately usable) part of the chunk.
			NumBytes += UsableMemorySize;
			Color1 = TypeColor[0][MET_Relocating];
			Color2 = TypeColor[1][MET_Resized];
		}

		FillVizualizeData(TextureData, X, Y, NumBytes, Color1, Color2, SizeX, SizeY, Pitch, PixelSize);

		CurrentType = ChunkType;
		CurrentChunk = Chunk;
		NumBytes += Chunk->Size;
		Chunk = Chunk->NextChunk;
	}

	// Fill rest of pixels with black.
	int32 NumRemainingPixels = SizeY * Pitch - (Y*Pitch + X);
	if (NumRemainingPixels > 0)
	{
		FMemory::Memzero(&TextureData[Y*Pitch + X], NumRemainingPixels * sizeof(FColor));
	}

	return true;
}

void FGPUDefragAllocator::GetMemoryLayout(TArray<FMemoryLayoutElement>& MemoryLayout)
{
	FMemoryChunk* Chunk = FirstChunk;
	MemoryLayout.Empty(512);
	while (Chunk)
	{
		EMemoryElementType ChunkType = GetChunkType(Chunk);
		MemoryLayout.Emplace(Chunk->Size, ChunkType);
		Chunk = Chunk->NextChunk;
	}
}

FGPUDefragAllocator::EMemoryElementType FGPUDefragAllocator::GetChunkType(FMemoryChunk* Chunk) const
{
	EMemoryElementType ChunkType;
	if (Chunk == nullptr)
	{
		ChunkType = MET_Max;			// End-of-memory (n/a)
	}
	else if (Chunk->IsRelocating())
	{
		ChunkType = MET_Relocating;		// Currently being relocated (yellow)
	}
	else if (Chunk->bIsAvailable)
	{
		ChunkType = MET_Free;			// Free (dark grey)
	}	
	else if (CanRelocate(Chunk) == false)
	{
		ChunkType = MET_Locked;			// Allocated but can't me relocated at this time (locked) (red)
	}
	else
	{
		ChunkType = MET_Allocated;
	}
	return ChunkType;
}

/**
* Scans the free chunks and returns the largest size you can allocate.
*
* @param OutNumFreeChunks	Upon return, contains the total number of free chunks. May be nullptr.
* @return					The largest size of all free chunks.
*/
int32 FGPUDefragAllocator::GetLargestAvailableAllocation(int32* OutNumFreeChunks/*=nullptr*/)
{
	FScopeLock SyncLock(&SynchronizationObject);
	int32 NumFreeChunks = 0;
	int64 LargestChunkSize = 0;
	FMemoryChunk* FreeChunk = FirstFreeChunk;
	while (FreeChunk)
	{
		NumFreeChunks++;
		LargestChunkSize = FMath::Max(LargestChunkSize, FreeChunk->Size);
		FreeChunk = FreeChunk->NextFreeChunk;
	}
	if (OutNumFreeChunks)
	{
		*OutNumFreeChunks = NumFreeChunks;
	}
	return LargestChunkSize;
}

/**
* Fully defragments the memory and blocks until it's done.
*
* @param Stats			[out] Stats
*/
void FGPUDefragAllocator::DefragmentMemory(FRelocationStats& Stats)
{
	double StartTime = FPlatformTime::Seconds();

	Tick(Stats, true);

	double MidTime = FPlatformTime::Seconds();

	if (Stats.NumRelocations > 0)
	{
		BlockOnFence();
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	int32 NumHolesBefore = Stats.NumHoles;
	int32 NumHolesAfter = 0;
	int64 LargestHoleBefore = Stats.LargestHoleSize;
	int64 LargestHoleAfter = GetLargestAvailableAllocation(&NumHolesAfter);
	double EndTime = FPlatformTime::Seconds();
	double TotalDuration = EndTime - StartTime;
	double GPUDuration = EndTime - MidTime;
	LargestHoleAfter = GetLargestAvailableAllocation(&NumHolesAfter);
	UE_LOGF(LogRHI, Warning, "DEFRAG: %.1f ms (GPU %.1f ms), Available: %.3f MB, NumRelocations: %d, Relocated: %.3f MB, NumHolesBefore: %d, NumHolesAfter: %d, LargestHoleBefore: %.3f MB, LargestHoleAfter: %.3f MB",
		TotalDuration*1000.0, GPUDuration*1000.0, AvailableMemorySize / 1024.0f / 1024.0f, Stats.NumRelocations, float(Stats.NumBytesRelocated) / 1024.0f / 1024.0f, NumHolesBefore, NumHolesAfter, float(LargestHoleBefore) / 1024.f / 1024.0f, float(LargestHoleAfter) / 1024.0f / 1024.0f);
#endif
}

/**
* Merges any adjacent free chunks into the specified free chunk.
* Doesn't affect the free-list sort order.
*
* @param	FreedChunk	Chunk that just became available.
*/
void FGPUDefragAllocator::Coalesce(FMemoryChunk* FreedChunk)
{
	check(FreedChunk);
	check(!FreedChunk->IsLocked())

	uint32 LatestSyncIndex = 0;
	int64 LatestSyncSize = 0;
	int64 LeftSize = 0;
	int64 RightSize = 0;

	// Check if the previous chunk is available.
	FMemoryChunk* LeftChunk = FreedChunk->PreviousChunk;
	if (LeftChunk && LeftChunk->bIsAvailable)
	{
		check(!LeftChunk->IsLocked());
		LeftSize = LeftChunk->Size;

		// Update relocation data for the left chunk.
		if (LeftChunk->IsRelocating())
		{			
			LatestSyncIndex = LeftChunk->SyncIndex;
			LatestSyncSize = LeftChunk->SyncSize;
		}

		// Deletion will unlink.
		delete LeftChunk;
	}

	// Update relocation data for the middle chunk.
	if (FreedChunk->IsRelocating())
	{
		LatestSyncIndex = FMath::Max(LatestSyncIndex, FreedChunk->SyncIndex);
		LatestSyncSize = LeftSize + FreedChunk->SyncSize;
	}

	// Check if the next chunk is available.
	FMemoryChunk* RightChunk = FreedChunk->NextChunk;
	if (RightChunk && RightChunk->bIsAvailable)
	{
		check(!RightChunk->IsLocked());
		RightSize = RightChunk->Size;

		// Update relocation data for the right chunk.
		if (RightChunk->IsRelocating())
		{
			LatestSyncIndex = FMath::Max(LatestSyncIndex, RightChunk->SyncIndex);
			LatestSyncSize = LeftSize + FreedChunk->Size + RightChunk->SyncSize;
		}

		// Deletion will unlink.
		delete RightChunk;
	}

#if TRACK_RELOCATIONS
	if (GGPUDefragDumpRelocationsToTTY)
	{
		printf("FreeChunk Before Coalesce: 0x%p, %i, %i, %i\n", FreedChunk->Base, (int32)FreedChunk->Size, (int32)FreedChunk->SyncSize, (int32)FreedChunk->SyncIndex);
	}
#endif

	// Merge.
	FreedChunk->Base -= LeftSize;
	FreedChunk->Size += LeftSize + RightSize;
	FreedChunk->SetSyncIndex(LatestSyncIndex, LatestSyncSize);

#if TRACK_RELOCATIONS
	if (GGPUDefragDumpRelocationsToTTY)
	{
		printf("FreeChunk AFter Coalesce: 0x%p, %i, %i, %i\n", FreedChunk->Base, (int32)FreedChunk->Size, (int32)FreedChunk->SyncSize, (int32)FreedChunk->SyncIndex);
	}
#endif
}

/**
* Performs a benchmark of the allocator and outputs the result to the log.
*
* @param MinChunkSize	Minimum number of bytes per random chunk
* @param MaxChunkSize	Maximum number of bytes per random chunk
* @param FreeRatio		Free 0.0-1.0 % of the memory before benchmarking
* @param LockRatio		Lock 0.0-1.0 % of the memory before benchmarking
* @param bFullDefrag	Whether to test full defrag (true) or continuous defrag (false)
* @param bSaveImages	Whether to save before/after images to hard disk (TexturePoolBenchmark-*.bmp)
* @param Filename		[opt] Filename to a previously saved memory layout to use for benchmarking, or nullptr
*/
void FGPUDefragAllocator::Benchmark(int32 MinChunkSize, int32 MaxChunkSize, float FreeRatio, float LockRatio, bool bFullDefrag, bool bSaveImages, const TCHAR* Filename)
{
//untested
#if 0
#if !(UE_BUILD_SHIPPING ||UE_BUILD_TEST)
	int64 OriginalAllocatedMemorySize = AllocatedMemorySize;
	int64 OriginalAvailableMemorySize = AvailableMemorySize;
	FRandomStream Rand(0x1ee7c0de);
	TArray<void*> Chunks;
	TArray<FMemoryChunk*> OriginalChunks;
	TArray<FMemoryChunk*> TempLockedChunks;
	Chunks.Reserve(512);
	OriginalChunks.Reserve(512);
	TempLockedChunks.Reserve(512);

	// Lock existing chunks (that aren't already locked) and save them off.
	FMemoryChunk* OriginalChunk = FirstChunk;
	while (OriginalChunk)
	{
		if (!OriginalChunk->bIsAvailable && !OriginalChunk->IsLocked())
		{
			OriginalChunk->LockCount++;
			OriginalChunks.Add(OriginalChunk);
		}
		OriginalChunk = OriginalChunk->NextChunk;
	}
	
	TArray<FMemoryLayoutElement> MemoryLayout;
	if (Filename)
	{
		FArchive* Ar = IFileManager::Get().CreateFileReader(Filename);
		if (Ar)
		{
			*Ar << MemoryLayout;
			delete Ar;
		}
	}

	if (MemoryLayout.Num() > 0)
	{
		// Try to recreate the memory layout.
		for (int32 LayoutIndex = 0; LayoutIndex < MemoryLayout.Num(); ++LayoutIndex)
		{
			FMemoryLayoutElement& LayoutElement = MemoryLayout[LayoutIndex];
			void* Ptr = Allocate(LayoutElement.Size, 1, true);
			if (Ptr)
			{
				Chunks.Add(Ptr);
			}
			else
			{
				break;
			}
		}

		// Set the type for the elements
		for (int32 LayoutIndex = 0; LayoutIndex < Chunks.Num(); ++LayoutIndex)
		{
			FMemoryLayoutElement& LayoutElement = MemoryLayout[LayoutIndex];
			switch (LayoutElement.Type)
			{
			case MET_Free:
			{
							 Free(Chunks[LayoutIndex]);
							 break;
			}
			case MET_Relocating:
			case MET_Resized:
			case MET_Locked:
			{
							   FMemoryChunk* Chunk = PointerToChunkMap.FindRef(Chunks[LayoutIndex]);
							   Chunk->LockCount++;
							   TempLockedChunks.Add(Chunk);
							   break;
			}
			}
		}
	}
	else
	{
		// Fill memory with random 32-1024 KB chunks.
		void* Ptr = nullptr;
		do
		{
			int32 ChunkSize = Align((FMath::TruncToInt((MaxChunkSize - MinChunkSize)*Rand.GetFraction()) + MinChunkSize), 4096);
			Ptr = Allocate(ChunkSize, 1, true);
			if (Ptr)
			{
				Chunks.Add(Ptr);
			}
		} while (Ptr);

		// Free some of the pool to create random holes.
		int32 SizeToFree = FMath::TruncToInt(FreeRatio * MemorySize);
		while (SizeToFree > 0 && Chunks.Num() > 0)
		{
			int32 ChunkIndex = FMath::TruncToInt(Chunks.Num() * Rand.GetFraction());
			void* FreePtr = Chunks[ChunkIndex];
			int32 ChunkSize = GetAllocatedSize(FreePtr);
			Free(FreePtr);
			Chunks.RemoveAtSwap(ChunkIndex);
			SizeToFree -= ChunkSize;
		}

		// Lock some random chunks.
		int32 SizeToLock = FMath::TruncToInt(LockRatio * MemorySize);
		while (SizeToLock > 0 && Chunks.Num() > 0)
		{
			int32 ChunkIndex = FMath::TruncToInt(Chunks.Num() * Rand.GetFraction());
			void* ChunkPtr = Chunks[ChunkIndex];
			FMemoryChunk* Chunk = PointerToChunkMap.FindRef(ChunkPtr);
			Chunk->LockCount++;
			TempLockedChunks.Add(Chunk);
			SizeToLock -= Chunk->Size;
		}
	}

	bBenchmarkMode = true;

	// Save a "before" image
	TArray<FColor> Bitmap;
	if (bSaveImages)
	{
		Bitmap.AddZeroed(256 * 256);
		GetTextureMemoryVisualizeData(Bitmap.GetData(), 256, 256, 256 * sizeof(FColor), 4096);
		//appCreateBitmap(TEXT("TexturePoolBenchmark-Before.bmp"), 256, 256, Bitmap.GetTypedData());
	}

	// Keep Ticking until there are no more holes.
	int32 NumTicks = 0;
	int32 NumBytesRelocated = 0;
	TArray<int32> LargestHole;
	TArray<double> Timers;
	LargestHole.Reserve(4096);
	Timers.Reserve(4096);
	int32 Result = 1;
	double StartTime = FPlatformTime::Seconds();
	while (Result && FirstFreeChunk && FirstFreeChunk->NextFreeChunk != nullptr)
	{
		FGPUDefragAllocator::FRelocationStats Stats;
		Result = Tick(Stats, bFullDefrag);
		LargestHole.Add(Stats.LargestHoleSize);
		Timers.Add(FPlatformTime::Seconds());
		NumBytesRelocated += Stats.NumBytesRelocated;
		NumTicks++;

		// If benchmarking a specific memory layout, only iterate one time.
		if (Filename)
		{
			Result = 0;
		}
	}
	double Duration = FPlatformTime::Seconds() - StartTime;
	FinishAllRelocations();

	// Time two "empty" runs.
	double StartTime2 = FPlatformTime::Seconds();
	for (int32 ExtraTiming = 0; ExtraTiming < 2; ++ExtraTiming)
	{
		FGPUDefragAllocator::FRelocationStats Stats;
		Result = Tick(Stats, false);
	}
	double Duration2 = (FPlatformTime::Seconds() - StartTime2) / 2;

	// Save an "after" image
	if (bSaveImages)
	{
		GetTextureMemoryVisualizeData(Bitmap.GetData(), 256, 256, 256 * sizeof(FColor), 4096);
		//appCreateBitmap(TEXT("TexturePoolBenchmark-After.bmp"), 256, 256, Bitmap.GetTypedData());
	}

	bBenchmarkMode = false;

	// What's the defragmentation after 0%, 25%, 50%, 75%, 100% of the Ticks?
	float DefragmentationRatios[5];
	DefragmentationRatios[0] = LargestHole[FMath::TruncToInt(LargestHole.Num() * 0.00f)] / float(AvailableMemorySize);
	DefragmentationRatios[1] = LargestHole[FMath::TruncToInt(LargestHole.Num() * 0.25f)] / float(AvailableMemorySize);
	DefragmentationRatios[2] = LargestHole[FMath::TruncToInt(LargestHole.Num() * 0.50f)] / float(AvailableMemorySize);
	DefragmentationRatios[3] = LargestHole[FMath::TruncToInt(LargestHole.Num() * 0.75f)] / float(AvailableMemorySize);
	DefragmentationRatios[4] = LargestHole[FMath::TruncToInt(LargestHole.Num() - 1)] / float(AvailableMemorySize);
	float Durations[5];
	Durations[0] = Timers[FMath::TruncToInt(Timers.Num() * 0.00f)] - StartTime;
	Durations[1] = Timers[FMath::TruncToInt(Timers.Num() * 0.25f)] - StartTime;
	Durations[2] = Timers[FMath::TruncToInt(Timers.Num() * 0.50f)] - StartTime;
	Durations[3] = Timers[FMath::TruncToInt(Timers.Num() * 0.75f)] - StartTime;
	Durations[4] = Timers[FMath::TruncToInt(Timers.Num() - 1)] - StartTime;

	// Unlock our temp chunks we locked before.
	for (int32 ChunkIndex = 0; ChunkIndex < TempLockedChunks.Num(); ++ChunkIndex)
	{
		FMemoryChunk* TempChunk = TempLockedChunks[ChunkIndex];
		TempChunk->LockCount--;
	}
	// Free all unlocked chunks (our temp chunks).
	FMemoryChunk* Chunk = FirstChunk;
	while (Chunk)
	{
		if (!Chunk->bIsAvailable && !Chunk->IsLocked())
		{
			FreeChunk(Chunk, false);
		}
		Chunk = Chunk->NextChunk;
	}
	// Unlock the original chunks we locked before.
	for (int32 ChunkIndex = 0; ChunkIndex < OriginalChunks.Num(); ++ChunkIndex)
	{
		FMemoryChunk* OriginalChunkUnlock = OriginalChunks[ChunkIndex];
		OriginalChunkUnlock->LockCount--;
	}

	UE_LOGF(LogRHI, Warning, "Defrag benchmark: %.1f ms, %d ticks, additional %.1f ms/tick, %.1f MB relocated, defragmentation: %.1f%% (%.1f ms), %.1f%% (%.1f ms), %.1f%% (%.1f ms), %.1f%% (%.1f ms), %.1f%% (%.1f ms)",
		Duration*1000.0,
		NumTicks,
		Duration2*1000.0,
		NumBytesRelocated / 1024.0f / 1024.0f,
		DefragmentationRatios[0] * 100.0f,
		Durations[0] * 1000.0,
		DefragmentationRatios[1] * 100.0f,
		Durations[1] * 1000.0,
		DefragmentationRatios[2] * 100.0f,
		Durations[2] * 1000.0,
		DefragmentationRatios[3] * 100.0f,
		Durations[3] * 1000.0,
		DefragmentationRatios[4] * 100.0f,
		Durations[4] * 1000.0);

	GLog->Flush();
	check(OriginalAllocatedMemorySize == AllocatedMemorySize && OriginalAvailableMemorySize == AvailableMemorySize);
#endif
#endif
}


/*-----------------------------------------------------------------------------
FBestFitAllocator implementation.
-----------------------------------------------------------------------------*/

#if 0
/**
* Allocates texture memory.
*
* @param	Size			Size of allocation
* @param	bAllowFailure	Whether to allow allocation failure or not
* @returns					Pointer to allocated memory
*/
void* FPresizedMemoryPool::Allocate(uint32 Size, bool bAllowFailure)
{
	FScopeLock ScopeLock(&SynchronizationObject);

#if DUMP_ALLOC_FREQUENCY
	static int32 AllocationCounter;
	if (++AllocationCounter % DUMP_ALLOC_FREQUENCY == 0)
	{
		FBestFitAllocator::DumpAllocs();
	}
#endif

	// Initialize allocator if it hasn't already.
	if (!FGPUDefragAllocator::IsInitialized())
	{
	}

	// actually do the allocation
	void* Pointer = FGPUDefragAllocator::Allocate(Size, bAllowFailure);

	// We ran out of memory. Instead of crashing rather corrupt the content and display an error message.
	if (Pointer == nullptr)
	{
		if (!bAllowFailure)
		{
			// Mark texture memory as having been corrupted.
			bIsCorrupted = true;
		}

		// Use special pointer, which is being identified by free.
		Pointer = AllocationFailurePointer;
	}

#if LOG_EVERY_ALLOCATION
	int32 AllocSize, AvailSize, PendingMemoryAdjustment;
	FGPUDefragAllocator::GetMemoryStats(AllocSize, AvailSize, PendingMemoryAdjustment);
	debugf(TEXT("Texture Alloc: %p  Size: %6i     Alloc: %8i Avail: %8i Pending: %8i"), Pointer, Size, AllocSize, AvailSize, PendingMemoryAdjustment);
#endif
	return Pointer;
}


/**
* Frees texture memory allocated via Allocate
*
* @param	Pointer		Allocation to free
*/
void FPresizedMemoryPool::Free(void* Pointer)
{
	FScopeLock ScopeLock(&SynchronizationObject);

#if LOG_EVERY_ALLOCATION
	int32 AllocSize, AvailSize, PendingMemoryAdjustment;
	FGPUDefragAllocator::GetMemoryStats(AllocSize, AvailSize, PendingMemoryAdjustment);
	debugf(TEXT("Texture Free : %p   Before free     Alloc: %8i Avail: %8i Pending: %8i"), Pointer, AllocSize, AvailSize, PendingMemoryAdjustment);
#endif
	// we never want to free the special pointer
	if (Pointer != AllocationFailurePointer)
	{
		// do the free
		FGPUDefragAllocator::Free(Pointer);
	}
}

/**
* Returns the amount of memory allocated for the specified address.
*
* @param	Pointer		Pointer to check.
* @return				Number of bytes allocated
*/
int32 FPresizedMemoryPool::GetAllocatedSize(void* Pointer)
{
	return FGPUDefragAllocator::GetAllocatedSize(Pointer);
}

/**
* Tries to reallocate texture memory in-place (without relocating),
* by adjusting the base address of the allocation but keeping the end address the same.
*
* @param	OldBaseAddress	Pointer to the original allocation
* @returns	New base address if it succeeded, otherwise nullptr
**/
void* FPresizedMemoryPool::Reallocate(void* OldBaseAddress, int32 NewSize)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	// Initialize allocator if it hasn't already.
	if (!FGPUDefragAllocator::IsInitialized())
	{
	}

	// we never want to reallocate the special pointer
	if (IsValidTextureData(OldBaseAddress))
	{
		// Actually try to do the reallocation.
		return FGPUDefragAllocator::Reallocate(OldBaseAddress, NewSize);
	}
	else
	{
		return nullptr;
	}
}

/**
* Requests an async allocation or reallocation.
* The caller must hold on to the request until it has been completed or canceled.
*
* @param ReallocationRequest	The request
* @param bForceRequest			If true, the request will be accepted even if there's currently not enough free space
* @return						true if the request was accepted
*/
bool FPresizedMemoryPool::AsyncReallocate(FAsyncReallocationRequest* ReallocationRequest, bool bForceRequest)
{
	FScopeLock ScopeLock(&SynchronizationObject);

	// Initialize allocator if it hasn't already.
	if (!FGPUDefragAllocator::IsInitialized())
	{
	}

	// we never want to reallocate the special pointer
	if (ReallocationRequest->GetOldBaseAddress() != AllocationFailurePointer)
	{
		// Actually try to do the reallocation.
		return FGPUDefragAllocator::AsyncReallocate(ReallocationRequest, bForceRequest);
	}
	else
	{
		return false;
	}
}

/**
* Partially defragments the memory and tries to process all async reallocation requests at the same time.
* Call this once per frame.
*
* @param Stats			[out] Stats
* @param bPanicDefrag	If true, performs a full defrag and ignores all reallocation requests
*/
int32 FPresizedMemoryPool::Tick(FRelocationStats& Stats)
{
	uint32 StartTime = FPlatformTime::Cycles();

	FScopeLock ScopeLock(&SynchronizationObject);

	// Initialize allocator if it hasn't already.
	if (!FGPUDefragAllocator::IsInitialized())
	{
	}

	int32 Result = FGPUDefragAllocator::Tick(Stats, false);

	TickCycles = FPlatformTime::Cycles() - StartTime - BlockedCycles;

	SET_DWORD_STAT(STAT_TexturePool_RelocatedSize, Stats.NumBytesRelocated);
	SET_DWORD_STAT(STAT_TexturePool_NumRelocations, Stats.NumRelocations);
	SET_DWORD_STAT(STAT_TexturePool_Allocated, AllocatedMemorySize);
	SET_DWORD_STAT(STAT_TexturePool_Free, AvailableMemorySize);
	SET_DWORD_STAT(STAT_TexturePool_LargestHole, Stats.LargestHoleSize);
	SET_DWORD_STAT(STAT_TexturePool_NumHoles, Stats.NumHoles);

	SET_DWORD_STAT(STAT_TexturePool_TotalAsyncReallocations, NumFinishedAsyncReallocations);
	SET_DWORD_STAT(STAT_TexturePool_TotalAsyncAllocations, NumFinishedAsyncAllocations);
	SET_DWORD_STAT(STAT_TexturePool_TotalAsyncCancellations, NumCanceledAsyncRequests);
	SET_CYCLE_COUNTER(STAT_TexturePool_DefragTime, TickCycles);

	return Result;
}
#endif
