// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUProfiler.h: Hierarchical GPU Profiler Implementation.
=============================================================================*/

#include "GPUProfiler.h"
#include "Async/TaskGraphInterfaces.h"
#include "Misc/WildcardString.h"
#include "Misc/CommandLine.h"
#include "RHI.h"
#include "DynamicRHI.h"
#include "GpuProfilerTrace.h"
#include "Containers/AnsiString.h"
#include "Stats/StatsData.h"

#if !UE_BUILD_SHIPPING
#include "VisualizerEvents.h"
#include "ProfileVisualizerModule.h"
#include "Modules/ModuleManager.h"
#endif

#define LOCTEXT_NAMESPACE "GpuProfiler"

enum class EGPUProfileSortMode
{
	Chronological,
	TimeElapsed,
	NumPrims,
	NumVerts,

	Max
};

static TAutoConsoleVariable<int32> GCVarProfileGPU_Sort(
	TEXT("r.ProfileGPU.Sort"),
	0,
	TEXT("Sorts the TTY Dump independently at each level of the tree in various modes.\n")
	TEXT("0 : Chronological\n")
	TEXT("1 : By time elapsed\n")
	TEXT("2 : By number of prims\n")
	TEXT("3 : By number of verts\n"),
	ECVF_Default);

static TAutoConsoleVariable<FString> GCVarProfileGPU_Root(
	TEXT("r.ProfileGPU.Root"),
	TEXT("*"),
	TEXT("Allows to filter the tree when using ProfileGPU, the pattern match is case sensitive."),
	ECVF_Default);

static TAutoConsoleVariable<float> GCVarProfileGPU_ThresholdPercent(
	TEXT("r.ProfileGPU.ThresholdPercent"),
	0.0f,
	TEXT("Percent of the total execution duration the event needs to be larger than to be printed."),
	ECVF_Default);

static TAutoConsoleVariable<bool> GCVarProfileGPU_TableFormatting(
	TEXT("r.ProfileGPU.TableFormatting"),
	true,
	TEXT("When enabled, the output results will be formatted in a table with many secondary stats. When disabled, only inclusive times and event names are printed in an indented list for compactness."),
	ECVF_Default);

static TAutoConsoleVariable<bool> GCVarProfileGPU_UnicodeOutput(
	TEXT("r.ProfileGPU.UnicodeOutput"),
	true,
	TEXT("When enabled, the output results will be formatted in a unicode table."),
	ECVF_Default);

static TAutoConsoleVariable<bool> GCVarProfileGPU_ShowLeafEvents(
	TEXT("r.ProfileGPU.ShowLeafEvents"),
	true,
	TEXT("Allows profileGPU to display event-only leaf nodes with no draws associated."),
	ECVF_Default);

static TAutoConsoleVariable<bool> GCVarProfileGPU_ShowHeader(
	TEXT("r.ProfileGPU.ShowHeader"),
	true,
	TEXT("When true, prints a summary of the profileGPU settings before the report table in the log."),
	ECVF_Default);

static TAutoConsoleVariable<bool> GCVarProfileGPU_ShowEmptyQueues(
	TEXT("r.ProfileGPU.ShowEmptyQueues"),
	true,
	TEXT("When true, GPU queues without any registered work are still displayed in the report tables."),
	ECVF_Default);

static TAutoConsoleVariable<bool> GCVarProfileGPU_ShowStats(
	TEXT("r.ProfileGPU.ShowStats"),
	true,
	TEXT("When true, additional stat columns are shown in the report (numbers of draws, dispatches, vertices and primitives)."),
	ECVF_Default);

static TAutoConsoleVariable<bool> GCVarProfileGPU_ShowPercentColumn(
	TEXT("r.ProfileGPU.ShowPercentColumn"),
	true,
	TEXT("When true, a column showing the relative portion of time each stat takes as a percentage is displayed, including a visual unicode bar when unicode output is enabled."),
	ECVF_Default);

static TAutoConsoleVariable<bool> GCVarProfileGPU_ShowInclusive(
	TEXT("r.ProfileGPU.ShowInclusive"),
	true,
	TEXT("When true, inclusive GPU times are shown."),
	ECVF_Default);

static TAutoConsoleVariable<bool> GCVarProfileGPU_ShowExclusive(
	TEXT("r.ProfileGPU.ShowExclusive"),
	true,
	TEXT("When true, exclusive GPU times are shown."),
	ECVF_Default);

static TAutoConsoleVariable<bool> GCVarProfileGPU_ShowUI(
	TEXT("r.ProfileGPU.ShowUI"),
	true,
	TEXT("Whether the user interface profiler should be displayed after profiling the GPU.\n")
	TEXT("The results will always go to the log/console."),
	ECVF_Default);

static TAutoConsoleVariable<int> CVarGPUCsvStatsEnabled(
	TEXT("r.GPUCsvStatsEnabled"),
	0,
	TEXT("Enables or disables GPU stat recording to CSVs"));

CSV_DEFINE_CATEGORY(RHI, true);

#if UE_BUILD_SHIPPING
	CSV_DEFINE_CATEGORY(DrawCall, false);
#else
	CSV_DEFINE_CATEGORY(DrawCall, true);
#endif

namespace UE::RHI::GPUProfiler
{
	FEvent::FFrameBoundary::FFrameBoundary(ERHIPipeline Pipeline, FRHIEndFrameArgs const& Args, uint64 CPUTimestamp)
		: CPUTimestamp     (CPUTimestamp)
		, FrameNumber	   (Args.FrameNumber)
		, bProfileNextFrame(Args.bProfileNextFrame)
	#if STATS
		, bStatsFrameSet   (Args.StatsFrame.IsSet())
		, StatsFrame       (Args.StatsFrame.IsSet() ? *Args.StatsFrame : 0)
	#endif
	#if WITH_RHI_BREADCRUMBS
		, Breadcrumb	   ((Pipeline != ERHIPipeline::None) ? Args.GPUBreadcrumbs[Pipeline] : nullptr)
	#endif
	{}

	TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> FEventStream::FChunk::MemoryPool;

	FEvent const* FEventStream::Peek() const
	{
		if (!First)
		{
			return nullptr;
		}

		return First->GetElement(First->Header.Iter);
	}

	void FEventStream::Pop()
	{
		++First->Header.Iter;

		if (First->Header.Iter == First->Header.Num)
		{
			FChunk* Next = First->Header.Next;
			if (!Next)
			{
				check(First == Current);
				Current = nullptr;
			}

			delete First;
			First = Next;

			check(!First || (First->Header.Iter == 0 && First->Header.Num > 0));
		}
		else
		{
			check(First->Header.Iter < First->Header.Num);
		}
	}

#if WITH_PROFILEGPU
	template <uint32 Width>
	struct TUnicodeHorizontalBar
	{
		TCHAR Text[Width + 1];

		// 0 <= Value <= 1
		TUnicodeHorizontalBar(double Value)
		{
			TCHAR* Output = Text;
			int32 Solid, Partial, Blank;
			{
				double Integer;
				double Remainder = FMath::Modf(FMath::Clamp(Value, 0.0, 1.0) * Width, &Integer);

				Solid = (int32)Integer;
				Partial = (int32)FMath::Floor(Remainder * 8);
				Blank = (Width - Solid - (Partial > 0 ? 1 : 0));
			}

			// Solid characters
			for (int32 Index = 0; Index < Solid; ++Index)
			{
				*Output++ = TEXT('█');
			}

			// Partially filled character
			if (Partial > 0)
			{
				static constexpr TCHAR const Data[] = TEXT("▏▎▍▌▋▊▉");
				*Output++ = Data[Partial - 1];
			}

			// Blank Characters to pad out the width
			for (int32 Index = 0; Index < Blank; ++Index)
			{
				*Output++ = TEXT(' ');
			}

			*Output++ = 0;
			check(uintptr_t(Output) == (uintptr_t(Text) + sizeof(Text)));
		}
	};

	struct FNode
	{
		FString Name;

		FNode* Parent = nullptr;
		FNode* Next = nullptr;

		TArray<FNode*> Children;

		struct FStats
		{
			uint32 NumDraws      = 0;
			uint32 NumDispatches = 0;
			uint32 NumPrimitives = 0;
			uint32 NumVertices   = 0;

			uint64 BusyCycles    = 0;
			uint64 IdleCycles    = 0;
			uint64 WaitCycles    = 0;

			double GetBusyMilliseconds() const
			{
				return FPlatformTime::ToMilliseconds64(BusyCycles);
			}

			double GetWaitMilliseconds() const
			{
				return FPlatformTime::ToMilliseconds64(WaitCycles);
			}

			bool HasWork() const
			{
				return NumDraws > 0 || NumDispatches > 0;
			}

			FStats& operator += (FStats const& Stats)
			{
				NumDraws      += Stats.NumDraws;
				NumDispatches += Stats.NumDispatches;
				NumPrimitives += Stats.NumPrimitives;
				NumVertices   += Stats.NumVertices;
				BusyCycles    += Stats.BusyCycles;
				IdleCycles    += Stats.IdleCycles;
				WaitCycles    += Stats.WaitCycles;
				return *this;
			}

			FStats& operator += (FEvent::FStats const& Stats)
			{
				NumDraws      += Stats.NumDraws;
				NumDispatches += Stats.NumDispatches;
				NumPrimitives += Stats.NumPrimitives;
				NumVertices   += Stats.NumVertices;

				return *this;
			}

			void Accumulate(uint64 Busy, uint64 Wait, uint64 Idle)
			{
				BusyCycles += Busy;
				IdleCycles += Idle;
				WaitCycles += Wait;
			}
		};

		// Exclusive stats for this node
		FStats Exclusive;

		// Sum of stats including all children
		FStats Inclusive;

		FNode(FString&& Name)
			: Name(MoveTemp(Name))
		{}
	};

	struct FTable
	{
		bool const bUnicodeOutput;
		bool const bShowStats;
		bool const bShowPercent;
		bool const bShowInclusive;
		bool const bShowExclusive;

		FTable()
			: bUnicodeOutput  (GCVarProfileGPU_UnicodeOutput    .GetValueOnAnyThread())
			, bShowStats      (GCVarProfileGPU_ShowStats        .GetValueOnAnyThread())
			, bShowPercent    (GCVarProfileGPU_ShowPercentColumn.GetValueOnAnyThread())
			, bShowInclusive  (GCVarProfileGPU_ShowInclusive    .GetValueOnAnyThread())
			, bShowExclusive  (GCVarProfileGPU_ShowExclusive    .GetValueOnAnyThread())
		{}

		enum class EColumn : uint32
		{
			Exclusive_NumDraws,
			Exclusive_NumDispatches,
			Exclusive_NumPrimitives,
			Exclusive_NumVertices,
			Exclusive_Percent,
			Exclusive_Time,

			Inclusive_NumDraws,
			Inclusive_NumDispatches,
			Inclusive_NumPrimitives,
			Inclusive_NumVertices,
			Inclusive_Percent,
			Inclusive_Time,

			Events,

			Num
		};

		uint32 GetColumnMinimumWidth(EColumn Column) const
		{
			switch (Column)
			{
			case EColumn::Events:
				return 6;
			}

			return 0;
		}

		TCHAR const* GetColumnHeader(EColumn Column) const
		{
			switch (Column)
			{
			case EColumn::Exclusive_NumDraws:
			case EColumn::Inclusive_NumDraws:
				return TEXT("Draws");

			case EColumn::Exclusive_NumDispatches:
			case EColumn::Inclusive_NumDispatches:
				return TEXT("Dsptch");
				
			case EColumn::Exclusive_NumPrimitives:
			case EColumn::Inclusive_NumPrimitives:
				return TEXT("Prim");

			case EColumn::Exclusive_NumVertices:
			case EColumn::Inclusive_NumVertices:
				return TEXT("Vert");

			case EColumn::Exclusive_Percent:
			case EColumn::Inclusive_Percent:
				return TEXT("Percent");
				
			case EColumn::Exclusive_Time:
			case EColumn::Inclusive_Time:
				return TEXT("Time");
			}

			return TEXT("");
		}

		uint32 GetColumnGroup(EColumn Column) const
		{
		switch (Column)
			{
			case EColumn::Exclusive_NumDraws:
			case EColumn::Exclusive_NumDispatches:
			case EColumn::Exclusive_NumPrimitives:
			case EColumn::Exclusive_NumVertices:
			case EColumn::Exclusive_Percent:
			case EColumn::Exclusive_Time:
				return 0;

			case EColumn::Inclusive_NumDraws:
			case EColumn::Inclusive_NumDispatches:
			case EColumn::Inclusive_NumPrimitives:
			case EColumn::Inclusive_NumVertices:
			case EColumn::Inclusive_Percent:
			case EColumn::Inclusive_Time:
				return 1;

			default:
			case EColumn::Events:
				return 2;
			}
		}

		TCHAR const* GetGroupName(uint32 GroupIndex) const
		{
			switch (GroupIndex)
			{
			case 0: return TEXT("Exclusive");
			case 1: return TEXT("Inclusive");
			case 2: return TEXT("Events");
			}
			return TEXT("");
		}

		uint32 NumRows = 0;
		TStaticArray<TArray<FString>, uint32(EColumn::Num)> Columns { InPlace };
		TArray<bool> RowBreaks;

		FString& Col(EColumn Column)
		{
			return Columns[uint32(Column)].Emplace_GetRef();
		}

		bool HasRows() const
		{
			return NumRows > 0;
		}

		void AddRow(FNode* Root, FNode::FStats const& Inclusive, FNode::FStats const& Exclusive, FString const& Name, uint32 Level)
		{
			double ExclusivePercent = double(Exclusive.BusyCycles) / Root->Inclusive.BusyCycles;
			double InclusivePercent = double(Inclusive.BusyCycles) / Root->Inclusive.BusyCycles;

			static constexpr uint32 BarWidth = 8;
			TUnicodeHorizontalBar<BarWidth> ExclusiveBar = ExclusivePercent;
			TUnicodeHorizontalBar<BarWidth> InclusiveBar = InclusivePercent;

			static constexpr TCHAR const BarSeparator[] = TEXT(" ┊ ");

			if (bShowExclusive)
			{
				if (bShowStats)
				{
					Col(EColumn::Exclusive_NumDraws     ) = FString::Printf(TEXT("%d"), Exclusive.NumDraws);
					Col(EColumn::Exclusive_NumDispatches) = FString::Printf(TEXT("%d"), Exclusive.NumDispatches);
					Col(EColumn::Exclusive_NumPrimitives) = FString::Printf(TEXT("%d"), Exclusive.NumPrimitives);
					Col(EColumn::Exclusive_NumVertices  ) = FString::Printf(TEXT("%d"), Exclusive.NumVertices);
				}

				if (bShowPercent)
				{
					Col(EColumn::Exclusive_Percent) = FString::Printf(TEXT("%.1f%%%s%s"), ExclusivePercent * 100.0, bUnicodeOutput ? BarSeparator : TEXT(""), bUnicodeOutput ? ExclusiveBar.Text : TEXT(""));
				}

				Col(EColumn::Exclusive_Time) = FString::Printf(TEXT("%.3f ms"), FPlatformTime::ToMilliseconds64(Exclusive.BusyCycles));
			}

			if (bShowInclusive)
			{
				if (bShowStats)
				{
					Col(EColumn::Inclusive_NumDraws     ) = FString::Printf(TEXT("%d"), Inclusive.NumDraws);
					Col(EColumn::Inclusive_NumDispatches) = FString::Printf(TEXT("%d"), Inclusive.NumDispatches);
					Col(EColumn::Inclusive_NumPrimitives) = FString::Printf(TEXT("%d"), Inclusive.NumPrimitives);
					Col(EColumn::Inclusive_NumVertices  ) = FString::Printf(TEXT("%d"), Inclusive.NumVertices);
				}

				if (bShowPercent)
				{
					Col(EColumn::Inclusive_Percent) = FString::Printf(TEXT("%.1f%%%s%s"), InclusivePercent * 100.0, bUnicodeOutput ? BarSeparator : TEXT(""), bUnicodeOutput ? InclusiveBar.Text : TEXT(""));
				}

				Col(EColumn::Inclusive_Time) = FString::Printf(TEXT("%.3f ms"), FPlatformTime::ToMilliseconds64(Inclusive.BusyCycles));
			}

			static constexpr uint32 SpacesPerIndent = 3;
			Col(EColumn::Events) = FString::Printf(TEXT("%*s"), Name.Len() + (Level * SpacesPerIndent), *Name);

			// Insert a horizontal rule before each root level row.
			RowBreaks.Add(Level == 0);

			NumRows++;
		}

		struct FChars
		{
			TCHAR const* Left;
			TCHAR const* GroupSeparator;
			TCHAR const* LastGroupSeparator;
			TCHAR const* Right;
			TCHAR const* CellSeparator;
		};

		struct FFormat
		{
			TCHAR const* LineMajor;
			TCHAR const* LineMinor;
			TCHAR const* Indent;

			FChars const TopRow;
			FChars const GroupNameRow;
			FChars const GroupBorderRow;
			FChars const ValueRow;
			FChars const DividorRow;
			FChars const BottomRow;
		};

		FString ToString() const
		{
			if (bUnicodeOutput)
			{
				static constexpr FFormat Unicode = 
				{
					.LineMajor  = TEXT("━"),
					.LineMinor  = TEXT("─"),
					.Indent     = TEXT("    "),

					//                 Left     GrpSep     LastGrp     Right     CellSep
					.TopRow        { TEXT("┏"), TEXT("┳"), TEXT("┳"), TEXT("┓"), TEXT(" ") },
					.GroupNameRow  { TEXT("┃"), TEXT("┃"), TEXT("┃"), TEXT("┃"), TEXT(" ") },
					.GroupBorderRow{ TEXT("┠"), TEXT("╂"), TEXT("┨"), TEXT("┃"), TEXT("┬") },
					.ValueRow      { TEXT("┃"), TEXT("┃"), TEXT("┃"), TEXT("┃"), TEXT("│") },
					.DividorRow    { TEXT("┠"), TEXT("╂"), TEXT("╂"), TEXT("┨"), TEXT("┼") },
					.BottomRow     { TEXT("┗"), TEXT("┻"), TEXT("┻"), TEXT("┛"), TEXT("┷") },
				};

				return ToStringInner(Unicode);
			}
			else
			{
				static constexpr FFormat Ascii = 
				{
					.LineMajor  = TEXT("-"),
					.LineMinor  = TEXT("-"),
					.Indent = TEXT("    "),

					//                 Left     GrpSep     LastGrp     Right     CellSep
					.TopRow        { TEXT("+"), TEXT("+"), TEXT("+"), TEXT("+"), TEXT(" ") },
					.GroupNameRow  { TEXT("|"), TEXT("|"), TEXT("|"), TEXT("|"), TEXT(" ") },
					.GroupBorderRow{ TEXT("+"), TEXT("+"), TEXT("+"), TEXT("|"), TEXT("+") },
					.ValueRow      { TEXT("|"), TEXT("|"), TEXT("|"), TEXT("|"), TEXT("|") },
					.DividorRow    { TEXT("+"), TEXT("+"), TEXT("+"), TEXT("+"), TEXT("+") },
					.BottomRow     { TEXT("+"), TEXT("+"), TEXT("+"), TEXT("+"), TEXT("+") },
				};

				return ToStringInner(Ascii);
			}
		}

		FString ToStringInner(FFormat const& Format) const
		{
			struct FGroup  { uint32 Index, Width; };
			struct FColumn { uint32 Index, Width; };

			static constexpr uint32 NumGroups = 3;
			static constexpr uint32 CellPadding = 1;

			// Auto-size column widths to their contents
			TStaticArray<int32, uint32(EColumn::Num)> ColumnWidths{ InPlace, 0 };
			for (uint32 ColumnIndex = 0; ColumnIndex < uint32(EColumn::Num); ++ColumnIndex)
			{
				if (Columns[ColumnIndex].Num() == 0)
					continue;

				check(Columns[ColumnIndex].Num() == NumRows);

				int32& Width = ColumnWidths[ColumnIndex];

				// Auto-size column width
				Width = GetColumnMinimumWidth(EColumn(ColumnIndex));
				Width = FMath::Max(Width, FCString::Strlen(GetColumnHeader(EColumn(ColumnIndex))));

				for (FString const& Cell : Columns[ColumnIndex])
				{
					Width = FMath::Max(Width, Cell.Len());
				}
			}

			FString Result;

			auto EmitGroupRow = [&](FChars const& Chars, TUniqueFunction<void(FGroup)> GroupCallback)
			{
				uint32 const CellSeparatorLength = FCString::Strlen(Chars.CellSeparator);

				Result += Format.Indent;
				Result += Chars.Left;

				uint32 GroupWidth = 0;
				uint32 GroupIndex = 0;

				for (uint32 ColumnIndex = 0; ColumnIndex < uint32(EColumn::Num); ++ColumnIndex)
				{
					if (Columns[ColumnIndex].Num() == 0)
						continue;

					GroupWidth += ColumnWidths[ColumnIndex] + CellPadding * 2;
					GroupIndex = GetColumnGroup(EColumn(ColumnIndex));

					if (GroupIndex != GetColumnGroup(EColumn(ColumnIndex + 1)))
					{ 
						// Group Change
						GroupCallback({ GroupIndex, GroupWidth });

						// Add the group separator character
						Result += GroupIndex < NumGroups - 2
							? Chars.GroupSeparator
							: Chars.LastGroupSeparator;

						GroupWidth = 0;
					}
					else if (ColumnIndex < uint32(EColumn::Num) - 1)
					{
						// Same group. Count the (missing) cell division
						GroupWidth += CellSeparatorLength;
					}
				}

				// Emit final group
				GroupCallback({ GroupIndex, GroupWidth });

				// Close the row
				Result += Chars.Right;
				Result += TEXT("\n");
			};

			auto EmitValueRow = [&](bool bGroupBorderRow, FChars const& Chars, TUniqueFunction<void(FColumn)> CellCallback)
			{
				Result += Format.Indent;

				uint32 FirstColumnIndex = 0;

				if (bGroupBorderRow)
				{
					// Find first visible column
					while (FirstColumnIndex < uint32(EColumn::Num) && Columns[FirstColumnIndex].Num() == 0)
					{
						++FirstColumnIndex;
					}

					Result += FirstColumnIndex < uint32(EColumn::Events)
						? Chars.Left
						: Chars.Right;
				}
				else
				{
					Result += Chars.Left;
				}

				for (uint32 ColumnIndex = FirstColumnIndex; ColumnIndex < uint32(EColumn::Num); ++ColumnIndex)
				{
					if (Columns[ColumnIndex].Num() == 0)
						continue;

					CellCallback({ ColumnIndex, ColumnWidths[ColumnIndex] + (CellPadding * 2) });

					if (ColumnIndex < uint32(EColumn::Num) - 1)
					{
						// Find next visible column
						uint32 NextColumnIndex = ColumnIndex + 1;
						while (NextColumnIndex < uint32(EColumn::Num) && Columns[NextColumnIndex].Num() == 0)
						{
							++NextColumnIndex;
						}

						uint32 CurrentGroupIndex = GetColumnGroup(EColumn(ColumnIndex));
						uint32 NextGroupIndex    = GetColumnGroup(EColumn(NextColumnIndex));

						if (CurrentGroupIndex != NextGroupIndex)
						{
							// Group change, add the group separator
							Result += NextGroupIndex < NumGroups - 1
								? Chars.GroupSeparator
								: Chars.LastGroupSeparator;
						}
						else
						{
							// Same group, add the cell separator
							Result += Chars.CellSeparator;
						}
					}
				}

				// Close the row
				Result += Chars.Right;
				Result += TEXT("\n");
			};

			auto AlignCenter = [&](TCHAR const* Str, uint32 Width)
			{
				int32 PaddingLeft = FMath::Max(0, int32(Width) - FCString::Strlen(Str));
				int32 PaddingRight = (PaddingLeft / 2) + (PaddingLeft & 1);
				PaddingLeft /= 2;

				Result += FString::Printf(TEXT("%*s%s%*s"), PaddingLeft, TEXT(""), Str, PaddingRight, TEXT(""));
			};

			// Top Border
			EmitGroupRow(Format.TopRow, [&](FGroup Group)
			{
				while (Group.Width--)
				{
					Result += Format.LineMajor;
				}
			});

			// Exclusive / Inclusive Group Row
			EmitGroupRow(Format.GroupNameRow, [&](FGroup Group)
			{
				TCHAR const* Str = Group.Index != GetColumnGroup(EColumn::Events)
					? GetGroupName(Group.Index)
					: TEXT("");

				AlignCenter(Str, Group.Width);
			});

			// Events Group Row
			EmitValueRow(true, Format.GroupBorderRow, [&](FColumn Column)
			{
				if (Column.Index == uint32(EColumn::Events))
				{
					AlignCenter(GetGroupName(GetColumnGroup(EColumn::Events)), Column.Width);
				}
				else
				{
					while (Column.Width--)
					{
						Result += Format.LineMinor;
					}
				}
			});

			// Header Row
			EmitValueRow(false, Format.ValueRow, [&](FColumn Column)
			{
				AlignCenter(GetColumnHeader(EColumn(Column.Index)), Column.Width);
			});

			// Header Border Row
			EmitValueRow(false, Format.DividorRow, [&](FColumn Column)
			{
				while (Column.Width--)
				{
					Result += Format.LineMinor;
				}
			});

			// Value rows
			for (uint32 RowIndex = 0; RowIndex < NumRows; ++RowIndex)
			{
				if (RowIndex > 0 && RowBreaks[RowIndex])
				{
					// Add a horizontal rule
					EmitValueRow(false, Format.DividorRow, [&](FColumn Column)
					{
						while (Column.Width--)
						{
							Result += Format.LineMinor;
						}
					});
				}

				EmitValueRow(false, Format.ValueRow, [&](FColumn Column)
				{
					int32 Width = Column.Width - (CellPadding * 2);
					if (EColumn(Column.Index) == EColumn::Events)
					{
						Width = -Width; // Align left
					}

					FString const& Cell = Columns[Column.Index][RowIndex];
					Result += FString::Printf(TEXT("%*s%*s%*s")
						, CellPadding, TEXT("")
						, Width, *Cell
						, CellPadding, TEXT(""));
				});
			}

			// Bottom Border
			EmitValueRow(false, Format.BottomRow, [&](FColumn Column)
			{
				while (Column.Width--)
				{
					Result += Format.LineMajor;
				}
			});

			return Result;
		}
	};
#endif

#if HAS_GPU_STATS
	// Per queue GPU stats

	// Total busy time on the current queue. StatName == "Unaccounted" is used by the Csv profiler
	static FGPUStat GPUStat_Total(TEXT("Unaccounted"), TEXT("Queue Total"), nullptr);
#endif

#if STATS
	TCHAR const* FGPUStat::GetTypeString(EType Type)
	{
		switch (Type)
		{
		default: checkNoEntry(); [[fallthrough]];
		case EType::Busy: return TEXT("Busy");
		case EType::Wait: return TEXT("Wait");
		case EType::Idle: return TEXT("Idle");
		}
	}

	FString FGPUStat::GetIDString(FQueue Queue, bool bFriendly)
	{
		if (bFriendly)
		{
			return FString::Printf(TEXT("GPU %d %s Queue %d")
				, Queue.GPU
				, Queue.GetTypeString()
				, Queue.Index
			);
		}
		else
		{
			return FString::Printf(TEXT("GPU%d_%s%d")
				, Queue.GPU
				, Queue.GetTypeString()
				, Queue.Index
			);
		}
	}

	FGPUStat::FStatInstance::FInner& FGPUStat::GetStatInstance(FQueue Queue, EType Type)
	{
		FStatInstance& Instance = Instances.FindOrAdd(Queue);

		switch (Type)
		{
		default: checkNoEntry(); [[fallthrough]];
		case EType::Busy: return Instance.Busy;
		case EType::Wait: return Instance.Wait;
		case EType::Idle: return Instance.Idle;
		}
	}

	TMap<FQueue, TUniquePtr<FGPUStat::FStatCategory>> FGPUStat::FStatCategory::Categories;

	FGPUStat::FStatCategory::FStatCategory(FQueue Queue)
		: GroupName(FString::Printf(TEXT("STATGROUP_%s"), *GetIDString(Queue, false)))
		, GroupDesc(FString::Printf(TEXT("%s Timing"), *GetIDString(Queue, true)))
	{}

	TStatId FGPUStat::GetStatId(FQueue Queue, EType Type)
	{
		FStatInstance::FInner& Instance = GetStatInstance(Queue, Type);

		if (!Instance.Stat)
		{
			TUniquePtr<FStatCategory>& Category = FStatCategory::Categories.FindOrAdd(Queue);
			if (!Category)
			{
				Category = MakeUnique<FStatCategory>(Queue);
			}

			// Encode the stat type in the FName number
			Instance.StatName = FName(*FString::Printf(TEXT("STAT_%s_%s"), *GetIDString(Queue, false), DisplayName), int32(Type));

			Instance.Stat = MakeUnique<FDynamicStat>(
				Instance.StatName,
				DisplayName,
				*Category->GroupName,
				FStatNameAndInfo::GpuStatCategory,
				*Category->GroupDesc,
				true, // IsDefaultEnabled
				true, // IsClearEveryFrame
				EStatDataType::ST_double,
				false, // IsCycleStat
				false, // SortByName
				FPlatformMemory::MCR_Invalid
			);
		}

		return Instance.Stat->GetStatId();
	}

#endif

	struct FGPUProfiler
	{
		class FTimestampStream
		{
		private:
			TArray<uint64> Values;

		public:
			struct FState
			{
				FTimestampStream const& Stream;
				int32 TimestampIndex = 0;
				uint64 BusyCycles = 0;

				FState(FTimestampStream const& Stream)
					: Stream(Stream)
				{}

				uint64 GetCurrentTimestamp (uint64 Anchor) const { return Stream.Values[TimestampIndex] - Anchor; }
				uint64 GetPreviousTimestamp(uint64 Anchor) const { return Stream.Values[TimestampIndex - 1] - Anchor; }

				bool HasMoreTimestamps() const { return TimestampIndex < Stream.Values.Num(); }
				bool IsStartingWork   () const { return (TimestampIndex & 0x01) == 0x00; }
				void AdvanceTimestamp () { TimestampIndex++; }
			};

			void AddTimestamp(uint64 Value, bool bBegin)
			{
				if (bBegin)
				{
					if (!Values.IsEmpty() && Value <= Values.Last())
					{
						//
						// The Begin TOP event is sooner than the last End BOP event.
						// The markers overlap, and the GPU was not idle.
						// 
						// Remove the previous End event, and discard this Begin event.
						//
						Values.RemoveAt(Values.Num() - 1, EAllowShrinking::No);
					}
					else
					{
						// GPU was idle. Keep this timestamp.
						Values.Add(Value);
					}
				}
				else
				{
					Values.Add(Value);
				}
			}

			static uint64 ComputeUnion(TArrayView<FTimestampStream::FState> Streams)
			{
				// The total number of cycles where at least one GPU pipe was busy.
				uint64 UnionBusyCycles = 0;

				uint64 LastMinCycles = 0;
				int32 BusyPipes = 0;
				bool bFirst = true;

				uint64 Anchor = 0; // @todo - handle possible timestamp wraparound

				// Process the time ranges from each pipe.
				while (true)
				{
					// Find the next minimum timestamp
					FTimestampStream::FState* NextMin = nullptr;
					for (auto& Current : Streams)
					{
						if (Current.HasMoreTimestamps() && (!NextMin || Current.GetCurrentTimestamp(Anchor) < NextMin->GetCurrentTimestamp(Anchor)))
						{
							NextMin = &Current;
						}
					}

					if (!NextMin)
						break; // No more timestamps to process

					if (!bFirst)
					{
						if (BusyPipes > 0 && NextMin->GetCurrentTimestamp(Anchor) > LastMinCycles)
						{
							// Accumulate the union busy time across all pipes
							UnionBusyCycles += NextMin->GetCurrentTimestamp(Anchor) - LastMinCycles;
						}

						if (!NextMin->IsStartingWork())
						{
							// Accumulate the busy time for this pipe specifically.
							NextMin->BusyCycles += NextMin->GetCurrentTimestamp(Anchor) - NextMin->GetPreviousTimestamp(Anchor);
						}
					}

					LastMinCycles = NextMin->GetCurrentTimestamp(Anchor);

					BusyPipes += NextMin->IsStartingWork() ? 1 : -1;
					check(BusyPipes >= 0);

					NextMin->AdvanceTimestamp();
					bFirst = false;
				}

				check(BusyPipes == 0);

				return UnionBusyCycles;
			}
		};

		struct FStatState
		{
			struct
			{
				uint64 BusyCycles = 0;
				uint64 IdleCycles = 0;
				uint64 WaitCycles = 0;

				void Accumulate(uint64 Busy, uint64 Wait, uint64 Idle)
				{
					BusyCycles += Busy;
					IdleCycles += Idle;
					WaitCycles += Wait;
				}
			} Exclusive, Inclusive;

			FStatState() = default;
			FStatState(FStatState const&) = default;

			FStatState(FStatState&& Other)
				: FStatState(Other)
			{
				Other.Exclusive = {};
				Other.Inclusive = {};
			}

		#if HAS_GPU_STATS
			void EmitResults(FQueue Queue, FGPUStat& GPUStat, TSet<FGPUStat*>& OutStatsToDelete
			#if STATS
				, FEndOfPipeStats* Stats
			#endif
			#if CSV_PROFILER_STATS
				, FCsvProfiler* CsvProfiler
			#endif
			) const
			{
			#if STATS
				if (GPUStat.bEmitToEngineStats)
				{
					Stats->AddMessage(GPUStat.GetStatId(Queue, FGPUStat::EType::Busy).GetName(), EStatOperation::Set, FPlatformTime::ToMilliseconds64(Inclusive.BusyCycles));
					Stats->AddMessage(GPUStat.GetStatId(Queue, FGPUStat::EType::Idle).GetName(), EStatOperation::Set, FPlatformTime::ToMilliseconds64(Inclusive.IdleCycles));
					Stats->AddMessage(GPUStat.GetStatId(Queue, FGPUStat::EType::Wait).GetName(), EStatOperation::Set, FPlatformTime::ToMilliseconds64(Inclusive.WaitCycles));
				}
			#endif

			#if CSV_PROFILER_STATS
				if (GPUStat.bEmitToEngineStats && CsvProfiler && Queue.Type == FQueue::EType::Graphics && Queue.Index == 0)
				{
					if (!GPUStat.CsvStat.IsSet())
					{
						static TArray<TUniquePtr<FCsvCategory>> CsvGPUCategories;
						if (!CsvGPUCategories.IsValidIndex(Queue.GPU))
						{
							CsvGPUCategories.SetNum(Queue.GPU + 1);
						}

						TUniquePtr<FCsvCategory>& Category = CsvGPUCategories[Queue.GPU];
						if (!Category)
						{
							Category = Queue.GPU > 0
								? MakeUnique<FCsvCategory>(*FString::Printf(TEXT("GPU%d"), Queue.GPU + 1), true)
								: MakeUnique<FCsvCategory>(TEXT("GPU"), true);
						}

						GPUStat.CsvStat.Emplace(GPUStat.StatName, Category->Index);
					}

					uint64 TotalCycles = Exclusive.BusyCycles + Exclusive.WaitCycles;
					CsvProfiler->RecordEndOfPipeCustomStat(GPUStat.CsvStat->Name, GPUStat.CsvStat->CategoryIndex, FPlatformTime::ToMilliseconds64(TotalCycles), ECsvCustomStatOp::Set);
				}
			#endif

				if (GPUStat.OnTimingResults(Queue, FPlatformTime::ToMilliseconds64(Inclusive.BusyCycles), FPlatformTime::ToMilliseconds64(Inclusive.IdleCycles), FPlatformTime::ToMilliseconds64(Inclusive.WaitCycles)) == FGPUStat::EOnTimingResultsAction::Delete)
				{
					OutStatsToDelete.Add(&GPUStat);
				}
			}
		#endif
		};

		struct FQueueDrawStats
		{
		#if HAS_GPU_STATS
			// The +1 is for "uncategorised"
			static constexpr int32 NumCategories = FRHIDrawStatsCategory::MAX_DRAWCALL_CATEGORY + 1;
		#else
			static constexpr int32 NumCategories = 1;
		#endif

			static constexpr int32 NoCategory = NumCategories - 1;
			FEvent::FStats Categories[NumCategories];

			void AddStats(FEvent::FStats const& Event, FRHIDrawStatsCategory const* Category)
			{
				uint32 CategoryIndex = Category ? Category->Index : NoCategory;
				Categories[CategoryIndex] += Event;
			}

			FQueueDrawStats() = default;
			FQueueDrawStats(FQueueDrawStats&& Other)
			{
				FMemory::Memcpy(Categories, Other.Categories, sizeof(Categories));
				FMemory::Memzero(Other.Categories, sizeof(Categories));
			}

			FQueueDrawStats& operator += (FQueueDrawStats const& RHS)
			{
				for (int32 Index = 0; Index < NumCategories; ++Index)
				{
					Categories[Index] += RHS.Categories[Index];
				}

				return *this;
			}
		};

		struct FQueueFrameState
		{
			FTimestampStream Timestamps;
			FStatState WholeQueueStat;

			uint64 CPUFrameBoundary = 0;

			// Used to override the GPU time calculation for this queue, if an FFrameTime event is in the stream
			TOptional<uint64> TotalBusyCycles;

			FQueueDrawStats DrawStats;

		#if WITH_RHI_BREADCRUMBS
			TMap<UE::RHI::GPUProfiler::FGPUStat*, FStatState> Stats;
		#endif
		};

		struct FResolvedWait
		{
			uint64 GPUTimestampTOP = 0;
			uint64 CPUTimestamp = 0;
		};

		struct FResolvedSignal
		{
			uint64 GPUTimestampBOP = 0;
			uint64 Value = 0;
		};

		struct FFrameState : TMap<FQueue, FQueueFrameState>
		{
		#if STATS
			TOptional<int64> StatsFrame;
		#endif
		};

		struct FQueueState
		{
			FQueue const Queue;
			FEventStream EventStream;

			// Array of fence signal history. Events are kept until all queues have processed events
			// later than the CPU timestamps of these signals. The old events are then trimmed.
			TArray<FResolvedSignal> Signals;

			// The value of the latest signaled fence on this queue.
			FResolvedSignal MaxSignal;

			// The GPU timestamp of the last event processed.
			uint64 LastGPUCycles = 0;

			FQueueFrameState QueueFrameState;

			bool bBusy = false;
			bool bWasTraced = false;

		#if WITH_RHI_BREADCRUMBS
			TMap<UE::RHI::GPUProfiler::FGPUStat*, int32> ActiveStats;
			TArray<UE::RHI::GPUProfiler::FGPUStat*> ActiveStatsStack;
		#endif

			TArray<FRHIDrawStatsCategory const*> DrawStatsStack { nullptr };

		#if WITH_PROFILEGPU
			struct
			{
				TArray<TUniquePtr<FNode>> Nodes;
				FNode* Current = nullptr;
				FNode* Prev = nullptr;
				FNode* First = nullptr;
				bool bProfileFrame = false;

				void PushNode(FString&& Name)
				{
					FNode* Parent = Current;
					Current = Nodes.Emplace_GetRef(MakeUnique<FNode>(MoveTemp(Name))).Get();
					Current->Parent = Parent;

					if (!First)
					{
						First = Current;
					}

					if (Parent)
					{
						Parent->Children.Add(Current);
					}

					if (Prev)
					{
						Prev->Next = Current;
					}
					Prev = Current;
				}

				void PopNode()
				{
					check(Current && Current->Parent);
					Current = Current->Parent;
				}

				void LogTree(FQueueState const& QueueState, uint32 FrameNumber) const
				{
					EGPUProfileSortMode SortMode = (EGPUProfileSortMode)FMath::Clamp(GCVarProfileGPU_Sort.GetValueOnAnyThread(), 0, ((int32)EGPUProfileSortMode::Max - 1));
					FWildcardString RootWildcard(GCVarProfileGPU_Root.GetValueOnAnyThread());
					const bool bShowEmptyNodes = GCVarProfileGPU_ShowLeafEvents.GetValueOnAnyThread();
					const double PercentThreshold = FMath::Clamp(GCVarProfileGPU_ThresholdPercent.GetValueOnAnyThread(), 0.0f, 100.0f);
					const bool bGraphicsPipeline = QueueState.Queue.Type == FQueue::EType::Graphics;

					if (SortMode != EGPUProfileSortMode::Chronological)
					{
						for (FNode* Node = First; Node; Node = Node->Next)
						{
							Node->Children.Sort([SortMode](FNode const& A, FNode const& B)
							{
								switch (SortMode)
								{
								default:
								case EGPUProfileSortMode::TimeElapsed: return B.Inclusive.BusyCycles    < A.Inclusive.BusyCycles;
								case EGPUProfileSortMode::NumPrims   : return B.Inclusive.NumPrimitives < A.Inclusive.NumPrimitives;
								case EGPUProfileSortMode::NumVerts   : return B.Inclusive.NumVertices   < A.Inclusive.NumVertices;
								}
							});
						}
					}

					bool bHasRows = false;
					const bool bTableFormatting = GCVarProfileGPU_TableFormatting.GetValueOnAnyThread() != 0;

					FTable Table;
					FString NodeOutput;
					double FrameInclusiveTime = 0.0;
					
					if (First)
					{
						FrameInclusiveTime = First->Inclusive.GetBusyMilliseconds();

						if (bGraphicsPipeline)
						{
							FrameInclusiveTime += First->Inclusive.GetWaitMilliseconds();
						}
					}

					auto Recurse = [&](auto& Recurse, FNode* Root, FNode* CurrentNode, bool bParentMatchedFilter, int32 Level) -> bool
					{
						// Percent that this node was of the total frame time
						const double WaitMilliseconds = bGraphicsPipeline ? CurrentNode->Inclusive.GetWaitMilliseconds() : 0.0;
						const double Percent = ((CurrentNode->Inclusive.GetBusyMilliseconds() + WaitMilliseconds) / FrameInclusiveTime) * 100.0;

						// Filter nodes according to cvar settings
						const bool bAboveThreshold = Percent >= PercentThreshold;
						const bool bNameMatches = bParentMatchedFilter || RootWildcard.IsMatch(CurrentNode->Name);
						const bool bHasWork = bShowEmptyNodes || CurrentNode->Inclusive.HasWork();

						const bool bDisplayEvent = bNameMatches && bHasWork && bAboveThreshold;

						if (bDisplayEvent)
						{
							if (Root == nullptr)
							{
								Root = CurrentNode;
							}

							if (bTableFormatting)
							{
								Table.AddRow(
									Root,
									CurrentNode->Inclusive,
									CurrentNode->Exclusive,
									CurrentNode->Name,
									Level);
							}
							else
							{
								NodeOutput += FString::Printf(TEXT("%s%4.1f%% %5.2fms%s   %s\n"),
									*FString().LeftPad(Level * 3),
									Percent,
									CurrentNode->Inclusive.GetBusyMilliseconds(),
									WaitMilliseconds > 0 && (CurrentNode->Children.Num() == 0 || CurrentNode == First) ? *FString::Printf(TEXT(" + %2.2f Wait"), WaitMilliseconds) : TEXT(""),
									*CurrentNode->Name);
							}
						}

						FNode::FStats OtherChildrenInclusive;
						FNode::FStats OtherChildrenExclusive;
						uint32 NumHiddenChildren = 0;

						for (FNode* Child : CurrentNode->Children)
						{
							bool bChildShown = Recurse(Recurse, Root, Child, bDisplayEvent, bDisplayEvent ? Level + 1 : Level);
							if (!bChildShown)
							{
								OtherChildrenInclusive += Child->Inclusive;
								OtherChildrenExclusive += Child->Exclusive;

								NumHiddenChildren++;
							}
						}

						if (bDisplayEvent && NumHiddenChildren > 0)
						{
							const double OtherChildrenInclusiveWaitMs = bGraphicsPipeline ? FPlatformTime::ToMilliseconds64(OtherChildrenInclusive.WaitCycles) : 0.0;
							const double OtherChildrenInclusiveMs = FPlatformTime::ToMilliseconds64(OtherChildrenInclusive.BusyCycles) + OtherChildrenInclusiveWaitMs;
							const double OtherChildrenPercent = (OtherChildrenInclusiveMs / FrameInclusiveTime) * 100.0;
							// Show the "other children" node if their total inclusive time is above the threshold
							if (OtherChildrenPercent >= PercentThreshold)
							{
								if (bTableFormatting)
								{
									Table.AddRow(
										Root,
										OtherChildrenInclusive,
										OtherChildrenExclusive,
										FString::Printf(TEXT("%d Other %s"), NumHiddenChildren, NumHiddenChildren >= 2 ? TEXT("Children") : TEXT("Child")),
										Level + 1
									);
								}
								else
								{
									NodeOutput += FString::Printf(TEXT("%s%4.1f%% %5.2fms%s   %d Other Children\n"),
										*FString().LeftPad(Level * 3),
										OtherChildrenPercent,
										OtherChildrenInclusiveMs,
										OtherChildrenInclusiveWaitMs > 0 ? *FString::Printf(TEXT(" + %2.2f Wait"), OtherChildrenInclusiveWaitMs) : TEXT(""),
										NumHiddenChildren);
								}
							}
						}

						return bDisplayEvent;
					};

					// Skip building the table if there was no useful work
					if (First && First->Inclusive.BusyCycles > 0)
					{
						Recurse(Recurse, nullptr, First, false, 0);
					}

					FString Header;
					if (GCVarProfileGPU_ShowHeader.GetValueOnAnyThread() != 0)
					{
						Header = FString::Printf(
							TEXT("    - %-30s: %.2fms\n")
							TEXT("    - %-30s: \"%s\"\n")
							TEXT("    - %-30s: %.2f%%\n")
							TEXT("    - %-30s: %s\n")
							TEXT("\n")
							, TEXT("Frame Time")
							, First ? (First->Inclusive.GetBusyMilliseconds() + (bGraphicsPipeline ? First->Inclusive.GetWaitMilliseconds() : 0.0)) : 0.0
							, *IConsoleManager::Get().FindConsoleObjectName(GCVarProfileGPU_Root.AsVariable())
							, *RootWildcard
							, *IConsoleManager::Get().FindConsoleObjectName(GCVarProfileGPU_ThresholdPercent.AsVariable())
							, PercentThreshold
							, *IConsoleManager::Get().FindConsoleObjectName(GCVarProfileGPU_ShowLeafEvents.AsVariable())
							, bShowEmptyNodes ? TEXT("true") : TEXT("false")
						);
					}

					FString PipelineString = FString::Printf(TEXT("%s pipeline %d - GPU %d")
						, QueueState.Queue.GetTypeString()
						, QueueState.Queue.Index
						, QueueState.Queue.GPU
					);

					bool bShowQueue = false;

					if (bTableFormatting)
					{
						NodeOutput = Table.HasRows() ? *Table.ToString() : TEXT("    No recorded work for this queue.\n");
						bShowQueue = Table.HasRows()  || GCVarProfileGPU_ShowEmptyQueues.GetValueOnAnyThread() != 0;
					}
					else
					{
						bShowQueue = NodeOutput.Len() > 0;
					}

					FString Final;

					if (bGraphicsPipeline)
					{
						Final = FString::Printf(
							TEXT("\n")
							TEXT("GPU Profile for Frame %d - %s\n")
							TEXT("\n")
							TEXT("%s")
							TEXT("%s")
							, FrameNumber
							, *PipelineString
							, *Header
							, *NodeOutput
						);
					}
					else
					{
						Final = FString::Printf(TEXT("%s\n\n%s"), *PipelineString, *NodeOutput);
					}

					if (bShowQueue)
					{
						TArray<FString> Lines;
						Final.ParseIntoArrayLines(Lines, false);

						for (FString const& Line : Lines)
						{
							UE_LOGF(LogRHI, Display, "%ls", *Line);
						}
					}

				#if !UE_BUILD_SHIPPING
					// Create and display profile visualizer data for the graphics queue
					if (GCVarProfileGPU_ShowUI.GetValueOnAnyThread() && bGraphicsPipeline && bShowQueue)
					{
						// Count the total number of exclusive cycles in the frame. Needed to draw the horizontal bars in the visualizer.
						uint64 TotalBusyCycles = 0;
						{
							auto CountCycles = [&TotalBusyCycles](auto& CountCycles, FNode* CurrentNode) -> void
							{
								TotalBusyCycles += CurrentNode->Exclusive.BusyCycles;
								for (FNode* Child : CurrentNode->Children)
								{
									CountCycles(CountCycles, Child);
								}
							};
							CountCycles(CountCycles, First);
						}

						// Recursive function to build the visualizer data structs from our nodes.
						uint64 StartCycles = 0;
						auto BuildData = [&](auto& BuildData, FNode* CurrentNode, TSharedPtr<FVisualizerEvent> const& Parent) -> TSharedPtr<FVisualizerEvent>
						{
							TSharedPtr<FVisualizerEvent> VisualizerEvent = MakeShared<FVisualizerEvent>(
								double(StartCycles) / TotalBusyCycles,
								double(CurrentNode->Inclusive.BusyCycles) / TotalBusyCycles,
								CurrentNode->Inclusive.GetBusyMilliseconds(),
								0,
								CurrentNode->Name
							);
							VisualizerEvent->ParentEvent = Parent;

							StartCycles += CurrentNode->Exclusive.BusyCycles;

							for (FNode* ChildNode : CurrentNode->Children)
							{
								VisualizerEvent->Children.Add(BuildData(BuildData, ChildNode, VisualizerEvent));
							}

							return VisualizerEvent;
						};

						// Launch the visualizer on the game thread
						FFunctionGraphTask::CreateAndDispatchWhenReady([
							Title = PipelineString,
							VisualizerData = BuildData(BuildData, First, nullptr)
						]
						{
							static FName ProfileVisualizerModule("ProfileVisualizer");
							if (FModuleManager::Get().IsModuleLoaded(ProfileVisualizerModule))
							{
								IProfileVisualizerModule& ProfileVisualizer = FModuleManager::GetModuleChecked<IProfileVisualizerModule>(ProfileVisualizerModule);
								// Display a warning if this is a GPU profile and the GPU was profiled with v-sync enabled (otherwise InVsyncEnabledWarningText is empty)
								ProfileVisualizer.DisplayProfileVisualizer(VisualizerData, *Title);
							}
						}, TStatId(), nullptr, ENamedThreads::GameThread);
					}
				#endif
				}
			} Profile;
		#endif

			FQueueState(FQueue const& Queue)
				: Queue(Queue)
				, EventStream(Queue)
			{}

			void ResolveSignal(FEvent::FSignalFence const& Event)
			{
				FResolvedSignal& Result = Signals.Emplace_GetRef();
		
				//
				// Take the max between the previous GPU EndWork event and the CPU timestamp. The signal cannot have happened on the GPU until the CPU has submitted the command to the driver.
				// 
				// An example would be a GPU queue that completes work and goes idle at time T. Later, the CPU issues a Signal without other prior work at time T + 100ms.
				// The fence signal cannot have happened until time T + 100ms because the CPU hadn't instructed the GPU to do so until then.
				// LastGPUCycles would still be set to time T, since that was the time of the preceeding EndWork event.
				//
				Result.GPUTimestampBOP = FMath::Max(LastGPUCycles, Event.CPUTimestamp);
				Result.Value = Event.Value;

				FGpuProfilerTrace::SignalFence(Queue.Value, Result.GPUTimestampBOP, Event.Value);

				//
				// Fences signals *MUST* be sequential, to remove ambiguity caused by trimming the Signals array.
				// 
				// To explain why, assume non-sequential signals are allowed, and consider the following example events on an arbitrary queue:
				// 
				//		     [Signal 2]
				//		-- Frame Boundary --
				//		     [Signal 4]
				//
				// Assume, after trimming events earlier than the frame boundary, that only [Signal 4] remains in the Signals array.
				// Then, some other queue attempts to [Wait 3]. We need to compute when [Wait 3] is resolved with only the information about [Signal 4].
				// 
				// Given that fences resolve waits as soon as the signalled value is >= the wait value, we could assume the fence was resolved at [Signal 4].
				// However, we don't know if the fence was already signalled to value 3 before the frame boundary and the trimming.
				// 
				// Without this information, it is ambiguous whether [Wait 3] is already resolved by a [Signal 3] before the frame boundary that is no longer
				// in the Signals array, or won't be resolved until [Signal 4]. We could have had this sequence of events:
				// 
				//		     [Signal 2]
				//		     [Signal 3]
				//		-- Frame Boundary --
				//		     [Signal 4]
				// 
				// Requiring that fences are always signalled in sequential order solves this.
				// If the awaited value is less than the first Signal, the fence has already been signalled before the frame boundary.
				//
				checkf(Result.Value == MaxSignal.Value + 1, TEXT("Fence signals must be sequential. Result.Value: %llu, MaxSignal.Value + 1: %llu"), Result.Value, (MaxSignal.Value + 1));

				// Signals should always advance in time
				checkf(Result.GPUTimestampBOP >= MaxSignal.GPUTimestampBOP, TEXT("Signals should always advance in time. Result.GPUTimestampBOP: %llu, MaxSignal.GPUTimestampBOP: %llu"), Result.GPUTimestampBOP, MaxSignal.GPUTimestampBOP);

				MaxSignal = Result;
			}

			void AccumulateTime(uint64 Busy, uint64 Wait, uint64 Idle)
			{
			#if WITH_RHI_BREADCRUMBS
				// Apply the timings to all active stats
				for (auto const& [Stat, RefCount] : ActiveStats)
				{
					FStatState& State = QueueFrameState.Stats.FindChecked(Stat);
					State.Inclusive.Accumulate(Busy, Wait, Idle);

					if (ActiveStatsStack.Num() > 0 && ActiveStatsStack.Last() == Stat)
					{
						State.Exclusive.Accumulate(Busy, Wait, Idle);
					}
				}

				if (ActiveStatsStack.Num() == 0)
			#endif
				{
					QueueFrameState.WholeQueueStat.Exclusive.Accumulate(Busy, Wait, Idle);
				}

				QueueFrameState.WholeQueueStat.Inclusive.Accumulate(Busy, Wait, Idle);

			#if WITH_PROFILEGPU
				for (FNode* Node = Profile.Current; Node; Node = Node->Parent)
				{
					Node->Inclusive.Accumulate(Busy, Wait, Idle);

					if (Node == Profile.Current)
					{
						Node->Exclusive.Accumulate(Busy, Wait, Idle);
					}
				}
			#endif
			}

			void BeginWork(FEvent::FBeginWork const& Event)
			{
				QueueFrameState.Timestamps.AddTimestamp(Event.GPUTimestampTOP, true);

				uint64 Idle = Event.CPUTimestamp > LastGPUCycles
					? Event.CPUTimestamp - LastGPUCycles
					: 0;

				AccumulateTime(0, 0, Idle);

				FGpuProfilerTrace::BeginWork(Queue.Value, Event.GPUTimestampTOP, Event.CPUTimestamp);

				LastGPUCycles = FMath::Max(LastGPUCycles, Event.GPUTimestampTOP);
			}

			void EndWork(FEvent::FEndWork const& Event)
			{
				QueueFrameState.Timestamps.AddTimestamp(Event.GPUTimestampBOP, false);

				uint64 Busy = Event.GPUTimestampBOP > LastGPUCycles
					? Event.GPUTimestampBOP - LastGPUCycles
					: 0;

				AccumulateTime(Busy, 0, 0);

				FGpuProfilerTrace::EndWork(Queue.Value, Event.GPUTimestampBOP);

				LastGPUCycles = FMath::Max(LastGPUCycles, Event.GPUTimestampBOP);
			}

		#if WITH_RHI_BREADCRUMBS
			void BeginBreadcrumb(FEvent::FBeginBreadcrumb const& Event)
			{
				uint64 Busy = Event.GPUTimestampTOP > LastGPUCycles
					? Event.GPUTimestampTOP - LastGPUCycles
					: 0;

				AccumulateTime(Busy, 0, 0);
				LastGPUCycles = FMath::Max(LastGPUCycles, Event.GPUTimestampTOP);

			#if HAS_GPU_STATS
				UE::RHI::GPUProfiler::FGPUStat* GPUStat = Event.Breadcrumb->GPUStat;
				if (GPUStat != nullptr)
				{
					// Disregard the stat if it is nested within itself (i.e. its already in the ActiveStats map with a non-zero ref count).
					// Only the outermost stat will count the busy time, otherwise we'd be double-counting the nested time.
					int32 RefCount = ActiveStats.FindOrAdd(GPUStat)++;
					if (RefCount == 0)
					{
						QueueFrameState.Stats.FindOrAdd(GPUStat);
					}

					ActiveStatsStack.Add(GPUStat);

					if (GPUStat->DrawStatsCategory)
					{
						DrawStatsStack.Add(GPUStat->DrawStatsCategory);
					}
				}
			#endif

				Event.Breadcrumb->TraceBeginGPU(Queue.Value, Event.GPUTimestampTOP);

			#if WITH_PROFILEGPU
				if (Profile.bProfileFrame)
				{
					FRHIBreadcrumb::FBuffer Buffer;
					const TCHAR* Name = Event.Breadcrumb->GetTCHAR(Buffer);

					// Push a new node
					Profile.PushNode(Name);
				}
			#endif
			}

			void EndBreadcrumb(FEvent::FEndBreadcrumb const& Event)
			{
				uint64 Busy = Event.GPUTimestampBOP > LastGPUCycles
					? Event.GPUTimestampBOP - LastGPUCycles
					: 0;

				AccumulateTime(Busy, 0, 0);
				LastGPUCycles = FMath::Max(LastGPUCycles, Event.GPUTimestampBOP);

			#if HAS_GPU_STATS
				UE::RHI::GPUProfiler::FGPUStat* GPUStat = Event.Breadcrumb->GPUStat;
				if (GPUStat != nullptr)
				{
					// Pop the stat when the refcount hits zero.
					int32 RefCount = --ActiveStats.FindChecked(GPUStat);
					if (RefCount == 0)
					{
						ActiveStats.FindAndRemoveChecked(GPUStat);
					}

					check(ActiveStatsStack.Last() == GPUStat);
					ActiveStatsStack.RemoveAt(ActiveStatsStack.Num() - 1, EAllowShrinking::No);

					if (GPUStat->DrawStatsCategory)
					{
						check(DrawStatsStack.Last() == GPUStat->DrawStatsCategory);
						DrawStatsStack.RemoveAt(DrawStatsStack.Num() - 1, EAllowShrinking::No);
					}
				}
			#endif

				Event.Breadcrumb->TraceEndGPU(Queue.Value, Event.GPUTimestampBOP);

			#if WITH_PROFILEGPU
				if (Profile.bProfileFrame)
				{
					Profile.PopNode();
				}
			#endif
			}
		#endif

			void Stats(FEvent::FStats const& Event)
			{
			#if WITH_PROFILEGPU
				if (Profile.Current)
				{
					Profile.Current->Exclusive += Event;

					for (FNode* Node = Profile.Current; Node; Node = Node->Parent)
					{
						Node->Inclusive += Event;
					}
				}
			#endif
				
				QueueFrameState.DrawStats.AddStats(Event, DrawStatsStack.Last());

				FGpuProfilerTrace::Stats(Queue.Value, Event.NumDraws, Event.NumPrimitives);
			}

			void Wait(FResolvedWait const& ResolvedWait, const FEvent::FWaitFence& WaitFence)
			{
				// Time the queue was idle between the last EndWork event, and the Wait command being submitted to the GPU driver.
				uint64 Idle = ResolvedWait.CPUTimestamp > LastGPUCycles
					? ResolvedWait.CPUTimestamp - LastGPUCycles
					: 0;

				uint64 WaitStart = FMath::Max(ResolvedWait.CPUTimestamp, LastGPUCycles);

				FGpuProfilerTrace::WaitFence(Queue.Value, ResolvedWait.GPUTimestampTOP, WaitFence.Queue.Value, WaitFence.Value);

				// Time the queue spent waiting for the fence to signal on another queue.
				uint64 Wait = 0;
				if (ResolvedWait.GPUTimestampTOP > WaitStart)
				{
					Wait = ResolvedWait.GPUTimestampTOP - WaitStart;
					FGpuProfilerTrace::TraceWait(Queue.Value, WaitStart, ResolvedWait.GPUTimestampTOP);
				}

				// Bring the last GPU busy end time forwards to where the wait is resolved.
				LastGPUCycles = ResolvedWait.GPUTimestampTOP;

				AccumulateTime(0, Wait, Idle);
			}

			void TrimSignals(uint64 CPUTimestamp)
			{
				// Remove all signals that occured on the GPU timeline before this frame boundary on the CPU.
				int32 Index = Algo::LowerBoundBy(Signals, CPUTimestamp, [](FResolvedSignal const& Signal) { return Signal.GPUTimestampBOP; });
				if (Index >= 0)
				{
					Signals.RemoveAt(0, Index, EAllowShrinking::No);
				}
			}

			void FrameTime(uint64 TotalGPUTime)
			{
				QueueFrameState.TotalBusyCycles = TotalGPUTime;
			}

			void FrameBoundary(FEvent::FFrameBoundary const& Event, FFrameState& FrameState)
			{
				check(!bBusy);
				QueueFrameState.CPUFrameBoundary = Event.CPUTimestamp;

				FGpuProfilerTrace::FrameBoundary(Queue.Value, Event.FrameNumber);

			#if WITH_PROFILEGPU
				if (Profile.bProfileFrame)
				{
					Profile.LogTree(*this, Event.FrameNumber);
					Profile = {};
				}
			#endif

				FrameState.Emplace(Queue, MoveTemp(QueueFrameState));

			#if WITH_RHI_BREADCRUMBS
				// Reinsert timestamp streams for the current active stats on 
				// this queue, since these got moved into the frame state.
				for (auto& [GPUStat, RefCount] : ActiveStats)
				{
					QueueFrameState.Stats.FindOrAdd(GPUStat);
				}
			#endif

			#if WITH_PROFILEGPU
				if (Event.bProfileNextFrame)
				{
					Profile.bProfileFrame = true;

					// Build the node tree 
					Profile.PushNode(TEXT("<root>"));

				#if WITH_RHI_BREADCRUMBS
					auto Recurse = [&](auto& Recurse, FRHIBreadcrumbNode* Current) -> void
					{
						if (!Current)
						{
							return;
						}

						Recurse(Recurse, Current->GetParent());

						FRHIBreadcrumb::FBuffer Buffer;
						Profile.PushNode(Current->GetTCHAR(Buffer));
					};
					Recurse(Recurse, Event.Breadcrumb);
				#endif // WITH_RHI_BREADCRUMBS
				}
			#endif
			}
		};

		std::atomic<int32> TriggerProfileRefCount { 0 };
		std::atomic<int32> IsProfilingRefCount    { 0 };

		TMap<uint32, FFrameState> Frames;
		TMap<FQueue, TUniquePtr<FQueueState>> QueueStates;

		// Attempts to retrieve the CPU and GPU timestamps of when a fence wait is resolved by a signal on another queue.
		TOptional<FResolvedWait> ResolveWait(FQueueState& LocalQueue, FEvent::FWaitFence const& WaitFenceEvent)
		{
			FQueueState const& RemoteQueue = static_cast<FQueueState const&>(*QueueStates.FindChecked(WaitFenceEvent.Queue));

			if (RemoteQueue.MaxSignal.Value < WaitFenceEvent.Value)
			{
				// Fence has not yet been signalled on the remote queue
				return {};
			}
			else
			{
				// Fence has been signalled, but it may be in the future.

				FResolvedWait Result;
				Result.CPUTimestamp = WaitFenceEvent.CPUTimestamp;

				//
				// The wait cannot be resolved any earlier than:
				//
				//		1) The wait command was issued to the driver (WaitFenceEvent.CPUTimestamp)
				//		2) The GPU completed prior work on this queue (LocalQueue.LastGPUCycles)
				//
				Result.GPUTimestampTOP = FMath::Max(WaitFenceEvent.CPUTimestamp, LocalQueue.LastGPUCycles);
				//
				//      3) The wait maybe be further delayed by the remote queue the GPU is awaiting.
				//
				int32 Index = Algo::LowerBoundBy(RemoteQueue.Signals, WaitFenceEvent.Value, [](FResolvedSignal const& Signal) { return Signal.Value; });
				if (RemoteQueue.Signals.IsValidIndex(Index))
				{
					FResolvedSignal const& Signal = RemoteQueue.Signals[Index];

					//
					// Only consider this signal's timestamp if the fence was not already signalled at the previous frame boundary.
					// See comment in ResolveSignal() for details.
					//
					if (!(Index == 0 && WaitFenceEvent.Value < Signal.Value))
					{
						Result.GPUTimestampTOP = FMath::Max(Result.GPUTimestampTOP, Signal.GPUTimestampBOP);
					}
				}

				return Result;
			}
		}

		void InitializeQueues(TConstArrayView<FQueue> Queues)
		{
			FGpuProfilerTrace::Initialize();

			for (FQueue Queue : Queues)
			{
				TUniquePtr<FQueueState>& Ptr = QueueStates.FindOrAdd(Queue);
				if (!Ptr.IsValid())
				{
					Ptr = MakeUnique<FQueueState>(Queue);
				}
			}
		}

		bool ProcessQueue(FQueueState& QueueState)
		{
			bool bProgress = false;

			if (FGpuProfilerTrace::IsAvailable() && !QueueState.bWasTraced)
			{
				FGpuProfilerTrace::InitializeQueue(QueueState.Queue.Value, QueueState.Queue.GetTypeString());
				QueueState.bWasTraced = true;
			}

			while (FEvent const* Event = QueueState.EventStream.Peek())
			{
				switch (Event->GetType())
				{
				case FEvent::EType::BeginWork:
					{
						check(!QueueState.bBusy);
						QueueState.bBusy = true;
						QueueState.BeginWork(Event->Value.Get<FEvent::FBeginWork>());
					}
					break;

				case FEvent::EType::EndWork:
					{
						check(QueueState.bBusy);
						QueueState.bBusy = false;
						QueueState.EndWork(Event->Value.Get<FEvent::FEndWork>());
					}
					break;

			#if WITH_RHI_BREADCRUMBS
				case FEvent::EType::BeginBreadcrumb:
					{
						check(QueueState.bBusy);
						QueueState.BeginBreadcrumb(Event->Value.Get<FEvent::FBeginBreadcrumb>());
					}
					break;

				case FEvent::EType::EndBreadcrumb:
					{
						check(QueueState.bBusy);
						QueueState.EndBreadcrumb(Event->Value.Get<FEvent::FEndBreadcrumb>());
					}
					break;
			#endif // WITH_RHI_BREADCRUMBS

				case FEvent::EType::Stats:
					{
						check(QueueState.bBusy);
						QueueState.Stats(Event->Value.Get<FEvent::FStats>());
					}
					break;

				case FEvent::EType::SignalFence:
					{
						check(!QueueState.bBusy);
						QueueState.ResolveSignal(Event->Value.Get<FEvent::FSignalFence>());
					}
					break;

				case FEvent::EType::WaitFence:
					{
						check(!QueueState.bBusy);
						const FEvent::FWaitFence& WaitFence= Event->Value.Get<FEvent::FWaitFence>();
						TOptional<FResolvedWait> ResolvedWait = ResolveWait(QueueState, Event->Value.Get<FEvent::FWaitFence>());

						if (!ResolvedWait.IsSet())
						{
							// Unresolved fence, pause processing
							return bProgress;
						}

						QueueState.Wait(*ResolvedWait, WaitFence);
					}
					break;

				case FEvent::EType::FrameTime:
					{
						const FEvent::FFrameTime& FrameTime = Event->Value.Get<FEvent::FFrameTime>();
						QueueState.FrameTime(FrameTime.TotalGPUTime);
					}
					break;

				case FEvent::EType::FrameBoundary:
					{
						FEvent::FFrameBoundary const& FrameBoundary = Event->Value.Get<FEvent::FFrameBoundary>();
						FFrameState& FrameState = Frames.FindOrAdd(FrameBoundary.FrameNumber);

					#if STATS
						FrameState.StatsFrame = FrameBoundary.bStatsFrameSet
							? FrameBoundary.StatsFrame
							: TOptional<int64>();
					#endif

					#if WITH_PROFILEGPU
						if (QueueState.Profile.bProfileFrame)
						{
							IsProfilingRefCount--;
						}
					#endif

						QueueState.FrameBoundary(FrameBoundary, FrameState);

						if (FrameState.Num() == QueueStates.Num())
						{
							// Trim the Signals array in each queue, up to the lowest frame boundary CPU timestamp.
							{
								uint64 MinFrameBoundary = TNumericLimits<uint64>::Max();
								for (auto& [Queue, QueueFrameState] : FrameState)
								{
									MinFrameBoundary = FMath::Min(MinFrameBoundary, QueueFrameState.CPUFrameBoundary);
								}

								for (auto& [Queue, LocalQueueState] : QueueStates)
								{
									LocalQueueState.Get()->TrimSignals(MinFrameBoundary);
								}
							}

							// All registered queues have reported their frame boundary event.
							// We have a full set of data to compute the total frame GPU stats.
							ProcessFrame(FrameState);

							Frames.Remove(FrameBoundary.FrameNumber);
						}
					}
					break;
				}

				QueueState.EventStream.Pop();
				bProgress = true;
			}

			return bProgress;
		}

		void ProcessFrame(FFrameState& FrameState)
		{
		#if STATS
			FEndOfPipeStats* Stats = FEndOfPipeStats::Get();
			if (FrameState.StatsFrame.IsSet())
			{
				Stats->AddMessage(FStatConstants::AdvanceFrame.GetEncodedName(), EStatOperation::AdvanceFrameEventEndOfPipe, *FrameState.StatsFrame);
			}
		#endif

		#if CSV_PROFILER_STATS
			const bool bCsvStatsEnabled = !!CVarGPUCsvStatsEnabled.GetValueOnAnyThread();
			FCsvProfiler* CsvProfiler = FCsvProfiler::Get();
			CsvProfiler->BeginFrameEOP();
		#endif

		#if HAS_GPU_STATS
			FRHIDrawStatsCategory::FManager::FDrawCounts DrawCounts;
		#endif

			// Summed stats across all GPUs
			FEvent::FStats Total = {};
			TStaticArray<FEvent::FStats, FQueueDrawStats::NumCategories> TotalPerCategory { InPlace };
			TStaticArray<FEvent::FStats, MAX_NUM_GPUS> TotalPerGPU { InPlace };

			TOptional<uint64> MaxQueueBusyCycles;

#if WITH_RHI_BREADCRUMBS && HAS_GPU_STATS
			// Stats that requested deletion are collected here and deleted after all per-queue callbacks have fired. Using a TSet ensures each stat is deleted exactly once even if it spans multiple
			// queues (e.g. mGPU) and OnTimingResults is called more than once for it.
			TSet<FGPUStat*> StatsToDelete;
#endif

			for (auto const& [Queue, QueueFrameState] : FrameState)
			{
			#if WITH_RHI_BREADCRUMBS && HAS_GPU_STATS
				// Compute the individual GPU stats
				for (auto const& [GPUStat, StatState] : QueueFrameState.Stats)
				{
					StatState.EmitResults(Queue, *GPUStat, StatsToDelete
					#if STATS
						, Stats
					#endif
					#if CSV_PROFILER_STATS
						, bCsvStatsEnabled ? CsvProfiler : nullptr
					#endif
					);
				}
			#endif // WITH_RHI_BREADCRUMBS && HAS_GPU_STATS

				// Set the whole-frame per queue stat
			#if HAS_GPU_STATS
				QueueFrameState.WholeQueueStat.EmitResults(Queue, GPUStat_Total, StatsToDelete
				#if STATS
					, Stats
				#endif
				#if CSV_PROFILER_STATS
					, bCsvStatsEnabled ? CsvProfiler : nullptr
				#endif
				);
			#endif

				if (QueueFrameState.TotalBusyCycles.IsSet())
				{
					uint64 CurrentMax = MaxQueueBusyCycles ? *MaxQueueBusyCycles : 0;
					MaxQueueBusyCycles = FMath::Max(CurrentMax, *QueueFrameState.TotalBusyCycles);
				}

				for (int32 CategoryIndex = 0; CategoryIndex < FQueueDrawStats::NumCategories; ++CategoryIndex)
				{
					FEvent::FStats const& Category = QueueFrameState.DrawStats.Categories[CategoryIndex];
					TotalPerCategory[CategoryIndex] += Category;
					TotalPerGPU[Queue.GPU]          += Category;
					Total                           += Category;

				#if HAS_GPU_STATS
					if (CategoryIndex < DrawCounts.Num())
					{
						DrawCounts[CategoryIndex][Queue.GPU] += Category.NumDraws;
					}
				#endif
				}
			}

#if WITH_RHI_BREADCRUMBS && HAS_GPU_STATS
			for (FGPUStat* Stat : StatsToDelete)
			{
				delete Stat;
			}
			StatsToDelete.Reset();
#endif

			for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; ++GPUIndex)
			{
				GNumDrawCallsRHI      [GPUIndex] = TotalPerGPU[GPUIndex].NumDraws;
				GNumPrimitivesDrawnRHI[GPUIndex] = TotalPerGPU[GPUIndex].NumPrimitives;
			}

		#if CSV_PROFILER_STATS
			// Multi-GPU support : CSV stats do not support MGPU yet. We're summing the totals across all GPUs here.
			CsvProfiler->RecordEndOfPipeCustomStat("DrawCalls"      , CSV_CATEGORY_INDEX(RHI), Total.NumDraws     , ECsvCustomStatOp::Set);
			CsvProfiler->RecordEndOfPipeCustomStat("PrimitivesDrawn", CSV_CATEGORY_INDEX(RHI), Total.NumPrimitives, ECsvCustomStatOp::Set);
		#endif

	#if HAS_GPU_STATS

			FRHIDrawStatsCategory::FManager& Manager = FRHIDrawStatsCategory::GetManager();
			Manager.AccumulateFrameStats(DrawCounts);

		#if STATS
			Stats->AddMessage(GET_STATFNAME(STAT_RHIDraws     ), EStatOperation::Set, int64(Total.NumDraws     ));
			Stats->AddMessage(GET_STATFNAME(STAT_RHIDispatches), EStatOperation::Set, int64(Total.NumDispatches));
			Stats->AddMessage(GET_STATFNAME(STAT_RHIPrimitives), EStatOperation::Set, int64(Total.NumPrimitives));
			Stats->AddMessage(GET_STATFNAME(STAT_RHIVertices  ), EStatOperation::Set, int64(Total.NumVertices  ));
		#endif

		#if CSV_PROFILER_STATS
			for (int32 CategoryIndex = 0; CategoryIndex < Manager.NumCategory; ++CategoryIndex)
			{
				FCsvProfiler::RecordCustomStat(Manager.Array[CategoryIndex]->Name, CSV_CATEGORY_INDEX(DrawCall), int32(TotalPerCategory[CategoryIndex].NumDraws), ECsvCustomStatOp::Set);
			}
		#endif

	#endif // HAS_GPU_STATS

			if (MaxQueueBusyCycles.IsSet())
			{
				// Set the total GPU time stat according to the value directly provided by the platform RHI
				GRHIGPUFrameTimeHistory.PushFrameCycles(1.0 / FPlatformTime::GetSecondsPerCycle64(), *MaxQueueBusyCycles);
			}
			else
			{
				// Compute the whole-frame total GPU time.
				TArray<FTimestampStream::FState, TInlineAllocator<GetRHIPipelineCount() * MAX_NUM_GPUS>> StreamPointers;
				for (auto const& [Queue, QueueFrameState] : FrameState)
				{
					StreamPointers.Emplace(QueueFrameState.Timestamps);
				}
				uint64 WholeFrameUnion = FTimestampStream::ComputeUnion(StreamPointers);

				// Update the global GPU frame time stats
				GRHIGPUFrameTimeHistory.PushFrameCycles(1.0 / FPlatformTime::GetSecondsPerCycle64(), WholeFrameUnion);
			}

		#if STATS
			Stats->Flush();
		#endif
		}

		void ProcessAllQueues()
		{
			// Process the queues as far as possible
			bool bProgress;
			do
			{
				bProgress = false;
				for (auto& [Queue, QueueState] : QueueStates)
				{
					bProgress |= ProcessQueue(*QueueState.Get());
				}
			} while (bProgress);
		}

		void ProcessEvents(TArrayView<FEventStream> EventStreams)
		{
			for (FEventStream& Stream : EventStreams)
			{
				if (!Stream.IsEmpty())
				{
					FQueueState& QueueState = *QueueStates.FindChecked(Stream.Queue);
					QueueState.EventStream.Append(MoveTemp(Stream));
				}
			}

			ProcessAllQueues();
		}

	} GGPUProfiler;

	void ProcessEvents(TArrayView<FEventStream> EventStreams)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::RHI::GPUProfiler::ProcessEvents);
		GGPUProfiler.ProcessEvents(EventStreams);
	}

	void InitializeQueues(TConstArrayView<FQueue> Queues)
	{
		GGPUProfiler.InitializeQueues(Queues);
	}

#if WITH_PROFILEGPU
	static FAutoConsoleCommand GCommand_ProfileGPU(
		TEXT("ProfileGPU"),
		TEXT("Captures statistics about a frame of GPU work and prints the results to the log."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			GGPUProfiler.IsProfilingRefCount += GGPUProfiler.QueueStates.Num();
			GGPUProfiler.TriggerProfileRefCount++;
		}));
#endif // WITH_PROFILEGPU

	RHI_API bool IsProfiling()
	{
	#if WITH_PROFILEGPU
		return GGPUProfiler.IsProfilingRefCount > 0;
	#else
		return false;
	#endif
	}

	RHI_API bool ShouldProfileNextFrame()
	{
	#if WITH_PROFILEGPU
		if (GGPUProfiler.TriggerProfileRefCount > 0)
		{
			GGPUProfiler.TriggerProfileRefCount--;
			return true;
		}
	#endif

		return false;
	}
}

RHI_API FRHIGPUFrameTimeHistory GRHIGPUFrameTimeHistory;

FRHIGPUFrameTimeHistory::EResult FRHIGPUFrameTimeHistory::FState::PopFrameCycles(uint64& OutCycles64)
{
	return GRHIGPUFrameTimeHistory.PopFrameCycles(*this, OutCycles64);
}

FRHIGPUFrameTimeHistory::EResult FRHIGPUFrameTimeHistory::PopFrameCycles(FState& State, uint64& OutCycles64)
{
	FScopeLock Lock(&CS);

	if (State.NextIndex == NextIndex)
	{
		OutCycles64 = 0;
		return EResult::Empty;
	}
	else
	{
		uint64 MinHistoryIndex = NextIndex >= MaxLength ? NextIndex - MaxLength : 0;

		if (State.NextIndex < MinHistoryIndex)
		{
			State.NextIndex = MinHistoryIndex;
			OutCycles64 = History[State.NextIndex++ % MaxLength];
			return EResult::Disjoint;
		}
		else
		{
			OutCycles64 = History[State.NextIndex++ % MaxLength];
			return EResult::Ok;
		}
	}
}

uint32 GGPUFrameTime = 0;

void FRHIGPUFrameTimeHistory::PushFrameCycles(double GPUFrequency, uint64 GPUCycles)
{
	double Seconds = double(GPUCycles) / GPUFrequency;
	double Cycles32 = Seconds / FPlatformTime::GetSecondsPerCycle();
	double Cycles64 = Seconds / FPlatformTime::GetSecondsPerCycle64();

	{
		FScopeLock Lock(&CS);
		History[NextIndex++ % MaxLength] = uint64(Cycles64);
	}

	FPlatformAtomics::InterlockedExchange(reinterpret_cast<volatile int32*>(&GGPUFrameTime), int32(Cycles32));
}

RHI_API uint32 RHIGetGPUFrameCycles(uint32 GPUIndex)
{
	return (uint32)FPlatformAtomics::AtomicRead(reinterpret_cast<volatile int32*>(&GGPUFrameTime));
}

#undef LOCTEXT_NAMESPACE
