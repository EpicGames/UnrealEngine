// Copyright Epic Games, Inc. All Rights Reserved.

#include "GpuProfilerTrace.h"

#include "CborWriter.h"
#include "GPUProfiler.h"
#include "HAL/IConsoleManager.h"
#include "Serialization/MemoryWriter.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Trace/Trace.inl"
#include "RHI.h"
#include "Trace/Detail/Field.h"
#include "Trace/Detail/Important/ImportantLogScope.h"
#include "Trace/Detail/LogScope.h"

#if UE_TRACE_GPU_PROFILER_ENABLED
	UE_TRACE_CHANNEL_EXTERN(GpuChannel, RHI_API)
	UE_TRACE_CHANNEL_DEFINE(GpuChannel, "GPU profiler events including timing, breadcrumbs, and queue synchronization. Data can be viewed in the Timing Profiler.")
#endif

namespace UE::RHI::GPUProfiler
{
// Tracing for the new GPU Profiler
#if UE_TRACE_GPU_PROFILER_ENABLED

UE_TRACE_EVENT_BEGIN(GpuProfiler, Init, NoSync | Important)
	UE_TRACE_EVENT_FIELD(uint8, Version)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(GpuProfiler, QueueSpec, NoSync | Important)
	UE_TRACE_EVENT_FIELD(uint32, QueueId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, TypeString)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(GpuProfiler, EventFrameBoundary)
	UE_TRACE_EVENT_FIELD(uint32, QueueId)
	UE_TRACE_EVENT_FIELD(uint32, FrameNumber)
UE_TRACE_EVENT_END()

#if WITH_RHI_BREADCRUMBS

UE_TRACE_EVENT_BEGIN(GpuProfiler, EventBreadcrumbSpec, NoSync | Important)
	UE_TRACE_EVENT_FIELD(uint32, SpecId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, StaticName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, NameFormat)
	UE_TRACE_EVENT_FIELD(uint8[], FieldNames)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(GpuProfiler, EventBeginBreadcrumb)
	UE_TRACE_EVENT_FIELD(uint32, SpecId)
	UE_TRACE_EVENT_FIELD(uint32, QueueId)
	UE_TRACE_EVENT_FIELD(uint64, GPUTimestampTOP)
	UE_TRACE_EVENT_FIELD(uint8[], Metadata)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(GpuProfiler, EventEndBreadcrumb)
	UE_TRACE_EVENT_FIELD(uint32, QueueId)
	UE_TRACE_EVENT_FIELD(uint64, GPUTimestampBOP)
UE_TRACE_EVENT_END()
#endif // WITH_RHI_BREADCRUMBS

UE_TRACE_EVENT_BEGIN(GpuProfiler, EventBeginWork)
	UE_TRACE_EVENT_FIELD(uint32, QueueId)
	UE_TRACE_EVENT_FIELD(uint64, GPUTimestampTOP)
	UE_TRACE_EVENT_FIELD(uint64, CPUTimestamp)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(GpuProfiler, EventEndWork)
	UE_TRACE_EVENT_FIELD(uint32, QueueId)
	UE_TRACE_EVENT_FIELD(uint64, GPUTimestampBOP)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(GpuProfiler, EventWait)
	UE_TRACE_EVENT_FIELD(uint32, QueueId)
	UE_TRACE_EVENT_FIELD(uint64, StartTime)
	UE_TRACE_EVENT_FIELD(uint64, EndTime)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(GpuProfiler, EventStats)
	UE_TRACE_EVENT_FIELD(uint32, QueueId)
	UE_TRACE_EVENT_FIELD(uint32, NumDraws)
	UE_TRACE_EVENT_FIELD(uint32, NumPrimitives)
	// @todo add num dispatches / num vertices stats
	//UE_TRACE_EVENT_FIELD(uint32, NumDispatches)
	//UE_TRACE_EVENT_FIELD(uint32, NumVertices)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(GpuProfiler, SignalFence)
	UE_TRACE_EVENT_FIELD(uint32, QueueId)
	UE_TRACE_EVENT_FIELD(uint64, CPUTimestamp)
	UE_TRACE_EVENT_FIELD(uint64, Value)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(GpuProfiler, WaitFence)
	UE_TRACE_EVENT_FIELD(uint32, QueueId)
	UE_TRACE_EVENT_FIELD(uint64, CPUTimestamp)
	UE_TRACE_EVENT_FIELD(uint32, QueueToWaitForId)
	UE_TRACE_EVENT_FIELD(uint64, Value)
UE_TRACE_EVENT_END()

bool FGpuProfilerTrace::IsAvailable()
{
	return UE_TRACE_CHANNELEXPR_IS_ENABLED(GpuChannel);
}

void FGpuProfilerTrace::Initialize()
{
	static bool bInitialized = false;
	ensure(!bInitialized);

	constexpr uint8 Version = 2;

	UE_TRACE_LOG(GpuProfiler, Init, GpuChannel)
		<< Init.Version(Version);

	bInitialized = true;
}

void FGpuProfilerTrace::InitializeQueue(uint32 QueueId, const TCHAR* Name)
{
	FStringView TypeString(Name);
	UE_TRACE_LOG(GpuProfiler, QueueSpec, GpuChannel, TypeString.Len() * sizeof(TCHAR))
		<< QueueSpec.QueueId(QueueId)
		<< QueueSpec.TypeString(TypeString.GetData(), TypeString.Len());
}

void FGpuProfilerTrace::FrameBoundary(uint32 QueueId, uint32 FrameId)
{
	UE_TRACE_LOG(GpuProfiler, EventFrameBoundary, GpuChannel)
		<< EventFrameBoundary.QueueId(QueueId)
		<< EventFrameBoundary.FrameNumber(FrameId);
}

void FGpuProfilerTrace::BeginWork(uint32 QueueId, uint64 GPUTimestampTOP, uint64 CPUTimestamp)
{
	UE_TRACE_LOG(GpuProfiler, EventBeginWork, GpuChannel)
		<< EventBeginWork.QueueId(QueueId)
		<< EventBeginWork.GPUTimestampTOP(GPUTimestampTOP)
		<< EventBeginWork.CPUTimestamp(CPUTimestamp);
}

void FGpuProfilerTrace::EndWork(uint32 QueueId, uint64 GPUTimestampBOP)
{
	UE_TRACE_LOG(GpuProfiler, EventEndWork, GpuChannel)
		<< EventEndWork.QueueId(QueueId)
		<< EventEndWork.GPUTimestampBOP(GPUTimestampBOP);
}

void FGpuProfilerTrace::TraceWait(uint32 QueueId, uint64 StartTime, uint64 EndTime)
{
	UE_TRACE_LOG(GpuProfiler, EventWait, GpuChannel)
		<< EventWait.QueueId(QueueId)
		<< EventWait.StartTime(StartTime)
		<< EventWait.EndTime(EndTime);
}

void FGpuProfilerTrace::Stats(uint32 QueueId, uint32 NumDraws, uint32 NumPrimitives)
{
	UE_TRACE_LOG(GpuProfiler, EventStats, GpuChannel)
		<< EventStats.QueueId(QueueId)
		<< EventStats.NumDraws(NumDraws)
		<< EventStats.NumPrimitives(NumPrimitives);
}

void FGpuProfilerTrace::SignalFence(uint32 QueueId, uint64 ResolvedTimestamp, uint64 Value)
{
	UE_TRACE_LOG(GpuProfiler, SignalFence, GpuChannel)
		<< SignalFence.QueueId(QueueId)
		<< SignalFence.CPUTimestamp(ResolvedTimestamp)
		<< SignalFence.Value(Value);
}

void FGpuProfilerTrace::WaitFence(uint32 QueueId, uint64 ResolvedTimestamp, uint32 QueueToWaitForId, uint64 Value)
{
	UE_TRACE_LOG(GpuProfiler, WaitFence, GpuChannel)
		<< WaitFence.QueueId(QueueId)
		<< WaitFence.CPUTimestamp(ResolvedTimestamp)
		<< WaitFence.QueueToWaitForId(QueueToWaitForId)
		<< WaitFence.Value(Value);
}

std::atomic<uint32> FGpuProfilerTrace::NextSpecId = 1;

uint32 FGpuProfilerTrace::InternalBreadcrumbSpec(const TCHAR* StaticName, const TCHAR* NameFormat, const TArray<uint8>& FieldNames)
{
	if (!FGpuProfilerTrace::IsAvailable())
	{
		return 0;
	}

	uint32 SpecId = NextSpecId.fetch_add(1);

	uint32 DataSize = FCString::Strlen(StaticName) * sizeof(TCHAR);
	DataSize += FCString::Strlen(NameFormat) * sizeof(TCHAR);
	DataSize += FieldNames.Num() * sizeof(uint8);

#if WITH_RHI_BREADCRUMBS
	UE_TRACE_LOG(GpuProfiler, EventBreadcrumbSpec, GpuChannel, DataSize)
		<< EventBreadcrumbSpec.SpecId(SpecId)
		<< EventBreadcrumbSpec.StaticName(StaticName)
		<< EventBreadcrumbSpec.NameFormat(NameFormat)
		<< EventBreadcrumbSpec.FieldNames(FieldNames.GetData(), FieldNames.Num());
#endif
	return SpecId;
}

void FGpuProfilerTrace::BeginBreadcrumb(uint32 SpecId, uint32 QueueId, uint64 GPUTimestampTOP, const TArray<uint8>& CborData)
{
#if WITH_RHI_BREADCRUMBS
	UE_TRACE_LOG(GpuProfiler, EventBeginBreadcrumb, GpuChannel)
		<< EventBeginBreadcrumb.SpecId(SpecId)
		<< EventBeginBreadcrumb.QueueId(QueueId)
		<< EventBeginBreadcrumb.GPUTimestampTOP(GPUTimestampTOP)
		<< EventBeginBreadcrumb.Metadata(CborData.GetData(), CborData.Num());
#endif
}

void FGpuProfilerTrace::EndBreadcrumb(uint32 QueueId, uint64 GPUTimestampTOP)
{
#if WITH_RHI_BREADCRUMBS
	UE_TRACE_LOG(GpuProfiler, EventEndBreadcrumb, GpuChannel)
		<< EventEndBreadcrumb.QueueId(QueueId)
		<< EventEndBreadcrumb.GPUTimestampBOP(GPUTimestampTOP);
#endif
}

#endif // UE_TRACE_GPU_PROFILER_ENABLED

FMetadataSerializer::FMetadataSerializer()
{
	CborData.Reserve(128);
	MemoryWriter = new FMemoryWriter(CborData, false, true);
	CborWriter = new FCborWriter(MemoryWriter, ECborEndianness::StandardCompliant);
}

FMetadataSerializer::~FMetadataSerializer()
{
	delete CborWriter;
	delete MemoryWriter;
}

void FMetadataSerializer::AppendValue(const ANSICHAR* Value)
{
	CborWriter->WriteValue(Value, FCStringAnsi::Strlen(Value));
}

void FMetadataSerializer::AppendValue(const WIDECHAR* Value)
{
	CborWriter->WriteValue(FWideStringView(Value));
}

void FMetadataSerializer::AppendValue(const UTF8CHAR* Value)
{
	CborWriter->WriteValue((FUtf8StringView)Value);
}

void FMetadataSerializer::AppendValue(uint64 Value)
{
	CborWriter->WriteValue(Value);
}

void FMetadataSerializer::AppendValue(int64 Value)
{
	CborWriter->WriteValue(Value);
}

void FMetadataSerializer::AppendValue(double Value)
{
	CborWriter->WriteValue(Value);
}

void FMetadataSerializer::AppendValue(bool Value)
{
	CborWriter->WriteValue(Value);
}

void FMetadataSerializer::AppendValue(const FName& Value)
{
	CborWriter->WriteValue(Value.ToString());
}

void FMetadataSerializer::AppendValue(const FDebugName& Value)
{
	CborWriter->WriteValue(Value.ToString());
}

void FMetadataSerializer::AppendValue(const FString& Value)
{
	CborWriter->WriteValue(Value);
}

} // namespace UE::RHI::GPUProfiler

