// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/App.h"
#include "DynamicRHI.h"
#include "Misc/MessageDialog.h"
#include "RHI.h"
#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/PlatformApplicationMisc.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIStrings.h"


static TOptional<ERHIFeatureLevel::Type> GetForcedFeatureLevel()
{
	TOptional<ERHIFeatureLevel::Type> ForcedFeatureLevel{};

	if (FParse::Param(FCommandLine::Get(), TEXT("es31")) || FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES31")) || FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES3_1")))
	{
		ForcedFeatureLevel = ERHIFeatureLevel::ES3_1;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("sm5")))
	{
		ForcedFeatureLevel = ERHIFeatureLevel::SM5;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("sm6")))
	{
		ForcedFeatureLevel = ERHIFeatureLevel::SM6;
	}

#if WITH_EDITOR
	if (TOptional<ERHIBindlessConfiguration> ForcedConfig = RHIGetForcedBindlessConfiguration(); ForcedConfig && ForcedConfig != ERHIBindlessConfiguration::Disabled)
	{
		ForcedFeatureLevel = ERHIFeatureLevel::SM6;
	}
#endif

	return ForcedFeatureLevel;
}

FDynamicRHI* PlatformCreateDynamicRHI()
{
	ERHIFeatureLevel::Type RequestedFeatureLevel = ERHIFeatureLevel::SM5;
	FDynamicRHI* DynamicRHI = nullptr;

	const bool bForceVulkan = FParse::Param(FCommandLine::Get(), TEXT("vulkan"));
	bool bForceOpenGL = false;
	if (!bForceVulkan)
	{
		// OpenGL can only be used for mobile preview.
		bForceOpenGL = FParse::Param(FCommandLine::Get(), TEXT("opengl"));
		ERHIFeatureLevel::Type PreviewFeatureLevel;
		const bool bUsePreviewFeatureLevel = RHIGetPreviewFeatureLevel(PreviewFeatureLevel);
		if (bForceOpenGL && !bUsePreviewFeatureLevel)
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "OpenGLRemoved", "Warning: OpenGL is no longer supported for desktop platforms. Vulkan will be used instead."));
			bForceOpenGL = false;
		}
	}

	bool bVulkanFailed = false;
	bool bVulkanSMFailed = false;
	bool bOpenGLFailed = false;

	IDynamicRHIModule* DynamicRHIModule = nullptr;

	// Read in command line params to force a given feature level
	const TOptional<ERHIFeatureLevel::Type> ForcedFeatureLevel = GetForcedFeatureLevel();
	if (ForcedFeatureLevel.IsSet())
	{
		RequestedFeatureLevel = ForcedFeatureLevel.GetValue();
		DynamicRHIModule = &FModuleManager::LoadModuleChecked<IDynamicRHIModule>(TEXT("VulkanRHI"));
		if (DynamicRHIModule->IsSupported(RequestedFeatureLevel))
		{
			const FString RHIName = FString::Printf(TEXT("Vulkan (%s)"), *LexToString(RequestedFeatureLevel));
			FApp::SetGraphicsRHI(RHIName);
			FPlatformApplicationMisc::UsingVulkan();
		}
		else
		{
			DynamicRHIModule = nullptr;
		}
	}
	else
	{
		TArray<FString> TargetedShaderFormats;
		GConfig->GetArray(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("TargetedRHIs"), TargetedShaderFormats, GEngineIni);

		if (!bForceOpenGL)
		{
			TArray<FString> VulkanTargetedShaderFormats = TargetedShaderFormats.FilterByPredicate([](const FString& ShaderFormatName) { return ShaderFormatName.StartsWith(TEXT("SF_VULKAN_")); });

			if (VulkanTargetedShaderFormats.Num())
			{
				DynamicRHIModule = &FModuleManager::LoadModuleChecked<IDynamicRHIModule>(TEXT("VulkanRHI"));
				if (DynamicRHIModule->IsSupported())
				{
					// Sort and start at the end to prefer higher feature levels (SM6 over SM5, for example).
					VulkanTargetedShaderFormats.Sort();
					for (int32 ShaderFormatIndex = VulkanTargetedShaderFormats.Num() - 1; ShaderFormatIndex >= 0; --ShaderFormatIndex)
					{
						const FName ShaderFormatName(*VulkanTargetedShaderFormats[ShaderFormatIndex]);
						const EShaderPlatform TargetedPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormatName);
						const ERHIFeatureLevel::Type MaxFeatureLevel = GetMaxSupportedFeatureLevel(TargetedPlatform);
						if (DynamicRHIModule->IsSupported(MaxFeatureLevel))
						{
							bVulkanFailed = false;
							RequestedFeatureLevel = MaxFeatureLevel;

							FString FeatureLevelName;
							GetFeatureLevelName(RequestedFeatureLevel, FeatureLevelName);
							FApp::SetGraphicsRHI(FString::Printf(TEXT("Vulkan (%s)"), *FeatureLevelName));
							FPlatformApplicationMisc::UsingVulkan();

							break;
						}
						else
						{
							UE_LOGF(LogRHI, Display, "Skipping %ls...", *ShaderFormatName.ToString());

							bVulkanFailed = true;
							bVulkanSMFailed = true;
						}
					}

					if (bVulkanFailed)
					{
						DynamicRHIModule = nullptr;
					}
				}
				else
				{
					DynamicRHIModule = nullptr;
					bVulkanFailed = true;
				}
			}
		}

		if (!bForceVulkan && (DynamicRHIModule == nullptr))
		{
			TArray<FString> OGLTargetedShaderFormats = TargetedShaderFormats.FilterByPredicate([](const FString& ShaderFormatName) { return ShaderFormatName.StartsWith(TEXT("GLSL_")); });
			if (OGLTargetedShaderFormats.Num())
			{
				DynamicRHIModule = &FModuleManager::LoadModuleChecked<IDynamicRHIModule>(TEXT("OpenGLDrv"));
				if (DynamicRHIModule->IsSupported())
				{
					FApp::SetGraphicsRHI(TEXT("OpenGL"));
					FPlatformApplicationMisc::UsingOpenGL();

					FName ShaderFormatName(*TargetedShaderFormats[0]);
					EShaderPlatform TargetedPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormatName);
					RequestedFeatureLevel = GetMaxSupportedFeatureLevel(TargetedPlatform);
				}
				else
				{
					DynamicRHIModule = nullptr;
					bOpenGLFailed = true;
				}
			}
		}
	}

	// Create the dynamic RHI.
	if (DynamicRHIModule)
	{
		DynamicRHI = DynamicRHIModule->CreateRHI(RequestedFeatureLevel);
	}
	else
	{
		if (ForcedFeatureLevel.IsSet())
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "UnsupportedVulkanTargetedRHI", "Trying to force specific Vulkan feature level but it is not supported."));
			FPlatformMisc::RequestExitWithStatus(true, 1);
			// unreachable
			return nullptr;	
		}
		else if (bForceVulkan)
		{
			if (bVulkanFailed)
			{
				if (bVulkanSMFailed)
				{
					FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "ForcedVulkanSM", "Forced Vulkan device could not be created at the project's supported feature levels (see log for details)."));
				}
				else
				{
					FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "RequiredVulkan", "Vulkan Driver is required to run the engine."));
				}
			}
			else
			{
				FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "NoVulkanTargetedRHI", "Trying to force Vulkan RHI but the project does not have it in TargetedRHIs list."));
			}

			FPlatformMisc::RequestExitWithStatus(true, 1);
			// unreachable
			return nullptr;
		}
		else if (bForceOpenGL)
		{
			if (bOpenGLFailed)
			{
				FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "RequiredOpenGL", "OpenGL 4.3 is required to run the engine."));
			}
			else
			{
				FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "NoOpenGLTargetedRHI", "Trying to force OpenGL RHI but the project does not have it in TargetedRHIs list."));
			}

			FPlatformMisc::RequestExitWithStatus(true, 1);
			// unreachable
			return nullptr;
		}
		else
		{
			if (bVulkanFailed && bOpenGLFailed)
			{
				FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "NoVulkanNoGL", "Vulkan or OpenGL (4.3) support is required to run the engine."));
			}
			else
			{
				if (bVulkanFailed)
				{
					if (bVulkanSMFailed)
					{
						FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "SupportedVulkanSM", "Vulkan device could not be created at the project's supported feature levels (see log for details)."));
					}
					else
					{
						FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "NoVulkanDriver", "Failed to load Vulkan Driver which is required to run the engine.\nThe engine no longer fallbacks to OpenGL4 which has been deprecated."));
					}
				}
				else if (bOpenGLFailed)
				{
					FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "NoOpenGLDriver", "Failed to load OpenGL Driver which is required to run the engine.\nOpenGL4 has been deprecated and should use Vulkan."));
				}
				else
				{
					FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("LinuxDynamicRHI", "NoTargetedRHI", "The project does not target Vulkan or OpenGL RHIs, check project settings or pass -nullrhi."));
				}
			}

			FPlatformMisc::RequestExitWithStatus(true, 1);
			// unreachable
			return nullptr;
		}
	}

	return DynamicRHI;
}
