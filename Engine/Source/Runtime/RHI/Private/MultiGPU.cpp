// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MultiGPU.cpp: Multi-GPU support
=============================================================================*/

#include "MultiGPU.h"

#if WITH_MGPU
uint32 GNumExplicitGPUsForRendering = 1;
uint32 GVirtualMGPU = 0;
#endif
