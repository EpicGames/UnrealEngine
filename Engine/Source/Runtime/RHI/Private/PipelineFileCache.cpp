// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
PipelineFileCache.cpp: Pipeline state cache implementation.
=============================================================================*/

#include "PipelineFileCache.h"

#if PLATFORM_APPLE
	#include "Apple/ScopeAutoreleasePool.h"
#if PLATFORM_MAC
	#include "Mac/MacSystemIncludes.h"
	#elif PLATFORM_IOS
	#include "IOS/IOSSystemIncludes.h"
	#endif 
#endif // PLATFORM_APPLE

#include "Containers/List.h"
#include "Containers/Ticker.h"
#include "PipelineStateCache.h"
#include "HAL/IConsoleManager.h"
#include "Misc/EngineVersion.h"
#include "HAL/PlatformFile.h"
#include "Serialization/MemoryReader.h"
#include "Misc/CommandLine.h"
#include "Serialization/MemoryWriter.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/Paths.h"
#include "Async/AsyncFileHandle.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RHIStrings.h"
#include "String/LexFromString.h"
#include "String/ParseTokens.h"
#include "Misc/ScopeExit.h"
#include <Algo/ForEach.h>

static FString JOURNAL_FILE_EXTENSION(".jnl");

// Loaded + New created
#if STATS // If STATS are not enabled RHI_API will DLLEXPORT on an empty line
RHI_API DEFINE_STAT(STAT_TotalGraphicsPipelineStateCount);
RHI_API DEFINE_STAT(STAT_TotalComputePipelineStateCount);
RHI_API DEFINE_STAT(STAT_TotalRayTracingPipelineStateCount);
#endif

// CSV category for PSO encounter and save events
CSV_DEFINE_CATEGORY(PSO, true);

// New Saved count
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Serialized Graphics Pipeline State Count"), STAT_SerializedGraphicsPipelineStateCount, STATGROUP_PipelineStateCache );
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Serialized Compute Pipeline State Count"), STAT_SerializedComputePipelineStateCount, STATGROUP_PipelineStateCache );
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Serialized RayTracing Pipeline State Count"), STAT_SerializedRayTracingPipelineStateCount, STATGROUP_PipelineStateCache);

// New created - Cache Miss count
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("New Graphics Pipeline State Count"), STAT_NewGraphicsPipelineStateCount, STATGROUP_PipelineStateCache );
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("New Compute Pipeline State Count"), STAT_NewComputePipelineStateCount, STATGROUP_PipelineStateCache );
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("New RayTracing Pipeline State Count"), STAT_NewRayTracingPipelineStateCount, STATGROUP_PipelineStateCache);

// Memory - Only track the file representation and new state cache stats
DECLARE_MEMORY_STAT(TEXT("New Cached PSO"), STAT_NewCachedPSOMemory, STATGROUP_PipelineStateCache);
DECLARE_MEMORY_STAT(TEXT("PSO Stat"), STAT_PSOStatMemory, STATGROUP_PipelineStateCache);
DECLARE_MEMORY_STAT(TEXT("File Cache"), STAT_FileCacheMemory, STATGROUP_PipelineStateCache);

void LexFromString(ETextureCreateFlags& OutValue, const FStringView& InString)
{
	__underlying_type(ETextureCreateFlags) TmpFlags = static_cast<__underlying_type(ETextureCreateFlags)>(OutValue);
	LexFromString(TmpFlags, InString);
	OutValue = static_cast<ETextureCreateFlags>(TmpFlags);
}

enum class EPipelineCacheFileFormatVersions : uint32
{
    FirstWorking = 7,
    LibraryID = 9,
    ShaderMetaData = 10,
	SortedVertexDesc = 11,
	TOCMagicGuard = 12,
	PSOUsageMask = 13,
	PSOBindCount = 14,
	EOFMarker = 15,
	EngineFlags = 16,
	Subpass = 17,
	PatchSizeReduction_NoDuplicatedGuid = 18,
	AlphaToCoverage = 19,
	AddingMeshShaders = 20,
	RemovingTessellationShaders = 21,
	LastUsedTime = 22,
	MoreRenderTargetFlags = 23,
	FragmentDensityAttachment = 24,
	AddingDepthClipMode = 25,
	BeforeStableCacheVersioning = 26,
	RemovingLineAA = 27,
	AddingDepthBounds = 28,
	AddRTPSOShaderBindingLayout = 29,
	VariableRateShading = 30,
	PSOPrecacheResult = 31,
};

const uint64 FPipelineCacheFileFormatMagic = 0x5049504543414348; // PIPECACH
const uint64 FPipelineCacheTOCFileFormatMagic = 0x544F435354415232; // TOCSTAR2
const uint64 FPipelineCacheEOFFileFormatMagic = 0x454F462D4D41524B; // EOF-MARK
const RHI_API uint32 FPipelineCacheFileFormatCurrentVersion = (uint32)EPipelineCacheFileFormatVersions::PSOPrecacheResult;
const int32  FPipelineCacheGraphicsDescPartsNum = 67; // parser will expect this number of parts in a description string

/**
  * PipelineFileCache API access
  **/

static TAutoConsoleVariable<int32> CVarPSOFileCacheEnabled(
														   TEXT("r.ShaderPipelineCache.Enabled"),
														   PIPELINE_CACHE_DEFAULT_ENABLED,
														   TEXT("1 Enables the PipelineFileCache, 0 disables it."),
														   ECVF_Default | ECVF_RenderThreadSafe
														   );

static TAutoConsoleVariable<int32> CVarPSOFileCacheLogPSO(
														   TEXT("r.ShaderPipelineCache.LogPSO"),
														   PIPELINE_CACHE_DEFAULT_ENABLED,
														   TEXT("1 Logs new PSO entries into the file cache and allows saving."),
														   ECVF_Default | ECVF_RenderThreadSafe
														   );

static TAutoConsoleVariable<int32> CVarPSOFileCacheReportPSO(
														   TEXT("r.ShaderPipelineCache.ReportPSO"),
														   PIPELINE_CACHE_DEFAULT_ENABLED,
														   TEXT("1 reports new PSO entries via a delegate, but does not record or modify any cache file. New PSOs are reported in bulk once per frame."),
														   ECVF_Default | ECVF_RenderThreadSafe
														   );

static int32 GPSOExcludePrecachePSOsInFileCache = 0;
static FAutoConsoleVariableRef CVarPSOFileCacheExcludePrecachePSO(
														   TEXT("r.ShaderPipelineCache.ExcludePrecachePSO"),
														   GPSOExcludePrecachePSOsInFileCache,
														   TEXT("1 excludes saving runtime-precached graphics & compute PSOs in the file cache which were successfully precached, 0 includes all of them (default). Excluding precached PSOs requires PSO precaching to be enabled."),
														   ECVF_ReadOnly
														   );

static int32 GPSOIncludeAllPrecachedPSOsInFileCache = 0;
static FAutoConsoleVariableRef CVarPSOFileCacheIncludeAllPrecachedPSOs(
														   TEXT("r.ShaderPipelineCache.IncludeAllPrecachedPSOs"),
														   GPSOIncludeAllPrecachedPSOsInFileCache,
														   TEXT("1 includes all successfully precached PSOs in the pipeline file cache even if they were never needed for rendering.\n")
														   TEXT("Logged entries carry EPSOPrecacheResult::Active; the result is upgraded to the actual runtime outcome if the PSO is later used at draw time.\n")
														   TEXT("Requires PSO precaching to be enabled. Default: 0 (disabled)."),
														   ECVF_ReadOnly
														   );

static int32 GPSOFileCachePrintNewPSODescriptors = 0;
static FAutoConsoleVariableRef CVarPSOFileCachePrintNewPSODescriptors(
														   TEXT("r.ShaderPipelineCache.PrintNewPSODescriptors"),
														   GPSOFileCachePrintNewPSODescriptors,
														   TEXT("1 prints descriptions for all new PSO entries to the log/console while 0 does not. 2 prints additional details, such as the PSO type (graphics, compute, or raytracing), usage mask and bind count. Defaults to 0."),
														   ECVF_Default
														   );

static TAutoConsoleVariable<int32> CVarPSOFileCacheSaveUserCache(
                                                            TEXT("r.ShaderPipelineCache.SaveUserCache"),
															PIPELINE_CACHE_DEFAULT_ENABLED && UE_BUILD_SHIPPING,
                                                            TEXT("If > 0 then any missed PSOs will be saved to a writable user cache file for subsequent runs to load and avoid in-game hitches. Enabled by default on macOS only."),
                                                            ECVF_Default | ECVF_RenderThreadSafe
                                                            );

static TAutoConsoleVariable<int32> CVarPSOFileCacheUserCacheUnusedElementRetainDays(
                                                            TEXT("r.ShaderPipelineCache.UserCacheUnusedElementRetainDays"),
															30,
                                                            TEXT("The amount of time in days to keep unused PSO entries in the cache."),
                                                            ECVF_Default
                                                            );

static TAutoConsoleVariable<int32> CVarPSOFileCacheUserCacheUnusedElementCheckPeriod(
                                                            TEXT("r.ShaderPipelineCache.UserCacheUnusedElementCheckPeriod"),
															-1,
                                                            TEXT("The amount of time in days between running the garbage collection on unused PSOs in the user cache. Use a negative value to disable."),
                                                            ECVF_Default
                                                            );

static TAutoConsoleVariable<int32> CVarLazyLoadShadersWhenPSOCacheIsPresent(
	TEXT("r.ShaderPipelineCache.LazyLoadShadersWhenPSOCacheIsPresent"),
	0,
	TEXT("Non-Zero: If we load a PSO cache, then lazy load from the shader code library. This assumes the PSO cache is more or less complete. This will only work on RHIs that support the library+Hash CreateShader API (GRHISupportsLazyShaderCodeLoading == true)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarClearOSPSOFileCache(
														   TEXT("r.ShaderPipelineCache.ClearOSCache"),
														   0,
														   TEXT("1 Enables the OS level clear after install, 0 disables it."),
														   ECVF_Default | ECVF_RenderThreadSafe
														   );

static TAutoConsoleVariable<int32> CVarAlwaysGeneratePOSSOFileCache(
														   TEXT("r.ShaderPipelineCache.AlwaysGenerateOSCache"),
														   1,
														   TEXT("1 generates the cache every run, 0 generates it only when it is missing."),
														   ECVF_Default | ECVF_RenderThreadSafe
														   );


FRWLock FPipelineFileCacheManager::FileCacheLock;
TMap<FString, TUniquePtr< class FPipelineCacheFile>> FPipelineFileCacheManager::FileCacheMap;
TMap<FGuid, FString> FPipelineFileCacheManager::GameGuidToCacheKey;
TMap<uint32, FPSOUsageData> FPipelineFileCacheManager::RunTimeToPSOUsage;
TMap<uint32, FPSOUsageData> FPipelineFileCacheManager::NewPSOUsage;
TMap<uint32, FPipelineStateStats*> FPipelineFileCacheManager::Stats;
TMap<uint32, FPipelineCacheFileFormatPSO> FPipelineFileCacheManager::NewPSOs;
TMpscQueue<FPipelineCacheFileFormatPSO> FPipelineFileCacheManager::NewPSOsToReport;
uint32 FPipelineFileCacheManager::NumNewPSOs;
FString FPipelineFileCacheManager::UserCacheKey;
FPipelineFileCacheManager::PSOOrder FPipelineFileCacheManager::RequestedOrder = FPipelineFileCacheManager::PSOOrder::MostToLeastUsed;
bool FPipelineFileCacheManager::FileCacheEnabled = false;
FPipelineFileCacheManager::FPipelineStateLoggedEvent FPipelineFileCacheManager::PSOLoggedEvent;
uint64 FPipelineFileCacheManager::GameUsageMask = 0;
bool FPipelineFileCacheManager::GameUsageMaskSet = false;
bool FPipelineFileCacheManager::LogNewPSOsToConsoleAndCSV = true;

static int64 GetCurrentUnixTime()
{
	return FDateTime::UtcNow().ToUnixTimestamp();
}

bool DefaultPSOMaskComparisonFunction(uint64 ReferenceMask, uint64 PSOMask)
{
	return (ReferenceMask & PSOMask) == ReferenceMask;
}
FPSOMaskComparisonFn FPipelineFileCacheManager::MaskComparisonFn = DefaultPSOMaskComparisonFunction;

static inline bool IsReferenceMaskSet(uint64 ReferenceMask, uint64 PSOMask)
{
	return (ReferenceMask & PSOMask) == ReferenceMask;
}

#if PLATFORM_WINDOWS

FRHIShader::~FRHIShader()
{
	if (InUseByPSOCompilation > 0)
	{
		UE_LOGF(LogRHI, Fatal, "FRHIShader with hash: %ls and Frequency: %d still in use by PSO compilation when being destroyed", *Hash.ToString(), Frequency);
	}
}

void FRHIShader::SetInUseByPSOCompilation(bool bInUse)
{
	if (bInUse)
	{
		FPlatformAtomics::InterlockedIncrement(&InUseByPSOCompilation);
	}
	else
	{
		check(InUseByPSOCompilation > 0);
		FPlatformAtomics::InterlockedDecrement(&InUseByPSOCompilation);
	}
}

#endif // PLATFORM_WINDOWS

void FRHIComputeShader::UpdateStats()
{
	FPipelineStateStats::UpdateStats(Stats);
}

void FPipelineStateStats::UpdateStats(FPipelineStateStats* Stats)
{
	if (Stats)
	{
		FPlatformAtomics::InterlockedExchange(&Stats->LastFrameUsed, GFrameCounter);
		FPlatformAtomics::InterlockedIncrement(&Stats->TotalBindCount);
		FPlatformAtomics::InterlockedCompareExchange(&Stats->FirstFrameUsed, GFrameCounter, -1);
	}
}

struct FPipelineCacheFileFormatHeader
{
	uint64 Magic;			// Sanity check
	uint32 Version; 		// File version must match engine version, otherwise we ignore
	uint32 GameVersion; 	// Same as above but game specific code can invalidate
	TEnumAsByte<EShaderPlatform> Platform; // The shader platform for all referenced PSOs.
	FGuid Guid;				// Guid to identify the file uniquely
	uint64 TableOffset;		// absolute file offset to TOC
	int64 LastGCUnixTime;   // Last time that the cache was scanned to remove out of date elements.
	
	friend FArchive& operator<<(FArchive& Ar, FPipelineCacheFileFormatHeader& Info)
	{
		Ar << Info.Magic;
		Ar << Info.Version;
		Ar << Info.GameVersion;
		Ar << Info.Platform;
		Ar << Info.Guid;
		Ar << Info.TableOffset;

		if (Info.Version >= (uint32)EPipelineCacheFileFormatVersions::LastUsedTime)
		{
			Ar << Info.LastGCUnixTime;
		}

		return Ar;
	}
};

FArchive& operator<<( FArchive& Ar, FPipelineStateStats& Info )
{
	Ar << Info.FirstFrameUsed;
	Ar << Info.LastFrameUsed;
	Ar << Info.CreateCount;
	Ar << Info.TotalBindCount;
	Ar << Info.PSOHash;
	
	return Ar;
}

/**
  * PipelineFileCache MetaData Engine Flags
  **/
const uint16 FPipelineCacheFlagInvalidPSO = 1 << 0;

struct FPipelineCacheFileFormatPSOMetaData
{
	FPipelineCacheFileFormatPSOMetaData()
	: FileOffset(0)
	, UsageMask(0)
	, LastUsedUnixTime(0)
	, EngineFlags(0)
	{
	}
	
	~FPipelineCacheFileFormatPSOMetaData()
	{
	}

	uint64 FileOffset;
	uint64 FileSize;
	FGuid FileGuid;
	FPipelineStateStats Stats;
	TSet<FShaderHash> Shaders;
	uint64 UsageMask;
	int64  LastUsedUnixTime;
	uint16 EngineFlags;

	void AddShaders(const FPipelineCacheFileFormatPSO& NewEntry)
	{
		switch (NewEntry.Type)
		{
			case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
			{
				INC_DWORD_STAT(STAT_SerializedComputePipelineStateCount);
				Shaders.Add(NewEntry.ComputeDesc.ComputeShader);
				break;
			}
			case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
			{
				INC_DWORD_STAT(STAT_SerializedGraphicsPipelineStateCount);

				if (NewEntry.GraphicsDesc.VertexShader != FShaderHash())
					Shaders.Add(NewEntry.GraphicsDesc.VertexShader);

				if (NewEntry.GraphicsDesc.FragmentShader != FShaderHash())
					Shaders.Add(NewEntry.GraphicsDesc.FragmentShader);

				if (NewEntry.GraphicsDesc.GeometryShader != FShaderHash())
					Shaders.Add(NewEntry.GraphicsDesc.GeometryShader);

				if (NewEntry.GraphicsDesc.MeshShader != FShaderHash())
					Shaders.Add(NewEntry.GraphicsDesc.MeshShader);

				if (NewEntry.GraphicsDesc.AmplificationShader != FShaderHash())
					Shaders.Add(NewEntry.GraphicsDesc.AmplificationShader);

				break;
			}
			case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
			{
				INC_DWORD_STAT(STAT_SerializedRayTracingPipelineStateCount);
				Shaders.Add(NewEntry.RayTracingDesc.ShaderHash);
				break;
			}
			default:
			{
				check(false);
				break;
			}
		}
	}
	
	friend FArchive& operator<<(FArchive& Ar, FPipelineCacheFileFormatPSOMetaData& Info)
	{
		Ar << Info.FileOffset;
		Ar << Info.FileSize;
		// if FileGuid is zeroed out (a frequent case), don't write all 16 bytes of it
		uint8 ArchiveFullGuid = 1;
		if (Ar.GameNetVer() == (uint32)EPipelineCacheFileFormatVersions::PatchSizeReduction_NoDuplicatedGuid)
		{
			if (Ar.IsSaving())
			{
				ArchiveFullGuid = (Info.FileGuid != FGuid()) ? 1 : 0;
			}
			Ar << ArchiveFullGuid;
		}
		if (ArchiveFullGuid != 0)
		{
			Ar << Info.FileGuid;
		}
		Ar << Info.Stats;
        if (Ar.GameNetVer() == (uint32)EPipelineCacheFileFormatVersions::LibraryID)
        {
            TSet<uint32> IDs;
            Ar << IDs;
        }
        else if (Ar.GameNetVer() >= (uint32)EPipelineCacheFileFormatVersions::ShaderMetaData)
        {
            Ar << Info.Shaders;
        }
		
		if(Ar.GameNetVer() >= (uint32)EPipelineCacheFileFormatVersions::PSOUsageMask)
		{
			Ar << Info.UsageMask;
		}
		
		if(Ar.GameNetVer() >= (uint32)EPipelineCacheFileFormatVersions::EngineFlags)
		{
			Ar << Info.EngineFlags;
		}

		if (Ar.GameNetVer() >= (uint32)EPipelineCacheFileFormatVersions::LastUsedTime)
		{
			Ar << Info.LastUsedUnixTime;
		}

		return Ar;
	}
};

RHI_API FArchive& operator<<(FArchive& Ar, FPipelineFileCacheRasterizerState& RasterizerStateInitializer)
{
	Ar << RasterizerStateInitializer.DepthBias;
	Ar << RasterizerStateInitializer.SlopeScaleDepthBias;
	Ar << RasterizerStateInitializer.FillMode;
	Ar << RasterizerStateInitializer.CullMode;
	Ar << RasterizerStateInitializer.DepthClipMode;
	Ar << RasterizerStateInitializer.bAllowMSAA;

	if (Ar.GameNetVer() < (uint32)EPipelineCacheFileFormatVersions::RemovingLineAA)
	{
		bool bEnableLineAA = false;
		Ar << bEnableLineAA;
	}
	return Ar;
}

FString FPipelineFileCacheRasterizerState::ToString() const
{
	return FString::Printf(TEXT("<%f %f %u %u %u %u>")
		, DepthBias
		, SlopeScaleDepthBias
		, uint32(FillMode)
		, uint32(CullMode)
		, uint32(DepthClipMode)
		, uint32(!!bAllowMSAA)
	);
}

void FPipelineFileCacheRasterizerState::FromString(const FStringView& Src)
{
	constexpr int32 PartCount = 6;

	TArray<FStringView, TInlineAllocator<PartCount>> Parts;
	UE::String::ParseTokensMultiple(Src.TrimStartAndEnd(), {TEXT('\r'), TEXT('\n'), TEXT('\t'), TEXT('<'), TEXT('>'), TEXT(' ')},
		[&Parts](FStringView Part) { if (!Part.IsEmpty()) { Parts.Add(Part); } });

	check(Parts.Num() == PartCount && sizeof(FillMode) == 1 && sizeof(CullMode) == 1 && sizeof(DepthClipMode) == 1 && sizeof(bAllowMSAA) == 1); //not a very robust parser
	const FStringView* PartIt = Parts.GetData();

	LexFromString(DepthBias, *PartIt++);
	LexFromString(SlopeScaleDepthBias, *PartIt++);
	LexFromString((uint8&)FillMode, *PartIt++);
	LexFromString((uint8&)CullMode, *PartIt++);
	LexFromString((uint8&)DepthClipMode, *PartIt++);
	LexFromString((uint8&)bAllowMSAA, *PartIt++);

	check(Parts.GetData() + PartCount == PartIt);
}

FString FPipelineCacheFileFormatPSO::ComputeDescriptor::ToString() const
{
	return ComputeShader.ToString();
}

void FPipelineCacheFileFormatPSO::ComputeDescriptor::AddToReadableString(TReadableStringBuilder& OutBuilder) const
{
	OutBuilder << TEXT(" CS:");
	OutBuilder << ComputeShader.ToString();
}

void FPipelineCacheFileFormatPSO::ComputeDescriptor::FromString(const FStringView& Src)
{
	ComputeShader.FromString(Src.TrimStartAndEnd());
}

FString FPipelineCacheFileFormatPSO::ComputeDescriptor::HeaderLine()
{
	return FString("ComputeShader");
}

FString FPipelineCacheFileFormatPSO::GraphicsDescriptor::ShadersToString() const
{
	FString Result;

	Result += FString::Printf(TEXT("%s,%s,%s,%s,%s")
		, *VertexShader.ToString()
		, *FragmentShader.ToString()
		, *GeometryShader.ToString()
		, *MeshShader.ToString()
		, *AmplificationShader.ToString()
	);

	return Result;
}

void FPipelineCacheFileFormatPSO::GraphicsDescriptor::AddShadersToReadableString(TReadableStringBuilder& OutBuilder) const
{
	if (VertexShader != FShaderHash())
	{
		OutBuilder << TEXT(" VS:");
		OutBuilder << VertexShader;
	}
	if (MeshShader != FShaderHash())
	{
		OutBuilder << TEXT(" MS:");
		OutBuilder << MeshShader;
	}
	if (GeometryShader != FShaderHash())
	{
		OutBuilder << TEXT(" GS:");
		OutBuilder << GeometryShader;
	}
	if (AmplificationShader != FShaderHash())
	{
		OutBuilder << TEXT(" AS:");
		OutBuilder << AmplificationShader;
	}
	if (FragmentShader != FShaderHash())
	{
		OutBuilder << TEXT(" PS:");
		OutBuilder << FragmentShader;
	}
}

void FPipelineCacheFileFormatPSO::GraphicsDescriptor::ShadersFromString(const FStringView& Src)
{
	constexpr int32 PartCount = 5;

	TArray<FStringView, TInlineAllocator<PartCount>> Parts;
	UE::String::ParseTokens(Src.TrimStartAndEnd(), TEXT(','), [&Parts](FStringView Part) { Parts.Add(Part); });

	check(Parts.Num() == PartCount); //not a very robust parser
	const FStringView* PartIt = Parts.GetData();

	VertexShader.FromString(*PartIt++);
	FragmentShader.FromString(*PartIt++);
	GeometryShader.FromString(*PartIt++);
	MeshShader.FromString(*PartIt++);
	AmplificationShader.FromString(*PartIt++);

	check(Parts.GetData() + PartCount == PartIt);
}

FString FPipelineCacheFileFormatPSO::GraphicsDescriptor::ShaderHeaderLine()
{
	return FString("VertexShader,FragmentShader,GeometryShader,MeshShader,AmplificationShader");
}

FString FPipelineCacheFileFormatPSO::GraphicsDescriptor::StateToString() const
{
	FString Result;

	Result += FString::Printf(TEXT("%s,%s,%s,")
		, *BlendState.ToString()
		, *RasterizerState.ToString()
		, *DepthStencilState.ToString()
	);
	Result += FString::Printf(TEXT("%d,%d,%lld,")
		, MSAASamples
		, uint32(DepthStencilFormat)
		, uint64{EnumToUnderlyingType(DepthStencilFlags)}
	);
	Result += FString::Printf(TEXT("%d,%d,%d,%d,%d,")
		, uint32(DepthLoad)
		, uint32(StencilLoad)
		, uint32(DepthStore)
		, uint32(StencilStore)
		, uint32(PrimitiveType)
	);

	Result += FString::Printf(TEXT("%d,")
		, RenderTargetsActive
	);
	for (int32 Index = 0; Index < MaxSimultaneousRenderTargets; Index++)
	{
		Result += FString::Printf(TEXT("%d,%lld,%d,%d,")
			, uint32(RenderTargetFormats[Index])
			, uint64{EnumToUnderlyingType(RenderTargetFlags[Index])}
			, 0/*Load*/
			, 0/*Store*/
		);
	}

	Result += FString::Printf(TEXT("%d,%d,")
		, uint32(SubpassHint)
		, uint32(SubpassIndex)
	);

	Result += FString::Printf(TEXT("%d,%d,")
		, uint32(MultiViewCount)
		, uint32(bHasFragmentDensityAttachment)
	);

	Result += FString::Printf(TEXT("%d,%d,")
		, uint32(bAllowVariableRateShading)
		, uint32(ShadingRate)
	);

	Result += FString::Printf(TEXT("%d,")
		, uint32(bDepthBounds)
	);
	
	FVertexElement NullVE;
	FMemory::Memzero(NullVE);
	Result += FString::Printf(TEXT("%d,")
		, VertexDescriptor.Num()
	);
	for (int32 Index = 0; Index < MaxVertexElementCount; Index++)
	{
		if (Index < VertexDescriptor.Num())
		{
			Result += FString::Printf(TEXT("%s,")
				, *VertexDescriptor[Index].ToString()
			);
		}
		else
		{
			Result += FString::Printf(TEXT("%s,")
				, *NullVE.ToString()
			);
		}
	}
	return Result.Left(Result.Len() - 1); // remove trailing comma
}

void FPipelineCacheFileFormatPSO::GraphicsDescriptor::AddStateToReadableString(TReadableStringBuilder& OutBuilder) const
{
	OutBuilder << TEXT(" BS:");
	OutBuilder << BlendState.ToString();
	OutBuilder << TEXT(" RS:");
	OutBuilder << RasterizerState.ToString();
	OutBuilder << TEXT(" DSS:");
	OutBuilder << DepthStencilState.ToString();
	OutBuilder << TEXT("\n");

	OutBuilder << TEXT(" NumMSAA:");
	OutBuilder << MSAASamples;
	OutBuilder << TEXT(" DSfmt:");
	OutBuilder << uint32(DepthStencilFormat);
	OutBuilder << TEXT(" DSflags:");
	OutBuilder << uint64(DepthStencilFlags);
	OutBuilder << TEXT("\n");

	OutBuilder << TEXT(" DL:");
	OutBuilder << uint32(DepthLoad);
	OutBuilder << TEXT(" SL:");
	OutBuilder << uint32(StencilLoad);
	OutBuilder << TEXT(" DS:");
	OutBuilder << uint32(DepthStore);
	OutBuilder << TEXT(" SS:");
	OutBuilder << uint32(StencilStore);
	OutBuilder << TEXT(" PT:");
	OutBuilder << uint32(PrimitiveType);
	OutBuilder << TEXT("\n");

	OutBuilder << TEXT(" RTA ");
	OutBuilder << RenderTargetsActive;
	OutBuilder << TEXT("\n");

	if (RenderTargetsActive)
	{
		OutBuilder << TEXT("    ");
		for (uint32 Index = 0; Index < RenderTargetsActive; Index++)
		{
			OutBuilder << TEXT(" RT");
			OutBuilder << Index,
			OutBuilder << TEXT(":fmt=");
			OutBuilder << uint32(RenderTargetFormats[Index]);
			OutBuilder << TEXT(" flg=");
			OutBuilder << uint64(RenderTargetFlags[Index]);
		}
		OutBuilder << TEXT("\n");
	}

	OutBuilder << TEXT(" SuH:");
	OutBuilder << uint32(SubpassHint);
	OutBuilder << TEXT(" SuI:");
	OutBuilder << uint32(SubpassIndex);
	OutBuilder << TEXT("\n");

	OutBuilder << TEXT(" MVC:");
	OutBuilder << MultiViewCount;
	OutBuilder << TEXT(" HasFDM:");
	OutBuilder << bHasFragmentDensityAttachment;
	OutBuilder << TEXT("\n");

	OutBuilder << TEXT(" AllowVRS:");
	OutBuilder << bAllowVariableRateShading;
	OutBuilder << TEXT(" SR:");
	OutBuilder << uint32(ShadingRate);
	OutBuilder << TEXT("\n");

	OutBuilder << TEXT(" DB:");
	OutBuilder << bDepthBounds;
	OutBuilder << TEXT("\n");

	OutBuilder << TEXT(" NumVE ");
	OutBuilder << VertexDescriptor.Num();
	OutBuilder << TEXT("\n");

	for (int32 Index = 0; Index < VertexDescriptor.Num(); Index++)
	{
		OutBuilder << TEXT(" ");
		OutBuilder << Index;
		OutBuilder << TEXT(":");
		OutBuilder << VertexDescriptor[Index].ToString();
	}
}

bool FPipelineCacheFileFormatPSO::GraphicsDescriptor::StateFromString(const FStringView& Src)
{
	static_assert(sizeof(EPixelFormat) == 1);
	static_assert(sizeof(ERenderTargetLoadAction) == 1);
	static_assert(sizeof(ERenderTargetStoreAction) == 1);
	static_assert(sizeof(DepthLoad) == 1);
	static_assert(sizeof(DepthStore) == 1);
	static_assert(sizeof(StencilLoad) == 1);
	static_assert(sizeof(StencilStore) == 1);
	static_assert(sizeof(PrimitiveType) == 4);

	constexpr int32 PartCount = FPipelineCacheGraphicsDescPartsNum;

	TArray<FStringView, TInlineAllocator<PartCount>> Parts;
	UE::String::ParseTokens(Src.TrimStartAndEnd(), TEXT(','), [&Parts](FStringView Part) { Parts.Add(Part); });

	// check if we have expected number of parts
	if (Parts.Num() != PartCount)
	{
		// instead of crashing let caller handle this case
		return false;
	}

	const FStringView* PartIt = Parts.GetData();
	const FStringView* PartEnd = PartIt + PartCount;

	check(PartEnd - PartIt >= 3); //not a very robust parser
	BlendState.FromString(*PartIt++);
	RasterizerState.FromString(*PartIt++);
	DepthStencilState.FromString(*PartIt++);

	check(PartEnd - PartIt >= 3); //not a very robust parser
	LexFromString(MSAASamples, *PartIt++);
	LexFromString((uint32&)DepthStencilFormat, *PartIt++);
	ETextureCreateFlags DSFlags;
	LexFromString(DSFlags, *PartIt++);
	DepthStencilFlags = ReduceDSFlags(DSFlags);

	check(PartEnd - PartIt >= 5); //not a very robust parser
	LexFromString((uint32&)DepthLoad, *PartIt++);
	LexFromString((uint32&)StencilLoad, *PartIt++);
	LexFromString((uint32&)DepthStore, *PartIt++);
	LexFromString((uint32&)StencilStore, *PartIt++);
	LexFromString((uint32&)PrimitiveType, *PartIt++);

	check(PartEnd - PartIt >= 1); //not a very robust parser
	LexFromString(RenderTargetsActive, *PartIt++);

	for (int32 Index = 0; Index < MaxSimultaneousRenderTargets; Index++)
	{
		check(PartEnd - PartIt >= 4); //not a very robust parser
		LexFromString((uint8&)(RenderTargetFormats[Index]), *PartIt++);
		ETextureCreateFlags RTFlags;
		LexFromString(RTFlags, *PartIt++);
		// going forward, the flags will already be reduced when logging the PSOs to disk. However as of 2021-06-17 there are still old stable cache files in existence that have flags recorded as is
		RenderTargetFlags[Index] = ReduceRTFlags(RTFlags);
		uint8 Load, Store;
		LexFromString(Load, *PartIt++);
		LexFromString(Store, *PartIt++);
	}

	// parse sub-pass information
	{
		uint32 LocalSubpassHint = 0;
		uint32 LocalSubpassIndex = 0;
		check(PartEnd - PartIt >= 2);
		LexFromString(LocalSubpassHint, *PartIt++);
		LexFromString(LocalSubpassIndex, *PartIt++);
		SubpassHint = LocalSubpassHint;
		SubpassIndex = LocalSubpassIndex;
	}

	// parse multiview and FDM information
	{
		uint32 LocalMultiViewCount = 0;
		uint32 LocalHasFDM = 0;
		check(PartEnd - PartIt >= 2);
		LexFromString(LocalMultiViewCount, *PartIt++);
		LexFromString(LocalHasFDM, *PartIt++);
		MultiViewCount = (uint8)LocalMultiViewCount;
		bHasFragmentDensityAttachment = (bool)LocalHasFDM;
	}

	// parse variable rate shading information
	{
		uint32 LocalAllowVariableRateShading = 0;
		uint32 LocalShadingRate = 0;
		check(PartEnd - PartIt >= 2);
		LexFromString(LocalAllowVariableRateShading, *PartIt++);
		LexFromString(LocalShadingRate, *PartIt++);
		bAllowVariableRateShading = (bool)LocalAllowVariableRateShading;
		ShadingRate = (EVRSShadingRate)LocalShadingRate;
	}

	// parse depth bounds
	{
		uint32 DepthBounds = 0;
		check(PartEnd - PartIt >= 1);
		LexFromString(DepthBounds, *PartIt++);
		bDepthBounds = (bool)DepthBounds;
	}

	check(PartEnd - PartIt >= 1); //not a very robust parser
	int32 VertDescNum = 0;
	LexFromString(VertDescNum, *PartIt++);
	check(VertDescNum >= 0 && VertDescNum <= MaxVertexElementCount);

	VertexDescriptor.Empty(VertDescNum);
	VertexDescriptor.AddZeroed(VertDescNum);

	check(PartEnd - PartIt == MaxVertexElementCount); //not a very robust parser
	for (int32 Index = 0; Index < VertDescNum; Index++)
	{
		VertexDescriptor[Index].FromString(*PartIt++);
	}

	check(PartIt + MaxVertexElementCount == PartEnd + VertDescNum);

	SortVertexElements(VertexDescriptor);

	return true;
}

ETextureCreateFlags FPipelineCacheFileFormatPSO::GraphicsDescriptor::ReduceRTFlags(ETextureCreateFlags InFlags)
{
	// We care about flags that influence RT formats (which is the only thing the underlying API cares about).
	// In most RHIs, the format is only influenced by TexCreate_SRGB. D3D12 additionally uses TexCreate_Shared in its format selection logic.
	return (InFlags & FGraphicsPipelineStateInitializer::RelevantRenderTargetFlagMask);
}

ETextureCreateFlags FPipelineCacheFileFormatPSO::GraphicsDescriptor::ReduceDSFlags(ETextureCreateFlags InFlags)
{
	return (InFlags & FGraphicsPipelineStateInitializer::RelevantDepthStencilFlagMask);
}

FString FPipelineCacheFileFormatPSO::GraphicsDescriptor::StateHeaderLine()
{
	FString Result;

	Result += FString::Printf(TEXT("%s,%s,%s,")
		, TEXT("BlendState")
		, TEXT("RasterizerState")
		, TEXT("DepthStencilState")
	);
	Result += FString::Printf(TEXT("%s,%s,%s,")
		, TEXT("MSAASamples")
		, TEXT("DepthStencilFormat")
		, TEXT("DepthStencilFlags")
	);
	Result += FString::Printf(TEXT("%s,%s,%s,%s,%s,")
		, TEXT("DepthLoad")
		, TEXT("StencilLoad")
		, TEXT("DepthStore")
		, TEXT("StencilStore")
		, TEXT("PrimitiveType")
	);
	
	Result += FString::Printf(TEXT("%s,")
		, TEXT("RenderTargetsActive")
	);
	for (int32 Index = 0; Index < MaxSimultaneousRenderTargets; Index++)
	{
		Result += FString::Printf(TEXT("%s%d,%s%d,%s%d,%s%d,")
			, TEXT("RenderTargetFormats"), Index
			, TEXT("RenderTargetFlags"), Index
			, TEXT("RenderTargetsLoad"), Index
			, TEXT("RenderTargetsStore"), Index
		);
	}

	Result += FString::Printf(TEXT("%s,%s,")
		, TEXT("SubpassHint")
		, TEXT("SubpassIndex")
	);

	Result += FString::Printf(TEXT("%s,%s,")
		, TEXT("MultiViewCount")
		, TEXT("bHasFDMAttachment")
	);
	
	Result += FString::Printf(TEXT("%s,")
		, TEXT("VertexDescriptorNum")
	);
	for (int32 Index = 0; Index < MaxVertexElementCount; Index++)
	{
		Result += FString::Printf(TEXT("%s%d,")
			, TEXT("VertexDescriptor"), Index
		);
	}
	return Result.Left(Result.Len() - 1); // remove trailing comma
}

FString FPipelineCacheFileFormatPSO::GraphicsDescriptor::ToString() const
{
	return FString::Printf(TEXT("%s,%s"), *ShadersToString(), *StateToString());
}

void FPipelineCacheFileFormatPSO::GraphicsDescriptor::AddToReadableString(TReadableStringBuilder& OutBuilder) const
{
	AddShadersToReadableString(OutBuilder);
	OutBuilder << TEXT("\n");
	AddStateToReadableString(OutBuilder);
	OutBuilder << TEXT("\n");
}

bool FPipelineCacheFileFormatPSO::GraphicsDescriptor::FromString(const FStringView& Src)
{
	constexpr int32 NumShaderParts = 5;

	int32 StateOffset = 0;
	for (int32 CommaCount = 0; CommaCount < NumShaderParts; ++CommaCount)
	{
		int32 CommaOffset = 0;
		bool FoundComma = Src.RightChop(StateOffset).FindChar(TEXT(','), CommaOffset);
		check(FoundComma);
		StateOffset += CommaOffset + 1;
	}

	ShadersFromString(Src.Left(StateOffset - 1));
	return StateFromString(Src.RightChop(StateOffset));
}

FString FPipelineCacheFileFormatPSO::GraphicsDescriptor::HeaderLine()
{
	return FString::Printf(TEXT("%s,%s"), *ShaderHeaderLine(), *StateHeaderLine());
}


FString FPipelineCacheFileFormatPSO::CommonHeaderLine()
{
	return TEXT("BindCount,UsageMask");
}

FString FPipelineCacheFileFormatPSO::CommonToString() const
{
	uint64 Mask = 0;
	int64 Count = 0;
#if PSO_COOKONLY_DATA
	Mask = UsageMask;
	Count = BindCount;
#endif
	return FString::Printf(TEXT("\"%" INT64_FMT ",%" UINT64_FMT "\""), Count, Mask);
}

FString FPipelineCacheFileFormatPSO::ToStringReadable() const
{
	TReadableStringBuilder Builder;

	Builder << TEXT("PSO hash ");
	Builder << GetTypeHash(*this);	
	Builder << TEXT(" PSOPrecacheResult ");
	Builder << LexToString(PSOPrecacheResult);
#if PSO_COOKONLY_DATA
	Builder << TEXT(" mask ");
	Builder << UsageMask;
	Builder << TEXT(" bindc ");
	Builder << BindCount;
#endif
	Builder << TEXT("\n");

	if (Type == DescriptorType::Graphics)
	{
		GraphicsDesc.AddToReadableString(Builder);
	}
	else if (Type == DescriptorType::Compute)
	{
		ComputeDesc.AddToReadableString(Builder);
	}
	else if (Type == DescriptorType::RayTracing)
	{
		RayTracingDesc.AddToReadableString(Builder);
	}
	else
	{
		Builder << TEXT(" Unknown PSO type ");
		Builder << static_cast<int32>(Type);
	}

	return FString(FStringView(Builder));
}


void FPipelineCacheFileFormatPSO::CommonFromString(const FStringView& Src)
{
#if PSO_COOKONLY_DATA
    TArray<FStringView, TInlineAllocator<2>> Parts;
	UE::String::ParseTokens(Src.TrimStartAndEnd(), TEXT(','), [&Parts](FStringView Part) { Parts.Add(Part); });

	if (Parts.Num() == 1)
	{
		LexFromString(UsageMask, Parts[0]);
	}
	else if(Parts.Num() > 1)
	{
		LexFromString(BindCount, Parts[0]);
		LexFromString(UsageMask, Parts[1]);
	}
#endif
}

bool FPipelineCacheFileFormatPSO::Verify() const
{
	if(Type == DescriptorType::Compute)
	{
		return ComputeDesc.ComputeShader != FShaderHash();
	}
	else if(Type == DescriptorType::Graphics)
	{
		if (GraphicsDesc.VertexShader == FShaderHash() && GraphicsDesc.MeshShader == FShaderHash())
		{
			// No vertex or mesh shader - no graphics - nothing else matters
			return false;
		}

#if PLATFORM_SUPPORTS_MESH_SHADERS
		if (GraphicsDesc.MeshShader != FShaderHash())
		{
			// this check is also done in commandlets, which don't set RHI settings properly. Exempt them.
			if (!IsRunningCommandlet() && !GRHISupportsMeshShadersTier0)
			{
				// do not allow precompilation of mesh shaders if runtime doesn't support them
				return false;
			}

			if (GraphicsDesc.VertexShader != FShaderHash())
			{
				// Vertex shader and mesh shader are mutually exclusive
				return false;
			}

			if (GraphicsDesc.VertexDescriptor.Num() > 0)
			{
				// mesh shader should not have descriptors
				return false;
			}
		}
#endif
		
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		// Is there anything to actually test here?
#endif
		
		if( GraphicsDesc.RenderTargetsActive > MaxSimultaneousRenderTargets ||
			GraphicsDesc.MSAASamples > 16 ||
			(uint32)GraphicsDesc.PrimitiveType >= (uint32)EPrimitiveType::PT_Num ||
			(uint32)GraphicsDesc.DepthStencilFormat >= (uint32)EPixelFormat::PF_MAX ||
			(uint8)GraphicsDesc.DepthLoad >= (uint8)ERenderTargetLoadAction::Num ||
			(uint8)GraphicsDesc.StencilLoad >= (uint8)ERenderTargetLoadAction::Num ||
			(uint8)GraphicsDesc.DepthStore >= (uint8)ERenderTargetStoreAction::Num ||
			(uint8)GraphicsDesc.StencilStore >= (uint8)ERenderTargetStoreAction::Num )
		{
			return false;
		}
		
		for(uint32 rt = 0;rt < GraphicsDesc.RenderTargetsActive;++rt)
		{
			if((uint32)GraphicsDesc.RenderTargetFormats[rt] >= (uint32)EPixelFormat::PF_MAX)
			{
				return false;
			}
			
			if( GraphicsDesc.BlendState.RenderTargets[rt].ColorBlendOp >= EBlendOperation::EBlendOperation_Num ||
				GraphicsDesc.BlendState.RenderTargets[rt].AlphaBlendOp >= EBlendOperation::EBlendOperation_Num ||
				GraphicsDesc.BlendState.RenderTargets[rt].ColorSrcBlend >= EBlendFactor::EBlendFactor_Num ||
				GraphicsDesc.BlendState.RenderTargets[rt].ColorDestBlend >= EBlendFactor::EBlendFactor_Num ||
				GraphicsDesc.BlendState.RenderTargets[rt].AlphaSrcBlend >= EBlendFactor::EBlendFactor_Num ||
				GraphicsDesc.BlendState.RenderTargets[rt].AlphaDestBlend >= EBlendFactor::EBlendFactor_Num ||
				GraphicsDesc.BlendState.RenderTargets[rt].ColorWriteMask > 0xf)
			{
				return false;
			}
		}
		
		if( (uint8)GraphicsDesc.RasterizerState.FillMode >= (uint8)ERasterizerFillMode::ERasterizerFillMode_Num ||
			(uint8)GraphicsDesc.RasterizerState.CullMode >= (uint8)ERasterizerCullMode_Num)
		{
			return false;
		}
		
		if( (uint8)GraphicsDesc.DepthStencilState.DepthTest >= (uint8)ECompareFunction::ECompareFunction_Num ||
			(uint8)GraphicsDesc.DepthStencilState.FrontFaceStencilTest >= (uint8)ECompareFunction::ECompareFunction_Num ||
			(uint8)GraphicsDesc.DepthStencilState.BackFaceStencilTest >= (uint8)ECompareFunction::ECompareFunction_Num ||
			(uint8)GraphicsDesc.DepthStencilState.FrontFaceStencilFailStencilOp >= (uint8)EStencilOp::EStencilOp_Num ||
			(uint8)GraphicsDesc.DepthStencilState.FrontFaceDepthFailStencilOp >= (uint8)EStencilOp::EStencilOp_Num ||
			(uint8)GraphicsDesc.DepthStencilState.FrontFacePassStencilOp >= (uint8)EStencilOp::EStencilOp_Num ||
			(uint8)GraphicsDesc.DepthStencilState.BackFaceStencilFailStencilOp >= (uint8)EStencilOp::EStencilOp_Num ||
			(uint8)GraphicsDesc.DepthStencilState.BackFaceDepthFailStencilOp >= (uint8)EStencilOp::EStencilOp_Num ||
			(uint8)GraphicsDesc.DepthStencilState.BackFacePassStencilOp >= (uint8)EStencilOp::EStencilOp_Num)
		{
			return false;
		}

		uint32 ElementCount = (uint32)GraphicsDesc.VertexDescriptor.Num();
		for (uint32 i = 0; i < ElementCount;++i)
		{
			if(GraphicsDesc.VertexDescriptor[i].Type >= EVertexElementType::VET_MAX)
			{
				return false;
			}
		}
		
		return true;
	}
	else if (Type == DescriptorType::RayTracing)
	{
		return RayTracingDesc.ShaderHash != FShaderHash() &&
			RayTracingDesc.Frequency >= SF_RayGen &&
			RayTracingDesc.Frequency <= SF_RayCallable;
	}
	else
	{
		checkNoEntry();
	}
	
	return false;
}

/**
  * FPipelineCacheFileFormatPSO
  **/

/*friend*/ uint32 GetTypeHash(const FPipelineCacheFileFormatPSO &Key)
{
	uint32 KeyHash = GetTypeHash(Key.Type);
	switch(Key.Type)
	{
		case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
		{
			KeyHash ^= GetTypeHash(Key.ComputeDesc.ComputeShader);
			break;
		}
		case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
		{
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.RenderTargetsActive, sizeof(Key.GraphicsDesc.RenderTargetsActive), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.MSAASamples, sizeof(Key.GraphicsDesc.MSAASamples), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.PrimitiveType, sizeof(Key.GraphicsDesc.PrimitiveType), KeyHash);
				
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.VertexShader.Hash, sizeof(Key.GraphicsDesc.VertexShader.Hash), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.FragmentShader.Hash, sizeof(Key.GraphicsDesc.FragmentShader.Hash), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.GeometryShader.Hash, sizeof(Key.GraphicsDesc.GeometryShader.Hash), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.MeshShader.Hash, sizeof(Key.GraphicsDesc.MeshShader.Hash), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.AmplificationShader.Hash, sizeof(Key.GraphicsDesc.AmplificationShader.Hash), KeyHash);

			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilFormat, sizeof(Key.GraphicsDesc.DepthStencilFormat), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilFlags, sizeof(Key.GraphicsDesc.DepthStencilFlags), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthLoad, sizeof(Key.GraphicsDesc.DepthLoad), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.StencilLoad, sizeof(Key.GraphicsDesc.StencilLoad), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStore, sizeof(Key.GraphicsDesc.DepthStore), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.StencilStore, sizeof(Key.GraphicsDesc.StencilStore), KeyHash);

			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.BlendState.bUseIndependentRenderTargetBlendStates, sizeof(Key.GraphicsDesc.BlendState.bUseIndependentRenderTargetBlendStates), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.BlendState.bUseAlphaToCoverage, sizeof(Key.GraphicsDesc.BlendState.bUseAlphaToCoverage), KeyHash);
			for( uint32 i = 0; i < MaxSimultaneousRenderTargets; i++ )
			{
				KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.BlendState.RenderTargets[i].ColorBlendOp, sizeof(Key.GraphicsDesc.BlendState.RenderTargets[i].ColorBlendOp), KeyHash);
				KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.BlendState.RenderTargets[i].ColorSrcBlend, sizeof(Key.GraphicsDesc.BlendState.RenderTargets[i].ColorSrcBlend), KeyHash);
				KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.BlendState.RenderTargets[i].ColorDestBlend, sizeof(Key.GraphicsDesc.BlendState.RenderTargets[i].ColorDestBlend), KeyHash);
				KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.BlendState.RenderTargets[i].ColorWriteMask, sizeof(Key.GraphicsDesc.BlendState.RenderTargets[i].ColorWriteMask), KeyHash);
				KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.BlendState.RenderTargets[i].AlphaBlendOp, sizeof(Key.GraphicsDesc.BlendState.RenderTargets[i].AlphaBlendOp), KeyHash);
				KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.BlendState.RenderTargets[i].AlphaSrcBlend, sizeof(Key.GraphicsDesc.BlendState.RenderTargets[i].AlphaSrcBlend), KeyHash);
				KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.BlendState.RenderTargets[i].AlphaDestBlend, sizeof(Key.GraphicsDesc.BlendState.RenderTargets[i].AlphaDestBlend), KeyHash);
			}

			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.RenderTargetFormats, sizeof(Key.GraphicsDesc.RenderTargetFormats), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.RenderTargetFlags, sizeof(Key.GraphicsDesc.RenderTargetFlags), KeyHash);
				
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.SubpassHint, sizeof(Key.GraphicsDesc.SubpassHint), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.SubpassIndex, sizeof(Key.GraphicsDesc.SubpassIndex), KeyHash);
				
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.MultiViewCount, sizeof(Key.GraphicsDesc.MultiViewCount), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.bHasFragmentDensityAttachment, sizeof(Key.GraphicsDesc.bHasFragmentDensityAttachment), KeyHash);

			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.bAllowVariableRateShading, sizeof(Key.GraphicsDesc.bAllowVariableRateShading), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.ShadingRate, sizeof(Key.GraphicsDesc.ShadingRate), KeyHash);

			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.bDepthBounds, sizeof(Key.GraphicsDesc.bDepthBounds), KeyHash);

			for(auto const& Element : Key.GraphicsDesc.VertexDescriptor)
			{
				KeyHash = FCrc::MemCrc32(&Element, sizeof(FVertexElement), KeyHash);
			}
				
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.RasterizerState.DepthBias, sizeof(Key.GraphicsDesc.RasterizerState.DepthBias), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.RasterizerState.SlopeScaleDepthBias, sizeof(Key.GraphicsDesc.RasterizerState.SlopeScaleDepthBias), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.RasterizerState.FillMode, sizeof(Key.GraphicsDesc.RasterizerState.FillMode), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.RasterizerState.CullMode, sizeof(Key.GraphicsDesc.RasterizerState.CullMode), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.RasterizerState.bAllowMSAA, sizeof(Key.GraphicsDesc.RasterizerState.bAllowMSAA), KeyHash);
				
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.bEnableDepthWrite, sizeof(Key.GraphicsDesc.DepthStencilState.bEnableDepthWrite), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.DepthTest, sizeof(Key.GraphicsDesc.DepthStencilState.DepthTest), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.bEnableFrontFaceStencil, sizeof(Key.GraphicsDesc.DepthStencilState.bEnableFrontFaceStencil), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.FrontFaceStencilTest, sizeof(Key.GraphicsDesc.DepthStencilState.FrontFaceStencilTest), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.FrontFaceStencilFailStencilOp, sizeof(Key.GraphicsDesc.DepthStencilState.FrontFaceStencilFailStencilOp), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.FrontFaceDepthFailStencilOp, sizeof(Key.GraphicsDesc.DepthStencilState.FrontFaceDepthFailStencilOp), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.FrontFacePassStencilOp, sizeof(Key.GraphicsDesc.DepthStencilState.FrontFacePassStencilOp), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.bEnableBackFaceStencil, sizeof(Key.GraphicsDesc.DepthStencilState.bEnableBackFaceStencil), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.BackFaceStencilTest, sizeof(Key.GraphicsDesc.DepthStencilState.BackFaceStencilTest), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.BackFaceStencilFailStencilOp, sizeof(Key.GraphicsDesc.DepthStencilState.BackFaceStencilFailStencilOp), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.BackFaceDepthFailStencilOp, sizeof(Key.GraphicsDesc.DepthStencilState.BackFaceDepthFailStencilOp), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.BackFacePassStencilOp, sizeof(Key.GraphicsDesc.DepthStencilState.BackFacePassStencilOp), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.StencilReadMask, sizeof(Key.GraphicsDesc.DepthStencilState.StencilReadMask), KeyHash);
			KeyHash = FCrc::MemCrc32(&Key.GraphicsDesc.DepthStencilState.StencilWriteMask, sizeof(Key.GraphicsDesc.DepthStencilState.StencilWriteMask), KeyHash);
				
			break;
		}
		case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
		{
			KeyHash ^= GetTypeHash(Key.RayTracingDesc);
			break;
		}
		default:
		{
			checkNoEntry();
		}
	}

	return KeyHash;
}

/*friend*/ FArchive& operator<<( FArchive& Ar, FPipelineCacheFileFormatPSO& Info )
{
	Ar << Info.Type;
	if (Ar.GameNetVer() >= (uint32)EPipelineCacheFileFormatVersions::PSOPrecacheResult)
	{
		Ar << Info.PSOPrecacheResult;
	}
#if PSO_COOKONLY_DATA
	/* Ignore: Ar << Info.UsageMask; during serialization */
	/* Ignore: Ar << Info.BindCoun during  serialization*/
#endif

	switch (Info.Type)
	{
		case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
		{
			Ar << Info.ComputeDesc.ComputeShader;
            if (Ar.GameNetVer() == (uint32)EPipelineCacheFileFormatVersions::LibraryID)
            {
                uint32 ID = 0;
                Ar << ID;
            }
			break;
		}
		case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
		{
			Ar << Info.GraphicsDesc.VertexShader;
			Ar << Info.GraphicsDesc.FragmentShader;
			Ar << Info.GraphicsDesc.GeometryShader;

			if (Ar.GameNetVer() < (uint32)EPipelineCacheFileFormatVersions::RemovingTessellationShaders)
			{
				FShaderHash HullShader;
				Ar << HullShader;
				FShaderHash DomainShader;
				Ar << DomainShader;
			}
			if (Ar.GameNetVer() >= (uint32)EPipelineCacheFileFormatVersions::AddingMeshShaders)
			{
				Ar << Info.GraphicsDesc.MeshShader;
				Ar << Info.GraphicsDesc.AmplificationShader;
			}
            if (Ar.GameNetVer() == (uint32)EPipelineCacheFileFormatVersions::LibraryID)
            {
				for (uint32 i = 0; i < SF_Compute; i++)
				{
                    uint32 ID = 0;
                    Ar << ID;
                }
			}
			if (Ar.GameNetVer() < (uint32)EPipelineCacheFileFormatVersions::SortedVertexDesc)
			{
				check(Ar.IsLoading());
				
				FVertexDeclarationElementList Elements;
				Ar << Elements;
				FPipelineCacheFileFormatPSO::SortVertexElements(Elements);
				
				Info.GraphicsDesc.VertexDescriptor.AddZeroed(Elements.Num());
				for (uint32 i = 0; i < (uint32)Elements.Num(); i++)
				{
					Info.GraphicsDesc.VertexDescriptor[i].StreamIndex = Elements[i].StreamIndex;
					Info.GraphicsDesc.VertexDescriptor[i].Offset = Elements[i].Offset;
					Info.GraphicsDesc.VertexDescriptor[i].Type = Elements[i].Type;
					Info.GraphicsDesc.VertexDescriptor[i].AttributeIndex = Elements[i].AttributeIndex;
					Info.GraphicsDesc.VertexDescriptor[i].Stride = Elements[i].Stride;
					Info.GraphicsDesc.VertexDescriptor[i].bUseInstanceIndex = Elements[i].bUseInstanceIndex;
				}
			}
			else
			{
				Ar << Info.GraphicsDesc.VertexDescriptor;
			}
			Ar << Info.GraphicsDesc.BlendState;
			Ar << Info.GraphicsDesc.RasterizerState;
			Ar << Info.GraphicsDesc.DepthStencilState;
			for ( uint32 i = 0; i < MaxSimultaneousRenderTargets; i++ )
			{
				uint32 Format = (uint32)Info.GraphicsDesc.RenderTargetFormats[i];
				Ar << Format;
				Info.GraphicsDesc.RenderTargetFormats[i] = (EPixelFormat)Format;

				if (Ar.GameNetVer() < (uint32)EPipelineCacheFileFormatVersions::MoreRenderTargetFlags)
				{
					uint32 RTFlags = 0;
					Ar << RTFlags;
					// going forward, the flags will already be reduced when logging the PSOs to disk. However as of 2021-06-17 there still exist cache files (e.g. user ones) that have flags recorded as is
					Info.GraphicsDesc.RenderTargetFlags[i] = FPipelineCacheFileFormatPSO::GraphicsDescriptor::ReduceRTFlags(static_cast<ETextureCreateFlags>(RTFlags));
				}
				else
				{
					static_assert(sizeof(uint64) == sizeof(Info.GraphicsDesc.RenderTargetFlags[i]), "ETextureCreateFlags size changed, please change serialization");
					uint64 RTFlags = static_cast<uint64>(FPipelineCacheFileFormatPSO::GraphicsDescriptor::ReduceRTFlags(Info.GraphicsDesc.RenderTargetFlags[i]));
					Ar << RTFlags;
					Info.GraphicsDesc.RenderTargetFlags[i] = FPipelineCacheFileFormatPSO::GraphicsDescriptor::ReduceRTFlags(static_cast<ETextureCreateFlags>(RTFlags));
				}
				uint8 LoadStore = 0;
				Ar << LoadStore;
				Ar << LoadStore;
			}
			Ar << Info.GraphicsDesc.RenderTargetsActive;
			Ar << Info.GraphicsDesc.MSAASamples;
			uint32 PrimType = (uint32)Info.GraphicsDesc.PrimitiveType;
			Ar << PrimType;
			Info.GraphicsDesc.PrimitiveType = (EPrimitiveType)PrimType;
			uint32 Format = (uint32)Info.GraphicsDesc.DepthStencilFormat;
			Ar << Format;
			Info.GraphicsDesc.DepthStencilFormat = (EPixelFormat)Format;
			if (Ar.GameNetVer() < (uint32)EPipelineCacheFileFormatVersions::MoreRenderTargetFlags)
			{
				uint32 DepthStencilFlags = 0;
				Ar << DepthStencilFlags;
				Info.GraphicsDesc.DepthStencilFlags = FPipelineCacheFileFormatPSO::GraphicsDescriptor::ReduceDSFlags(static_cast<ETextureCreateFlags>(DepthStencilFlags));
			}
			else
			{
				static_assert(sizeof(uint64) == sizeof(Info.GraphicsDesc.DepthStencilFlags), "ETextureCreateFlags size changed, please change serialization");
				uint64 DepthStencilFlags = static_cast<uint64>(FPipelineCacheFileFormatPSO::GraphicsDescriptor::ReduceDSFlags(Info.GraphicsDesc.DepthStencilFlags));
				Ar << DepthStencilFlags;
				Info.GraphicsDesc.DepthStencilFlags = FPipelineCacheFileFormatPSO::GraphicsDescriptor::ReduceDSFlags(static_cast<ETextureCreateFlags>(DepthStencilFlags));
			}
			Ar << Info.GraphicsDesc.DepthLoad;
			Ar << Info.GraphicsDesc.StencilLoad;
			Ar << Info.GraphicsDesc.DepthStore;
			Ar << Info.GraphicsDesc.StencilStore;

			Ar << Info.GraphicsDesc.SubpassHint;
			Ar << Info.GraphicsDesc.SubpassIndex;

			if (Ar.GameNetVer() >= (uint32)EPipelineCacheFileFormatVersions::FragmentDensityAttachment)
			{
				Ar << Info.GraphicsDesc.MultiViewCount;
				Ar << Info.GraphicsDesc.bHasFragmentDensityAttachment;
			}

			if (Ar.GameNetVer() >= (uint32)EPipelineCacheFileFormatVersions::VariableRateShading)
			{
				Ar << Info.GraphicsDesc.bAllowVariableRateShading;
				uint32 ShadingRate = (uint32)Info.GraphicsDesc.ShadingRate;
				Ar << ShadingRate;
				Info.GraphicsDesc.ShadingRate = (EVRSShadingRate)ShadingRate;
			}

			if (Ar.GameNetVer() >= (uint32)EPipelineCacheFileFormatVersions::AddingDepthBounds)
			{
				Ar << Info.GraphicsDesc.bDepthBounds;
			}

			break;
		}
		case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
		{
			Ar << Info.RayTracingDesc.ShaderHash;

			// Not used, kept for binary format compatibility
			Ar << Info.RayTracingDesc.DeprecatedMaxPayloadSizeInBytes;

			uint32 Frequency = uint32(Info.RayTracingDesc.Frequency);
			Ar << Frequency;
			Info.RayTracingDesc.Frequency = EShaderFrequency(Frequency);
			
			if (Ar.GameNetVer() >= (uint32)EPipelineCacheFileFormatVersions::AddRTPSOShaderBindingLayout)
			{
				// Serialize RHIShaderBindingLayout
				Ar << Info.RayTracingDesc.ShaderBindingLayout;		
			}
			
			break;
		}
		default:
		{
			checkNoEntry();
		}
	}
	return Ar;
}

FPipelineCacheFileFormatPSO::FPipelineCacheFileFormatPSO()
#if PSO_COOKONLY_DATA
: UsageMask(0)
, BindCount(0)
#endif
{
}

/*static*/ bool FPipelineCacheFileFormatPSO::Init(FPipelineCacheFileFormatPSO& PSO, FComputePipelineStateInitializer const& Init, EPSOPrecacheResult PSOPrecacheResult)
{
	check(Init.ComputeShader != nullptr);

	PSO.Type = DescriptorType::Compute;
	PSO.PSOPrecacheResult = PSOPrecacheResult;

#if PSO_COOKONLY_DATA
	PSO.UsageMask = 0;
	PSO.BindCount = 0;
#endif
	
	// Because of the cheat in the copy constructor - lets play this safe
	FMemory::Memset(&PSO.ComputeDesc, 0, sizeof(ComputeDescriptor));
	
	PSO.ComputeDesc.ComputeShader = Init.ComputeShader->GetHash();
	
	bool bOK = true;
	
#if !UE_BUILD_SHIPPING
	bOK = PSO.Verify();
#endif
	
	return bOK;
}

/*static*/ bool FPipelineCacheFileFormatPSO::Init(FPipelineCacheFileFormatPSO& PSO, FGraphicsPipelineStateInitializer const& Init, EPSOPrecacheResult PSOPrecacheResult)
{
	bool bOK = true;
	
	PSO.Type = DescriptorType::Graphics;
	PSO.PSOPrecacheResult = PSOPrecacheResult;

#if PSO_COOKONLY_DATA
	PSO.UsageMask = 0;
	PSO.BindCount = 0;
#endif
	
	// Because of the cheat in the copy constructor - lets play this safe
	FMemory::Memset(&PSO.GraphicsDesc, 0, sizeof(GraphicsDescriptor));

#if PLATFORM_SUPPORTS_MESH_SHADERS
	checkf(Init.BoundShaderState.GetVertexShader() || Init.BoundShaderState.GetMeshShader(), TEXT("A graphics pipeline must always have either a vertex or a mesh shader"));
	if (Init.BoundShaderState.GetVertexShader())
#else
	checkf(Init.BoundShaderState.GetVertexShader(), TEXT("A graphics pipeline must always have a vertex shader"));
#endif
	{
		check (Init.BoundShaderState.VertexDeclarationRHI);
		check (Init.BoundShaderState.VertexDeclarationRHI->IsValid());
		{
			bOK &= Init.BoundShaderState.VertexDeclarationRHI->GetInitializer(PSO.GraphicsDesc.VertexDescriptor);
			check(bOK);
		
			SortVertexElements(PSO.GraphicsDesc.VertexDescriptor);
		}

		PSO.GraphicsDesc.VertexShader = Init.BoundShaderState.VertexShaderRHI->GetHash();
	}

	if (Init.BoundShaderState.GetMeshShader())
	{
		PSO.GraphicsDesc.MeshShader = Init.BoundShaderState.GetMeshShader()->GetHash();
	}
	if (Init.BoundShaderState.GetAmplificationShader())
	{
		PSO.GraphicsDesc.AmplificationShader = Init.BoundShaderState.GetAmplificationShader()->GetHash();
	}
	
	if (Init.BoundShaderState.PixelShaderRHI)
	{
		PSO.GraphicsDesc.FragmentShader = Init.BoundShaderState.PixelShaderRHI->GetHash();
	}
	
	if (Init.BoundShaderState.GetGeometryShader())
	{
		PSO.GraphicsDesc.GeometryShader = Init.BoundShaderState.GetGeometryShader()->GetHash();
	}
	
	check (Init.BlendState);
	{
		bOK &= Init.BlendState->GetInitializer(PSO.GraphicsDesc.BlendState);
		check(bOK);
	}
	
	check (Init.RasterizerState);
	{
		FRasterizerStateInitializerRHI Temp;
		bOK &= Init.RasterizerState->GetInitializer(Temp);
		check(bOK);
		
		PSO.GraphicsDesc.RasterizerState = Temp;
	}
	
	check (Init.DepthStencilState);
	{
		bOK &= Init.DepthStencilState->GetInitializer(PSO.GraphicsDesc.DepthStencilState);
		check(bOK);
	}

	for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
	{
		PSO.GraphicsDesc.RenderTargetFormats[i] = (EPixelFormat)Init.RenderTargetFormats[i];
		PSO.GraphicsDesc.RenderTargetFlags[i] = FPipelineCacheFileFormatPSO::GraphicsDescriptor::ReduceRTFlags(Init.RenderTargetFlags[i]);
	}
	
	PSO.GraphicsDesc.RenderTargetsActive = Init.RenderTargetsEnabled;
	PSO.GraphicsDesc.MSAASamples = Init.NumSamples;
	
	PSO.GraphicsDesc.DepthStencilFormat = Init.DepthStencilTargetFormat;
	PSO.GraphicsDesc.DepthStencilFlags = FPipelineCacheFileFormatPSO::GraphicsDescriptor::ReduceDSFlags(Init.DepthStencilTargetFlag);
	PSO.GraphicsDesc.DepthLoad = Init.DepthTargetLoadAction;
	PSO.GraphicsDesc.StencilLoad = Init.StencilTargetLoadAction;
	PSO.GraphicsDesc.DepthStore = Init.DepthTargetStoreAction;
	PSO.GraphicsDesc.StencilStore = Init.StencilTargetStoreAction;
	
	PSO.GraphicsDesc.PrimitiveType = Init.PrimitiveType;

	PSO.GraphicsDesc.SubpassHint = (uint8)Init.SubpassHint;
	PSO.GraphicsDesc.SubpassIndex = Init.SubpassIndex;
	
	PSO.GraphicsDesc.MultiViewCount = (uint8)Init.MultiViewCount;
	PSO.GraphicsDesc.bHasFragmentDensityAttachment = Init.bHasFragmentDensityAttachment;

	PSO.GraphicsDesc.bAllowVariableRateShading = Init.bAllowVariableRateShading;
	PSO.GraphicsDesc.ShadingRate = Init.ShadingRate;

	PSO.GraphicsDesc.bDepthBounds = Init.bDepthBounds;

#if !UE_BUILD_SHIPPING
	bOK = bOK && PSO.Verify();
#endif
	
	return bOK;
}

void FPipelineCacheFileFormatPSO::SortVertexElements(FVertexDeclarationElementList& VertexElements)
{
	VertexElements.Sort([](FVertexElement const& A, FVertexElement const& B)
	{
		if (A.StreamIndex < B.StreamIndex)
		{
			return true;
		}
		if (A.StreamIndex > B.StreamIndex)
		{
			return false;
		}
		if (A.Offset < B.Offset)
		{
			return true;
		}
		if (A.Offset > B.Offset)
		{
			return false;
		}
		if (A.AttributeIndex < B.AttributeIndex)
		{
			return true;
		}
		if (A.AttributeIndex > B.AttributeIndex)
		{
			return false;
		}
		return false;
	});
}

FPipelineCacheFileFormatPSO::~FPipelineCacheFileFormatPSO()
{
	
}

bool FPipelineCacheFileFormatPSO::operator==(const FPipelineCacheFileFormatPSO& Other) const
{
	bool bSame = true;
	if (this != &Other)
	{
		bSame = Type == Other.Type;

		// ignore pso precache result for equals op

#if PSO_COOKONLY_DATA
		/* Ignore: [UsageMask == UsageMask] in this test. */
		/* Ignore: [BindCount == BindCount] in this test. */
#endif
		if(Type == Other.Type)
		{
			switch(Type)
			{
				case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
				{
					// If we implement a classic copy constructor without memcpy - remove memset in ::Init() function above
					bSame = (FMemory::Memcmp(&ComputeDesc, &Other.ComputeDesc, sizeof(ComputeDescriptor)) == 0);
					break;
				}
				case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
				{
					// If we implement a classic copy constructor without memcpy - remove memset in ::Init() function above
					
					bSame = GraphicsDesc.VertexDescriptor.Num() == Other.GraphicsDesc.VertexDescriptor.Num();
					for (uint32 i = 0; i < (uint32)FMath::Min(GraphicsDesc.VertexDescriptor.Num(), Other.GraphicsDesc.VertexDescriptor.Num()); i++)
					{
						bSame &= (FMemory::Memcmp(&GraphicsDesc.VertexDescriptor[i], &Other.GraphicsDesc.VertexDescriptor[i], sizeof(FVertexElement)) == 0);
					}
					bSame &=
						GraphicsDesc.PrimitiveType == Other.GraphicsDesc.PrimitiveType &&
						GraphicsDesc.VertexShader == Other.GraphicsDesc.VertexShader &&
						GraphicsDesc.FragmentShader == Other.GraphicsDesc.FragmentShader &&
						GraphicsDesc.GeometryShader == Other.GraphicsDesc.GeometryShader &&
						GraphicsDesc.MeshShader == Other.GraphicsDesc.MeshShader &&
						GraphicsDesc.AmplificationShader == Other.GraphicsDesc.AmplificationShader &&
						GraphicsDesc.RenderTargetsActive == Other.GraphicsDesc.RenderTargetsActive &&
						GraphicsDesc.MSAASamples == Other.GraphicsDesc.MSAASamples && GraphicsDesc.DepthStencilFormat == Other.GraphicsDesc.DepthStencilFormat &&
						GraphicsDesc.DepthStencilFlags == Other.GraphicsDesc.DepthStencilFlags && GraphicsDesc.DepthLoad == Other.GraphicsDesc.DepthLoad &&
						GraphicsDesc.DepthStore == Other.GraphicsDesc.DepthStore && GraphicsDesc.StencilLoad == Other.GraphicsDesc.StencilLoad && GraphicsDesc.StencilStore == Other.GraphicsDesc.StencilStore &&
						GraphicsDesc.SubpassHint == Other.GraphicsDesc.SubpassHint && GraphicsDesc.SubpassIndex == Other.GraphicsDesc.SubpassIndex &&
						GraphicsDesc.MultiViewCount == Other.GraphicsDesc.MultiViewCount && GraphicsDesc.bHasFragmentDensityAttachment == Other.GraphicsDesc.bHasFragmentDensityAttachment &&
						GraphicsDesc.bAllowVariableRateShading == Other.GraphicsDesc.bAllowVariableRateShading && GraphicsDesc.ShadingRate == Other.GraphicsDesc.ShadingRate &&
						GraphicsDesc.bDepthBounds == Other.GraphicsDesc.bDepthBounds &&
					FMemory::Memcmp(&GraphicsDesc.BlendState, &Other.GraphicsDesc.BlendState, sizeof(FBlendStateInitializerRHI)) == 0 &&
					FMemory::Memcmp(&GraphicsDesc.RasterizerState, &Other.GraphicsDesc.RasterizerState, sizeof(FPipelineFileCacheRasterizerState)) == 0 &&
					FMemory::Memcmp(&GraphicsDesc.DepthStencilState, &Other.GraphicsDesc.DepthStencilState, sizeof(FDepthStencilStateInitializerRHI)) == 0 &&
					FMemory::Memcmp(&GraphicsDesc.RenderTargetFormats, &Other.GraphicsDesc.RenderTargetFormats, sizeof(GraphicsDesc.RenderTargetFormats)) == 0 &&
					FMemory::Memcmp(&GraphicsDesc.RenderTargetFlags, &Other.GraphicsDesc.RenderTargetFlags, sizeof(GraphicsDesc.RenderTargetFlags)) == 0;
					break;
				}
				case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
				{
					bSame &= RayTracingDesc == Other.RayTracingDesc;
					break;
				}
				default:
				{
					check(false);
					break;
				}
			}
		}
	}
	return bSame;
}

FPipelineCacheFileFormatPSO::FPipelineCacheFileFormatPSO(const FPipelineCacheFileFormatPSO& Other)
: Type(Other.Type), PSOPrecacheResult(Other.PSOPrecacheResult)
#if PSO_COOKONLY_DATA
, UsageMask(Other.UsageMask)
, BindCount(Other.BindCount)
#endif
{
	switch(Type)
	{
		case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
		{
			// If we implement a classic copy constructor without memcpy - remove memset in ::Init() function above
			FMemory::Memcpy(&ComputeDesc, &Other.ComputeDesc, sizeof(ComputeDescriptor));
			break;
		}
		case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
		{
			// If we implement a classic copy constructor without memcpy - remove memset in ::Init() function above
			FMemory::Memcpy(&GraphicsDesc, &Other.GraphicsDesc, sizeof(GraphicsDescriptor));
			break;
		}
		case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
		{
			RayTracingDesc = Other.RayTracingDesc;
			break;
		}
		default:
		{
			check(false);
			break;
		}
	}
}

FPipelineCacheFileFormatPSO& FPipelineCacheFileFormatPSO::operator=(const FPipelineCacheFileFormatPSO& Other)
{
	if(this != &Other)
	{
		Type = Other.Type;
		PSOPrecacheResult = Other.PSOPrecacheResult;
#if PSO_COOKONLY_DATA
		UsageMask = Other.UsageMask;
		BindCount = Other.BindCount;
#endif
		switch(Type)
		{
			case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
			{
				// If we implement a classic copy constructor without memcpy - remove memset in ::Init() function above
				FMemory::Memcpy(&ComputeDesc, &Other.ComputeDesc, sizeof(ComputeDescriptor));
				break;
			}
			case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
			{
				// If we implement a classic copy constructor without memcpy - remove memset in ::Init() function above
				FMemory::Memcpy(&GraphicsDesc, &Other.GraphicsDesc, sizeof(GraphicsDescriptor));
				break;
			}
			case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
			{
				RayTracingDesc = Other.RayTracingDesc;
				break;
			}
			default:
			{
				check(false);
				break;
			}
		}
	}
	return *this;
}

struct FPipelineCacheFileFormatTOC
{
	FPipelineCacheFileFormatTOC()
	: SortedOrder(FPipelineFileCacheManager::PSOOrder::MostToLeastUsed)
	{}
	
	FPipelineFileCacheManager::PSOOrder SortedOrder;
	TMap<uint32, FPipelineCacheFileFormatPSOMetaData> MetaData;
	void DumpToLog() const
	{
		for (TMap<uint32, FPipelineCacheFileFormatPSOMetaData>::TConstIterator It(MetaData); It; ++It)
		{
			const FPipelineCacheFileFormatPSOMetaData& dat = It.Value();
			UE_LOG(LogRHI, VeryVerbose, TEXT("PSO hash %u - guid (%s), stats(FF %" INT64_FMT ", LF %" INT64_FMT ", bind %" INT64_FMT ")"), It.Key(), *dat.FileGuid.ToString(), dat.Stats.FirstFrameUsed, dat.Stats.LastFrameUsed, dat.Stats.TotalBindCount)
		}
		UE_LOGF(LogRHI, VeryVerbose, "Total PSOs %d", MetaData.Num());
	}

	friend FArchive& operator<<(FArchive& Ar, FPipelineCacheFileFormatTOC& Info)
	{
		// TOC is assumed to be at the end of the file
		// If this changes then the EOF read check and write need to moved out of here

		// if all entries are using the same GUID (which is the norm when saving a packaged cache with the "buildsc" command of the commandlet),
		// do not save it with every entry, reducing the surface of changes (GUID is regenerated on each save even if entries are the same)
		bool bAllEntriesUseSameGuid = true;
		FGuid FirstEntryGuid;

		if(Ar.IsLoading())
		{
			uint64 TOCMagic = 0;
			Ar << TOCMagic;
			if(FPipelineCacheTOCFileFormatMagic != TOCMagic)
			{
				Ar.SetError();
				return Ar;
			}
			
			uint64 EOFMagic = 0;
			const int64 FileSize = Ar.TotalSize();
			const int64 FilePosition = Ar.Tell();
			Ar.Seek(FileSize - sizeof(FPipelineCacheEOFFileFormatMagic));
			Ar << EOFMagic;
			Ar.Seek(FilePosition);

			if(FPipelineCacheEOFFileFormatMagic != EOFMagic)
			{
				Ar.SetError();
				return Ar;
			}
		}
		else
		{
			uint64 TOCMagic = FPipelineCacheTOCFileFormatMagic;
			Ar << TOCMagic;

			// check if the whole file is using the same GUID
			bool bGuidSet = false;
			for (TMap<uint32, FPipelineCacheFileFormatPSOMetaData>::TConstIterator It(Info.MetaData); It; ++It)
			{
				if (bGuidSet)
				{
					if (It.Value().FileGuid != FirstEntryGuid)
					{
						bAllEntriesUseSameGuid = false;
						break;
					}
				}
				else
				{
					FirstEntryGuid = It.Value().FileGuid;
					bGuidSet = true;
				}
			}

			if (!bGuidSet)
			{
				bAllEntriesUseSameGuid = false;	// no entries, so don't do save the guid at all
			}

			// if the whole file uses the same guids, zero out
			if (bAllEntriesUseSameGuid)
			{
				for (TMap<uint32, FPipelineCacheFileFormatPSOMetaData>::TIterator It(Info.MetaData); It; ++It)
				{
					It.Value().FileGuid = FGuid();
				}
			}
		}

		uint8 AllEntriesUseSameGuid = bAllEntriesUseSameGuid ? 1 : 0;
		Ar << AllEntriesUseSameGuid;
		bAllEntriesUseSameGuid = AllEntriesUseSameGuid != 0;

		if (bAllEntriesUseSameGuid)
		{
			Ar << FirstEntryGuid;
		}

		Ar << Info.SortedOrder;
		Ar << Info.MetaData;
		
		if(Ar.IsSaving())
		{
			uint64 EOFMagic = FPipelineCacheEOFFileFormatMagic;
			Ar << EOFMagic;
		}
		else if (bAllEntriesUseSameGuid)
		{
			for (TMap<uint32, FPipelineCacheFileFormatPSOMetaData>::TIterator It(Info.MetaData); It; ++It)
			{
				It.Value().FileGuid = FirstEntryGuid;
			}
		}
		
		return Ar;
	}
};

static bool ShouldDeleteExistingUserCache()
{
	static bool bOnce = false;
	static bool bCmdLineForce = false;
	if (!bOnce)
	{
		bOnce = true;
		bCmdLineForce = FParse::Param(FCommandLine::Get(), TEXT("deleteuserpsocache")) || FParse::Param(FCommandLine::Get(), TEXT("logPSO"));
		UE_CLOGF(bCmdLineForce, LogRHI, Warning, "****************************** Deleting user-writable PSO cache as requested on command line");
	}
	return bCmdLineForce;
}

class FPipelineCacheFile
{
	FString Name;
	EShaderPlatform ShaderPlatform;
	FName PlatformName;
	uint64 TOCOffset;

	FPipelineCacheFileFormatTOC TOC;
	FGuid FileGuid;
	FString FilePath;
	TSharedPtr<IAsyncReadFileHandle, ESPMode::ThreadSafe> AsyncFileHandle;

	FString RecordingFilename;
public:
	enum class EStatus : uint8
	{
		Unknown,
		BundledCache,
		UserCacheOpened,	// a user cache was successfully opened
		NewUserCache,		// user cache failed to open, started empty.
	};
	EStatus CacheStatus = EStatus::Unknown;

	static uint32 GameVersion;

	FPipelineCacheFile()
	: TOCOffset(0)
	, AsyncFileHandle(nullptr)
	{
	}
	~FPipelineCacheFile()
	{
		DEC_MEMORY_STAT_BY(STAT_FileCacheMemory, TOC.MetaData.GetAllocatedSize());
	}

	static bool OpenPipelineFileCache(const FString& FilePath, EShaderPlatform ShaderPlatform, FGuid& Guid, TSharedPtr<IAsyncReadFileHandle, ESPMode::ThreadSafe>& Handle, FPipelineCacheFileFormatTOC& Content, uint64& TOCOffsetOUT)
	{
		bool bSuccess = false;

		FArchive* FileReader = IFileManager::Get().CreateFileReader(*FilePath);
		if (FileReader)
		{
			FileReader->SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);
			FPipelineCacheFileFormatHeader Header = {};
			*FileReader << Header;
			if (Header.Magic == FPipelineCacheFileFormatMagic && Header.Version == FPipelineCacheFileFormatCurrentVersion && Header.GameVersion == GameVersion && Header.Platform == ShaderPlatform)
			{
				check(Header.TableOffset > 0);
				check(FileReader->TotalSize() > 0);
				
				UE_LOGF(LogRHI, Log, "FPipelineCacheFile Header Game Version: %d", Header.GameVersion);
				UE_LOGF(LogRHI, Log, "FPipelineCacheFile Header Engine Data Version: %d", Header.Version);
				UE_LOGF(LogRHI, Log, "FPipelineCacheFile Header TOC Offset: %llu", Header.TableOffset);
				UE_LOGF(LogRHI, Log, "FPipelineCacheFile File Size: %lld Bytes", FileReader->TotalSize());
				
				if(Header.TableOffset < (uint64)FileReader->TotalSize())
				{
					FileReader->Seek(Header.TableOffset);
					*FileReader << Content;
					
					// FPipelineCacheFileFormatTOC archive read can set the FArchive to error on failure
					bSuccess = !FileReader->IsError();
				}
				
				if(!bSuccess)
				{
					UE_LOGF(LogRHI, Log, "FPipelineCacheFile: %ls is corrupt reading TOC", *FilePath);
				}
			}
			else
			{
				bool bMagicMatch = (Header.Magic == FPipelineCacheFileFormatMagic);
				bool bVersionMatch = (Header.Version == FPipelineCacheFileFormatCurrentVersion);
				bool bGameVersionMatch = (Header.GameVersion == GameVersion);
				bool bSPMatch = (Header.Platform == ShaderPlatform);
				UE_LOGF(LogRHI, Log, "FPipelineCacheFile: skipping %ls (different %ls%ls%ls%ls)", *FilePath,
					bMagicMatch ? TEXT("") : TEXT(" magic"),
					bVersionMatch ? TEXT("") : TEXT(" version"),
					bGameVersionMatch ? TEXT("") : TEXT(" gameversion"),
					bSPMatch ? TEXT("") : TEXT(" shaderplatform")
					);
			}
			
			if(!FileReader->Close())
			{
				bSuccess = false;
			}
			
			delete FileReader;
			FileReader = nullptr;
			
			if(bSuccess)
			{
				Handle = MakeShareable(FPlatformFileManager::Get().GetPlatformFile().OpenAsyncRead(*FilePath));
				if(Handle.IsValid())
				{
					UE_LOGF(LogRHI, Log, "Opened FPipelineCacheFile: %ls (GUID: %ls) with %d entries.", *FilePath, *Header.Guid.ToString(), Content.MetaData.Num());
					
					Guid = Header.Guid;
					TOCOffsetOUT = Header.TableOffset;
				}
				else
				{
					UE_LOGF(LogRHI, Log, "Failed to create async read file handle to FPipelineCacheFile: %ls (GUID: %ls)", *FilePath, *Header.Guid.ToString());
					bSuccess = false;
				}
			}
		}
		else
		{
			UE_LOGF(LogRHI, Log, "Could not open FPipelineCacheFile: %ls", *FilePath);
		}
		
		return bSuccess;
	}

	void GarbageCollectUserCache(FString const& UserCacheFilePath, const TSet<FGuid>& KnownGuids)
	{
		UE_LOGF(LogRHI, Log, "FPipelineCacheFile: GarbageCollectUserCache() Begin");
		ON_SCOPE_EXIT{ UE_LOGF(LogRHI, Log, "FPipelineCacheFile: GarbageCollectUserCache() End"); };

		int32 GCPeriodInDays = CVarPSOFileCacheUserCacheUnusedElementCheckPeriod.GetValueOnAnyThread();
		if (GCPeriodInDays < 0)
		{
			UE_LOGF(LogRHI, Log, "User cache GC is disabled");
			return;
		}

		FArchive* FileReader = IFileManager::Get().CreateFileReader(*UserCacheFilePath);
		if (!FileReader)
		{
			UE_LOGF(LogRHI, Log, "No user cache file found");
			return;
		}

		ON_SCOPE_EXIT
		{
			if (FileReader)
			{
				FileReader->Close();
				delete FileReader;
			}
		};

		FileReader->SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);
		FPipelineCacheFileFormatHeader Header;
		*FileReader << Header;

		if (!(Header.Magic == FPipelineCacheFileFormatMagic && Header.Version == FPipelineCacheFileFormatCurrentVersion && Header.GameVersion == GameVersion && Header.Platform == ShaderPlatform))
		{
			UE_LOGF(LogRHI, Error, "File has invalid or out of date header");
			return;
		}

		FTimespan GCPeriod = FTimespan::FromDays(GCPeriodInDays);
		int64 NextGCTime = Header.LastGCUnixTime + GCPeriod.GetTotalSeconds();
		const int64 UnixTime = GetCurrentUnixTime();
		if (UnixTime < NextGCTime)
		{
			const FTimespan TimespanToNextGC = FTimespan::FromSeconds(NextGCTime - UnixTime);
			const double DaysToNextGC = TimespanToNextGC.GetTotalDays();
			UE_LOGF(LogRHI, Log, "Next GC on user cache is in %0.3f days.", DaysToNextGC);
			return;
		}

		FPipelineCacheFileFormatTOC Content;
		if (Header.TableOffset < (uint64)FileReader->TotalSize())
		{
			FileReader->Seek(Header.TableOffset);
			*FileReader << Content;

			// FPipelineCacheFileFormatTOC archive read can set the FArchive to error on failure
			if (FileReader->IsError())
			{
				UE_LOGF(LogRHI, Log, "Failed to read TOC");
				return;
			}
		}

		int64 StaleDays = CVarPSOFileCacheUserCacheUnusedElementRetainDays.GetValueOnAnyThread();
		FTimespan StaleTimespan = FTimespan::FromDays(StaleDays);
		int64 EvictionTime = UnixTime - (int64)StaleTimespan.GetTotalSeconds();

		auto EntryShouldBeRemovedFromUserCache = [&Header, FileGuid=this->FileGuid, EvictionTime,&KnownGuids](const FPipelineCacheFileFormatPSOMetaData& MetaData)
		{
			// Remove the element if it is in the user cache and the time has elapsed, or if it was in a cache that no longer exists.
			if (MetaData.FileGuid == Header.Guid)
			{
				return EvictionTime >= MetaData.LastUsedUnixTime;
			}
			else
			{
			// TODO: right now we do not have a way to supply known guids.
			// we use only the expired date check for now.
			//	return !KnownGuids.Contains(MetaData.FileGuid);
				return EvictionTime >= MetaData.LastUsedUnixTime;
			}
		};

		int32 NumOutOfDateEntries = 0;
		for (auto const& Entry : Content.MetaData)
		{
			if (EntryShouldBeRemovedFromUserCache(Entry.Value))
			{
				NumOutOfDateEntries++;
			}
		}

		if (NumOutOfDateEntries == 0)
		{
			UE_LOGF(LogRHI, Log, "No out of date entries.");
			return;
		}

		if (NumOutOfDateEntries == Content.MetaData.Num())
		{
			FileReader->Close();
			delete FileReader;
			FileReader = nullptr;
			if (IFileManager::Get().FileExists(*UserCacheFilePath))
			{
				IFileManager::Get().Delete(*UserCacheFilePath);
			}
			UE_LOGF(LogRHI, Log, "All entries are out of date, recreating cache.");
			return;
		}

		UE_LOGF(LogRHI, Log, "%d/%d elements are out of date, performing GC.", NumOutOfDateEntries, Content.MetaData.Num());
		
		TArray<uint8> Buffer;
		FMemoryWriter MemoryWriter(Buffer);
		MemoryWriter.SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);

		FPipelineCacheFileFormatHeader NewHeader = Header;
		check(NewHeader.Magic == FPipelineCacheFileFormatMagic);
		check(NewHeader.Platform == ShaderPlatform);
		NewHeader.Version = FPipelineCacheFileFormatCurrentVersion;
		NewHeader.LastGCUnixTime = UnixTime;
		NewHeader.TableOffset = 0; // Will overwrite with the correct offset after building the TOC
		MemoryWriter << NewHeader;

		FPipelineCacheFileFormatTOC NewTOC;
		NewTOC.SortedOrder = Content.SortedOrder; // Removal maintains sort order of existing cache

		for (auto const& Entry : Content.MetaData)
		{
			if (!EntryShouldBeRemovedFromUserCache(Entry.Value))
			{
				// Copy the meta data, and the FPipelineCacheFileFormatPSO if it exists in the user cache (the meta data can point into the game cache).
				FPipelineCacheFileFormatPSOMetaData NewEntry = Entry.Value;
				if (Entry.Value.FileGuid == Header.Guid && Entry.Value.FileSize > 0)
				{
					NewEntry.FileSize = Entry.Value.FileSize;
					NewEntry.FileOffset = MemoryWriter.Tell();

					// Copy from file to new memory writer
					FPipelineCacheFileFormatPSO ExistingPSO;

					FileReader->Seek(Entry.Value.FileOffset);
					*FileReader << ExistingPSO;
					MemoryWriter << ExistingPSO;
				}
				check(NewEntry.FileGuid != FGuid());
				NewTOC.MetaData.Add(Entry.Key, NewEntry);
			}
		}

		NewHeader.TableOffset = MemoryWriter.Tell();
		MemoryWriter << NewTOC;
		MemoryWriter.Seek(0);
		MemoryWriter << NewHeader;

		UE_LOGF(LogRHI, Log, "Deleting existing cache file");

		int64 OriginalSize = FileReader->TotalSize();
		FileReader->Close();
		delete FileReader;
		FileReader = nullptr;
		if (IFileManager::Get().FileExists(*UserCacheFilePath))
		{
			IFileManager::Get().Delete(*UserCacheFilePath);
		}

		FArchive* FileWriter = IFileManager::Get().CreateFileWriter(*UserCacheFilePath);
		if (!FileWriter)
		{
			UE_LOGF(LogRHI, Log, "Unable to open new cache file for writing");
			return;
		}

		int64 NewSize = MemoryWriter.TotalSize();
		FileWriter->Serialize(Buffer.GetData(), MemoryWriter.TotalSize());
		FileWriter->Close();
		delete FileWriter;
		UE_LOGF(LogRHI, Log, "Rewrote cache file. Old Size %lld, new size %lld (%lld byte reduction)", OriginalSize, NewSize, OriginalSize - NewSize);
	}
	
	bool OpenPipelineFileCache(FString const& NameIn, EShaderPlatform Platform, FGuid& OutGameFileGuid)
	{
		check(CacheStatus == EStatus::Unknown);

		OutGameFileGuid = FGuid();
		TOC.SortedOrder = FPipelineFileCacheManager::PSOOrder::Default;
		TOC.MetaData.Empty();
		
		Name = NameIn;
		
		ShaderPlatform = Platform;
		PlatformName = LegacyShaderPlatformToShaderFormat(Platform);
		
		FString GamePath	= FPaths::ProjectContentDir() / TEXT("PipelineCaches") / ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()) / FString::Printf(TEXT("%s_%s.stable.upipelinecache"), *Name, *PlatformName.ToString());
		static bool bCommandLineNotStable = FParse::Param(FCommandLine::Get(), TEXT("nostablepipelinecache"));
		if (bCommandLineNotStable)
		{
			GamePath.Empty();
		}

		const bool bGameFileOk = OpenPipelineFileCache(GamePath, ShaderPlatform, FileGuid, AsyncFileHandle, TOC, TOCOffset);

		if (bGameFileOk)
		{
			FilePath = GamePath;
			OutGameFileGuid = FileGuid;
			CacheStatus = EStatus::BundledCache;
		}

		if (bGameFileOk && GRHISupportsLazyShaderCodeLoading && CVarLazyLoadShadersWhenPSOCacheIsPresent.GetValueOnAnyThread())
		{
			UE_LOGF(LogRHI, Log, "Lazy loading from the shader code library is enabled.");
			GRHILazyShaderCodeLoading = true;
		}

#if !UE_BUILD_SHIPPING
		uint32 InvalidEntryCount = 0;
#endif
		
        for (auto const& Entry : TOC.MetaData)
        {
            FPipelineStateStats* Stat = FPipelineFileCacheManager::Stats.FindRef(Entry.Key);
            if (!Stat)
            {
                Stat = new FPipelineStateStats;
                Stat->PSOHash = Entry.Key;
                Stat->TotalBindCount = -1;
                FPipelineFileCacheManager::Stats.Add(Entry.Key, Stat);
            }

			UE_CLOGF(!!FPipelineFileCacheManager::NewPSOUsage.Find(Entry.Key), LogRHI, Warning, "loaded PSOFC %ls contains entry (%u) previously marked as new ", *Name, (Entry.Key));
			
#if !UE_BUILD_SHIPPING
			if((Entry.Value.EngineFlags & FPipelineCacheFlagInvalidPSO) != 0)
			{
				++InvalidEntryCount;
			}
#endif
        }
		
#if !UE_BUILD_SHIPPING
		if(InvalidEntryCount > 0)
        {
        	UE_LOGF(LogRHI, Warning, "Found %d / %d PSO entries marked as invalid.", InvalidEntryCount, TOC.MetaData.Num());
        }
#endif
		
		INC_MEMORY_STAT_BY(STAT_FileCacheMemory, TOC.MetaData.GetAllocatedSize());

		UE_LOGF(LogRHI, VeryVerbose, "-- opened bundled %ls cache:", *NameIn);
		TOC.DumpToLog();
		UE_LOGF(LogRHI, VeryVerbose, "-- end of dump (%ls)", *NameIn);

		return bGameFileOk;
	}
	
	// 
	bool OpenUserPipelineFileCache(FString const& CacheNameIn, EShaderPlatform Platform)
	{
		bool bUserFileOk = false;

		if (CacheStatus == EStatus::Unknown)
		{
			SET_DWORD_STAT(STAT_TotalGraphicsPipelineStateCount, 0);
			SET_DWORD_STAT(STAT_TotalComputePipelineStateCount, 0);
			SET_DWORD_STAT(STAT_TotalRayTracingPipelineStateCount, 0);
			SET_DWORD_STAT(STAT_SerializedGraphicsPipelineStateCount, 0);
			SET_DWORD_STAT(STAT_SerializedComputePipelineStateCount, 0);
			SET_DWORD_STAT(STAT_NewGraphicsPipelineStateCount, 0);
			SET_DWORD_STAT(STAT_NewComputePipelineStateCount, 0);
			SET_DWORD_STAT(STAT_NewRayTracingPipelineStateCount, 0);

			// one time attempt to open the user cache.
			TOC.SortedOrder = FPipelineFileCacheManager::PSOOrder::Default;
			TOC.MetaData.Empty();

			Name = CacheNameIn;

			ShaderPlatform = Platform;
			PlatformName = LegacyShaderPlatformToShaderFormat(Platform);


			FGuid UniqueFileGuid;
			FPlatformMisc::CreateGuid(UniqueFileGuid);  // not very unique on android, but won't matter much here

			RecordingFilename = FString::Printf(TEXT("%s-CL-%u-"), *FEngineVersion::Current().GetBranchDescriptor(), FEngineVersion::Current().GetChangelist());
			RecordingFilename += FString::Printf(TEXT("%s_%s_%s.rec.upipelinecache"), *Name, *PlatformName.ToString(), *UniqueFileGuid.ToString());
			RecordingFilename = FPaths::ProjectSavedDir() / TEXT("CollectedPSOs") / RecordingFilename;

			UE_LOGF(LogRHI, Log, "Base name for record PSOs is %ls", *RecordingFilename);
			FilePath = FPaths::ProjectSavedDir() / FString::Printf(TEXT("%s_%s.upipelinecache"), *Name, *PlatformName.ToString());

			if (ShouldDeleteExistingUserCache())
			{
				UE_LOGF(LogRHI, Log, "Deleting FPipelineCacheFile: %ls", *FilePath);
				if (IFileManager::Get().FileExists(*FilePath))
				{
					IFileManager::Get().Delete(*FilePath);
				}
			}

			FString JournalPath = FilePath + JOURNAL_FILE_EXTENSION;
			bool const bJournalFileExists = IFileManager::Get().FileExists(*JournalPath);
			if (bJournalFileExists || ShouldDeleteExistingUserCache())
			{
				UE_LOGF(LogRHI, Log, "Deleting FPipelineCacheFile: %ls", *FilePath);
				// If either of the above are true we need to dispose of this case as we consider it invalid
				if (IFileManager::Get().FileExists(*FilePath))
				{
					IFileManager::Get().Delete(*FilePath);
				}
				if (bJournalFileExists)
				{
					IFileManager::Get().Delete(*JournalPath);
				}
			}

			// TODO: we currently do not know the full set of valid PSOFC guids.
			// KnownGuids is a placeholder for all possible PSO cache guids.
			TSet<FGuid> KnownGuids;
			GarbageCollectUserCache(FilePath, KnownGuids);

			FPipelineCacheFileFormatTOC UserTOC;
			bUserFileOk = OpenPipelineFileCache(FilePath, ShaderPlatform, FileGuid, AsyncFileHandle, TOC, TOCOffset);
			CacheStatus = bUserFileOk ? EStatus::UserCacheOpened : EStatus::NewUserCache;

			if (!bUserFileOk)
			{
				FileGuid = FGuid::NewGuid();
				// Start the file again!
				IFileManager::Get().Delete(*FilePath);
				TOCOffset = 0;
			}

#if !UE_BUILD_SHIPPING
			uint32 InvalidEntryCount = 0;
#endif

			for (auto const& Entry : TOC.MetaData)
			{
				FPipelineStateStats* Stat = FPipelineFileCacheManager::Stats.FindRef(Entry.Key);
				if (!Stat)
				{
					Stat = new FPipelineStateStats;
					Stat->PSOHash = Entry.Key;
					Stat->TotalBindCount = -1;
					FPipelineFileCacheManager::Stats.Add(Entry.Key, Stat);
				}
#if !UE_BUILD_SHIPPING
				if ((Entry.Value.EngineFlags & FPipelineCacheFlagInvalidPSO) != 0)
				{
					++InvalidEntryCount;
				}
#endif
			}

#if !UE_BUILD_SHIPPING
			if (InvalidEntryCount > 0)
			{
				UE_LOGF(LogRHI, Warning, "Found %d / %d PSO entries marked as invalid.", InvalidEntryCount, TOC.MetaData.Num());
			}
#endif
		}
		INC_MEMORY_STAT_BY(STAT_FileCacheMemory, TOC.MetaData.GetAllocatedSize());
		return bUserFileOk;
	}

	static void MergePSOUsageToMetaData(TMap<uint32, FPSOUsageData>& NewPSOUsage, TMap<uint32, FPipelineCacheFileFormatPSOMetaData>& MetaData, int64 CurrentUnixTime, bool bRemoveUpdatedentries = false)
	{
		for(auto It = NewPSOUsage.CreateIterator(); It; ++It)
		{
			auto& MaskEntry = *It;
			
			//Don't use FindChecked as if new PSO was not bound - it might not be in the TOC.MetaData - they are not always added in every save mode - this is not an error
			auto* PSOMetaData = MetaData.Find(MaskEntry.Key);
			if(PSOMetaData != nullptr)
			{
				PSOMetaData->UsageMask |= MaskEntry.Value.UsageMask;
				PSOMetaData->EngineFlags |= MaskEntry.Value.EngineFlags;
				PSOMetaData->LastUsedUnixTime = CurrentUnixTime;
				
				if(bRemoveUpdatedentries)
				{
					It.RemoveCurrent();
				}
			}
		}
	}
	
	bool SavePipelineFileCache(FPipelineFileCacheManager::SaveMode Mode, TMap<uint32, FPipelineStateStats*> const& Stats, TMap<uint32, FPipelineCacheFileFormatPSO>& NewEntries, FPipelineFileCacheManager::PSOOrder Order, TMap<uint32, FPSOUsageData>& NewPSOUsage)
	{
		check(CacheStatus == EStatus::NewUserCache || CacheStatus == EStatus::UserCacheOpened);
		// remove from stats because this operation will modify the content and re-set the stat at the end.
		DEC_MEMORY_STAT_BY(STAT_FileCacheMemory, TOC.MetaData.GetAllocatedSize());
		QUICK_SCOPE_CYCLE_COUNTER(STAT_SavePipelineFileCache);
		double StartTime = FPlatformTime::Seconds();
		FString SaveFilePath = FilePath;
		
		if (FPipelineFileCacheManager::SaveMode::BoundPSOsOnly == Mode)
		{
			SaveFilePath = GetRecordingFilename();
		}
		
		bool bFileWriteSuccess = false;
		bool bPerformWrite = true;
		if (FPipelineFileCacheManager::SaveMode::Incremental == Mode)
		{
			bPerformWrite = NewEntries.Num() || Order != TOC.SortedOrder || NewPSOUsage.Num();
			bFileWriteSuccess = !bPerformWrite;
		}
		
		if (bPerformWrite)
		{
            uint32 NumNewEntries = 0;
            
			int64 UnixTime = GetCurrentUnixTime();
			FString JournalPath;
			if (Mode != FPipelineFileCacheManager::SaveMode::BoundPSOsOnly)
			{
				JournalPath = SaveFilePath + JOURNAL_FILE_EXTENSION;
				FArchive* JournalWriter = IFileManager::Get().CreateFileWriter(*JournalPath);
				check(JournalWriter);

				// Header
				{
					FPipelineCacheFileFormatHeader Header;

					Header.Magic = FPipelineCacheFileFormatMagic;
					Header.Version = FPipelineCacheFileFormatCurrentVersion;
					Header.GameVersion = GameVersion;
					Header.Platform = ShaderPlatform;
					Header.Guid = FileGuid;
					Header.TableOffset = 0;
					Header.LastGCUnixTime = UnixTime;

					*JournalWriter << Header;
				}

				check(!JournalWriter->IsError());
				JournalWriter->Close();
				delete JournalWriter;
				bPerformWrite = IFileManager::Get().FileExists(*JournalPath);
			}
			if (bPerformWrite)
			{
				struct FMemoryReaderAndMemory 
				{
					// Non-copyable
					FMemoryReaderAndMemory(const FMemoryReaderAndMemory&) = delete;
					FMemoryReaderAndMemory& operator=(const FMemoryReaderAndMemory&) = delete;

					TArray<uint8> Bytes;
					TUniquePtr<FMemoryReader> Reader;
					explicit FMemoryReaderAndMemory(FPipelineCacheFile* PipelineFile)
					{
						if(PipelineFile)
						{
							int64 FileSize = IFileManager::Get().FileSize(*PipelineFile->FilePath);
							if (FileSize > 0)
							{
								Bytes.SetNumUninitialized(FileSize);
								if (PipelineFile->AsyncFileHandle.IsValid())
								{
									IAsyncReadRequest* Request = PipelineFile->AsyncFileHandle->ReadRequest(0, FileSize, AIOP_Normal, nullptr, Bytes.GetData());
									Request->WaitCompletion();
									delete Request;
									UE_LOG(LogRHI, VeryVerbose, TEXT("1 Opening %s as guid %s, size %" INT64_FMT), *PipelineFile->FilePath, *PipelineFile->GetFileGuid().ToString(), FileSize);
								}
								else
								{
									bool bReadOK = FFileHelper::LoadFileToArray(Bytes, *PipelineFile->FilePath);
									UE_LOG(LogRHI, VeryVerbose, TEXT("2 Opening %s as guid %s, size %" INT64_FMT), *PipelineFile->FilePath, *PipelineFile->GetFileGuid().ToString(), FileSize);
									UE_CLOGF(!bReadOK, LogRHI, Warning, "Failed to read %lld bytes from %ls while re-saving the PipelineFileCache!", FileSize, *PipelineFile->FilePath);
								}
								Reader = MakeUnique<FMemoryReader>(Bytes);
								Reader->SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);
							}
						}
					}
					FMemoryReader* GetReader() { return Reader.Get(); }
				};

				TMap<FGuid, TUniquePtr<FMemoryReaderAndMemory>> GuidToFileCacheReader;

				auto GetFileCacheReaderFromGuid = [&GuidToFileCacheReader](const FGuid& guid)
				{
					check(guid != FGuid());
					if(!GuidToFileCacheReader.Contains(guid))
					{
						const FString& FoundKey = FPipelineFileCacheManager::GameGuidToCacheKey.FindRef(guid);

						FPipelineCacheFile* FoundPipelineFile = FPipelineFileCacheManager::GetPipelineCacheFileFromKey(FoundKey);

						GuidToFileCacheReader.Add(guid, MakeUnique<FMemoryReaderAndMemory>(FoundPipelineFile));
					}

					if (TUniquePtr<FMemoryReaderAndMemory>* Found = GuidToFileCacheReader.Find(guid))
					{
						return  (*Found)->GetReader();
					}
					checkNoEntry();
					return static_cast<FMemoryReader*>(nullptr);
				};

				// Assume caller has handled Platform specifc path + filename
				TArray<uint8> SaveBytes;
				FArchive* FileWriter;
				bool bUseMemoryWriter = (Mode == FPipelineFileCacheManager::SaveMode::BoundPSOsOnly);
				FString TempPath = SaveFilePath;
				// Only use a file switcheroo on Apple platforms as they are the only ones tested so far.
				// At least two other platforms MoveFile implementation looks broken when moving from a writable source file to a writeable destination.
				// They only handle moves/renames between the read-only -> writeable directories/devices.
				if ((PLATFORM_APPLE || PLATFORM_ANDROID) && Mode != FPipelineFileCacheManager::SaveMode::Incremental)
				{
					TempPath += TEXT(".tmp");
				}
				if (bUseMemoryWriter)
				{
					FileWriter = new FMemoryWriter(SaveBytes, true, false, FName(*SaveFilePath));
				}
				else
				{
					// parent directory creation is necessary because the deploy process from
					// AndroidPlatform.Automation.cs destroys the parent directories and recreates them
					IFileManager::Get().MakeDirectory(*FPaths::GetPath(TempPath), true);
					FileWriter = IFileManager::Get().CreateFileWriter(*TempPath, FILEWRITE_Append);
				}
				if (FileWriter)
				{
                    FileWriter->SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);
					FileWriter->Seek(0);
		 
					// Header
					FPipelineCacheFileFormatHeader Header;
					{
						Header.Magic = FPipelineCacheFileFormatMagic;
						Header.Version = FPipelineCacheFileFormatCurrentVersion;
						Header.GameVersion = GameVersion;
						Header.Platform = ShaderPlatform;
						Header.Guid = FileGuid;
						Header.TableOffset = 0;
						Header.LastGCUnixTime = UnixTime;
		 
						*FileWriter << Header;
		 
						TOCOffset = FMath::Max(TOCOffset, (uint64)FileWriter->Tell());
					}
		 
					uint32 TotalEntries = 0;
                    uint32 ConsolidatedEntries = 0;
                    uint32 RemovedEntries = 0;
                    switch (Mode)
                    {
						// This mode just writes new, used, entries to the end of the file and updates the TOC which will contain entries from the Game-Content file that are redundant.
						// Any current tasks are unaffected as the prior offsets are still valid.
                        case FPipelineFileCacheManager::SaveMode::Incremental:
                        {
                            // PSO Descriptors
                            uint64 PSOOffset = TOCOffset;
                            
                            FileWriter->Seek(PSOOffset);
                            
                            // Add new entries
							for(auto It = NewEntries.CreateIterator(); It; ++It)
                            {
								FPipelineCacheFileFormatPSO& NewEntry = It->Value;

								uint32 PSOHash = It->Key;
								
								bool bFound = FPipelineFileCacheManager::IsPSOEntryCached(NewEntry, nullptr);
								if (bFound)
								{
									// this could happen if another PSOFC loads after the PSO was encountered, if desired we could remove things from newentries when a psofc is mounted..
									UE_LOGF(LogRHI, Display, "Incrementally saving new PSOs but entry (%u), is already cached..", PSOHash);
									// Not removing it as the cached item could be legit if co-owner is not always loaded.
								}
								FPipelineStateStats const* Stat = Stats.FindRef(PSOHash);
								if (Stat && Stat->TotalBindCount > 0)
								{
									FPipelineCacheFileFormatPSOMetaData Meta;
									Meta.Stats.PSOHash = PSOHash;
									Meta.FileOffset = PSOOffset;
									Meta.FileGuid = FileGuid;
									Meta.AddShaders(NewEntry);

									TArray<uint8> Bytes;
									FMemoryWriter Wr(Bytes);
									Wr.SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);
									Wr << NewEntry;

									FileWriter->Serialize(Bytes.GetData(), Wr.TotalSize());

									Meta.FileSize = Wr.TotalSize();

									check(Meta.FileGuid != FGuid());
									TOC.MetaData.Add(PSOHash, Meta);
									PSOOffset += Meta.FileSize;

									check(PSOOffset == FileWriter->Tell());

									NumNewEntries++;

									It.RemoveCurrent();
									UE_LOGF(LogRHI, VeryVerbose, "Incremental save is appending new PSOs (%u)", PSOHash);
								}
                            }
							// We're appending to the current user cache here, Our TOC is the total.
							TotalEntries = TOC.MetaData.Num();

                            if(Order != FPipelineFileCacheManager::PSOOrder::Default)
                            {
                                SortMetaData(TOC.MetaData, Order);
                                TOC.SortedOrder = Order;
                            }
                            else
                            {
                                // Added new entries and not re-sorted - the sort order invalid - reset to default
                                TOC.SortedOrder = FPipelineFileCacheManager::PSOOrder::Default;
                            }
							
							// Update TOC Metadata usage and clear relevant entries in NewPSOUsage as we are saving this file cache TOC
							MergePSOUsageToMetaData(NewPSOUsage, TOC.MetaData, UnixTime, true);
							
                            Header.TableOffset = PSOOffset;
                            TOCOffset = PSOOffset;
							
                            FileWriter->Seek(Header.TableOffset);
							// use a temp here because serializing can destroy our metadata guids.
							FPipelineCacheFileFormatTOC TempTOC = TOC;
                            *FileWriter << TempTOC;
                            break;
                        }
						// This mode actually saves to a separate file that records only PSOs that were bound.
						// BoundPSOsOnly will record all those PSOs used in this run of the game.
                        case FPipelineFileCacheManager::SaveMode::BoundPSOsOnly:
                        {
                            FPipelineCacheFileFormatTOC TempTOC;
							// Merge all of the existing PSO caches together, including this (user cache)
							for (TPair<FString, TUniquePtr<class FPipelineCacheFile>>& PipelineCachePair : FPipelineFileCacheManager::FileCacheMap)
							{
								TempTOC.MetaData.Append(PipelineCachePair.Value->TOC.MetaData);
							}

                            TMap<uint32, FPipelineCacheFileFormatPSO> PSOs;
							
							Header.Guid = FGuid::NewGuid();
							
                            for (auto& Entry : NewEntries)
                            {
                                FPipelineCacheFileFormatPSOMetaData Meta;
                                Meta.Stats.PSOHash = Entry.Key;
                                Meta.FileOffset = 0;
                                Meta.FileSize = 0;
                                Meta.FileGuid = Header.Guid;
								Meta.AddShaders(Entry.Value);
								check(Meta.FileGuid != FGuid());

                                TempTOC.MetaData.Add(Meta.Stats.PSOHash, Meta);
                                PSOs.Add(Entry.Key, Entry.Value);
                            }
							
							// Update TOC Metadata usage masks - don't clear NewPSOUsage as we are using a TempTOC
							MergePSOUsageToMetaData(NewPSOUsage, TempTOC.MetaData, UnixTime);
                            
                            for (auto& Pair : Stats)
                            {
                                auto* MetaPtr = TempTOC.MetaData.Find(Pair.Key);
                                if (MetaPtr)
                                {
                                    auto& Meta = *MetaPtr;
                                    check(Meta.Stats.PSOHash == Pair.Value->PSOHash);
                                    Meta.Stats.CreateCount += Pair.Value->CreateCount;
                                    if (Pair.Value->FirstFrameUsed > Meta.Stats.FirstFrameUsed)
                                    {
                                        Meta.Stats.FirstFrameUsed = Pair.Value->FirstFrameUsed;
                                    }
                                    if (Pair.Value->LastFrameUsed > Meta.Stats.LastFrameUsed)
                                    {
                                        Meta.Stats.LastFrameUsed = Pair.Value->LastFrameUsed;
                                    }
									Meta.Stats.TotalBindCount = (int64)FMath::Min((uint64)INT64_MAX, (uint64)FMath::Max(Meta.Stats.TotalBindCount, 0ll) + (uint64)FMath::Max(Pair.Value->TotalBindCount, 0ll));
                                }
                            }
                            
                            for (auto It = TempTOC.MetaData.CreateIterator(); It; ++It)
                            {
                                FPipelineStateStats const* Stat = Stats.FindRef(It->Key);
								
								bool bUsed = (Stat && (Stat->TotalBindCount > 0));
								if (bUsed)
                                {
                                    if (!PSOs.Contains(It->Key))
                                    {
                                        check(It->Value.FileSize > 0);
										FMemoryReader* Reader = GetFileCacheReaderFromGuid(It->Value.FileGuid);
										if (Reader)
										{
											UE_LOG(LogRHI, VeryVerbose, TEXT("reading PSO (%u) from guid %s, Off %" UINT64_FMT " // %" INT64_FMT), It->Key, *It->Value.FileGuid.ToString(), (It->Value.FileOffset), Reader->TotalSize());
											check(It->Value.FileOffset < (uint32)Reader->TotalSize());
											Reader->Seek(It->Value.FileOffset);

											FPipelineCacheFileFormatPSO PSO;
											(*Reader) << PSO;

											PSOs.Add(It->Key, PSO);
										}
										else
										{
											FString GameGuids;
											Algo::ForEach(FPipelineFileCacheManager::FileCacheMap, [&GameGuids](auto& MapPair) { GameGuids += FString::Printf(TEXT("[%s - %s]"), *MapPair.Value->Name, *MapPair.Value->FileGuid.ToString()); });
											UE_LOG(LogRHI, Display, \
												TEXT("Trying to reconcile from unknown file GUID: %s but bound log file is: %s user file is: %s and the currently known game files are: %s - this means you have stale entries in a local cache file or the relevant game content file is yet to be mounted."), \
												*(It->Value.FileGuid.ToString()), *(Header.Guid.ToString()), *(FileGuid.ToString()), *(GameGuids));

											RemovedEntries++;
											It.RemoveCurrent();
										}
                                    }
                                }
                                else
                                {
                                    RemovedEntries++;
                                    It.RemoveCurrent();
                                }
                            }
							TotalEntries = TempTOC.MetaData.Num();
                            
                            SortMetaData(TempTOC.MetaData, Order);
                            TempTOC.SortedOrder = Order;
                            
                            uint64 TempTOCOffset = (uint64)FileWriter->Tell();
                            
                            uint64 PSOOffset = TempTOCOffset;
                            
                            for (auto& Entry : TempTOC.MetaData)
                            {
                                FPipelineCacheFileFormatPSO& PSO = PSOs.FindChecked(Entry.Key);
                                
                                FileWriter->Seek(PSOOffset);
                                
                                Entry.Value.FileGuid = Header.Guid;
                                Entry.Value.FileOffset = PSOOffset;

								int64 At = FileWriter->Tell();
                                
                                (*FileWriter) << PSO;
                                
                                Entry.Value.FileSize = FileWriter->Tell() - At;
                                
                                PSOOffset += Entry.Value.FileSize;
                                check(PSOOffset == FileWriter->Tell());
								
								NumNewEntries++;
                            }
                            
                            Header.TableOffset = PSOOffset;
                            TempTOCOffset = PSOOffset;
                            
                            FileWriter->Seek(Header.TableOffset);
                            *FileWriter << TempTOC;
                            
                            break;
                        }
                        default:
                        {
                            check(false);
                            break;
                        }
                    }
		 
					// Overwrite the header now that we have the TOC location.
					FileWriter->Seek(0);
					*FileWriter << Header;
		 
					FileWriter->Flush();
		 
					bFileWriteSuccess = !FileWriter->IsError();
		 
					if(!FileWriter->Close())
					{
						bFileWriteSuccess = false;
					}
					if (bFileWriteSuccess && bUseMemoryWriter)
					{
						if (TotalEntries > 0)
						{
							bFileWriteSuccess = FFileHelper::SaveArrayToFile(SaveBytes, *TempPath);
						}
						else
						{
							delete FileWriter;
							float ThisTimeMS = float(FPlatformTime::Seconds() - StartTime) * 1000.0f;
							UE_LOGF(LogRHI, Log, "FPipelineFileCacheManager skipping saving empty .upipelinecache (took %6.2fms): %ls.", ThisTimeMS, *SaveFilePath);
							return false;
						}
					}
		 
					if (bFileWriteSuccess)
                    {
						delete FileWriter;
						
						// As on POSIX only file moves on the same device are atomic
						if ((SaveFilePath == TempPath) || IFileManager::Get().Move(*SaveFilePath, *TempPath, true, true, true, true))
						{
							float ThisTimeMS = float(FPlatformTime::Seconds() - StartTime) * 1000.0f;

							TCHAR const* ModeName = nullptr;
							switch (Mode)
							{
							default:
								checkNoEntry();
							case FPipelineFileCacheManager::SaveMode::Incremental:
								ModeName = TEXT("Incremental");
								break;
							case FPipelineFileCacheManager::SaveMode::BoundPSOsOnly:
								ModeName = TEXT("BoundPSOsOnly");
								break;
							}
							UE_LOGF(LogRHI, Log, "FPipelineFileCacheManager %ls saved %u total, %u new, %u removed, %u cons .upipelinecache (took %6.2fms): %ls.", ModeName, TotalEntries, NumNewEntries, RemovedEntries, ConsolidatedEntries, ThisTimeMS, *SaveFilePath);
							
							if (JournalPath.Len())
							{
								IFileManager::Get().Delete(*JournalPath);
							}
						}
						else
						{
							float ThisTimeMS = float(FPlatformTime::Seconds() - StartTime) * 1000.0f;
							UE_LOGF(LogRHI, Error, "Failed to move .upipelinecache from %ls to %ls (took %6.2fms).", *TempPath, *SaveFilePath, ThisTimeMS);
						}
					}
                    else
                    {
						delete FileWriter;
						IFileManager::Get().Delete(*TempPath);
						float ThisTimeMS = float(FPlatformTime::Seconds() - StartTime) * 1000.0f;
						UE_LOGF(LogRHI, Error, "Failed to write .upipelinecache, (took %6.2fms): %ls.", ThisTimeMS, *SaveFilePath);
                    }
		 
				}
                else
                {
                    UE_LOGF(LogRHI, Error, "Failed to open .upipelinecache for write: %ls.", *SaveFilePath);
                }
			}
		}
		
		INC_MEMORY_STAT_BY(STAT_FileCacheMemory, TOC.MetaData.GetAllocatedSize());
		
		return bFileWriteSuccess;
	}
	
	bool IsPSOEntryCachedInternal(FPipelineCacheFileFormatPSO const& NewEntry, FPSOUsageData* EntryData = nullptr) const
	{
		uint32 PSOHash = GetTypeHash(NewEntry);
		check(!EntryData || EntryData->PSOHash == PSOHash);
		FPipelineCacheFileFormatPSOMetaData const * const Existing = TOC.MetaData.Find(PSOHash);
		
		if(Existing != nullptr && EntryData != nullptr)
		{
			EntryData->UsageMask = Existing->UsageMask;
			EntryData->EngineFlags = Existing->EngineFlags;
		}
		
		return Existing != nullptr;
	}
	
	bool IsBSSEquivalentPSOEntryCachedInternal(FPipelineCacheFileFormatPSO const& NewEntry) const
	{
		check(!IsPSOEntryCachedInternal(NewEntry)); // this routine should only be called after we have done the much faster test
		bool bResult = false;
		if (NewEntry.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
		{
			// this is O(N) and potentially slow, measured timing is 10s of us.
			TSet<FShaderHash> TempShaders;
			TempShaders.Add(NewEntry.GraphicsDesc.VertexShader);
			if (NewEntry.GraphicsDesc.FragmentShader != FShaderHash())
			{
				TempShaders.Add(NewEntry.GraphicsDesc.FragmentShader);
			}
			if (NewEntry.GraphicsDesc.GeometryShader != FShaderHash())
			{
				TempShaders.Add(NewEntry.GraphicsDesc.GeometryShader);
			}
			if (NewEntry.GraphicsDesc.MeshShader != FShaderHash())
			{
				TempShaders.Add(NewEntry.GraphicsDesc.MeshShader);
			}
			if (NewEntry.GraphicsDesc.AmplificationShader != FShaderHash())
			{
				TempShaders.Add(NewEntry.GraphicsDesc.AmplificationShader);
			}

			for (auto const& Hash : TOC.MetaData)
			{
				if (LegacyCompareEqual(TempShaders, Hash.Value.Shaders))
				{
					bResult = true;
					break;
				}
			}
		}

		return bResult;
	}
	
	static void SortMetaData(TMap<uint32, FPipelineCacheFileFormatPSOMetaData>& MetaData, FPipelineFileCacheManager::PSOOrder Order)
	{
		// Only sorting metadata ordering - this should not affect PSO data offsets / lookups
		switch(Order)
		{
			case FPipelineFileCacheManager::PSOOrder::FirstToLatestUsed:
			{
				MetaData.ValueSort([](const FPipelineCacheFileFormatPSOMetaData& A, const FPipelineCacheFileFormatPSOMetaData& B) {return A.Stats.FirstFrameUsed > B.Stats.FirstFrameUsed;});
				break;
			}
			case FPipelineFileCacheManager::PSOOrder::MostToLeastUsed:
			{
				MetaData.ValueSort([](const FPipelineCacheFileFormatPSOMetaData& A, const FPipelineCacheFileFormatPSOMetaData& B) {return A.Stats.TotalBindCount > B.Stats.TotalBindCount;});
				break;
			}
			case FPipelineFileCacheManager::PSOOrder::Default:
			default:
			{
				// NOP - leave as is
				break;
			}
		}
	}
	
	void GetOrderedPSOHashes(TArray<FPipelineCachePSOHeader>& PSOHashes, FPipelineFileCacheManager::PSOOrder Order, int64 MinBindCount, TSet<uint32> const& AlreadyCompiledHashes)
	{
		if(Order != TOC.SortedOrder)
		{
			SortMetaData(TOC.MetaData, Order);
			TOC.SortedOrder = Order;
		}
		
		for (auto const& Hash : TOC.MetaData)
		{
			if( (Hash.Value.EngineFlags & FPipelineCacheFlagInvalidPSO) == 0 &&
				FPipelineFileCacheManager::MaskComparisonFn(FPipelineFileCacheManager::GameUsageMask, Hash.Value.UsageMask) &&
				Hash.Value.Stats.TotalBindCount >= MinBindCount &&
				!AlreadyCompiledHashes.Contains(Hash.Key))
			{
				FPipelineCachePSOHeader Header;
				Header.Hash = Hash.Key;
				Header.Shaders = Hash.Value.Shaders;
				PSOHashes.Add(Header);
			}
		}
	}
	
	bool OnExternalReadCallback(FPipelineCacheFileFormatPSORead* Entry, double RemainingTime)
	{
		TSharedPtr<IAsyncReadRequest, ESPMode::ThreadSafe> LocalReadRequest = Entry->ReadRequest;
		
		check(LocalReadRequest.IsValid());
		
		if (RemainingTime < 0.0 && !LocalReadRequest->PollCompletion())
		{
			return false;
		}
		else if (RemainingTime >= 0.0 && !LocalReadRequest->WaitCompletion(RemainingTime))
		{
			return false;
		}
		
		Entry->bReadCompleted = 1;
		
		return true;
	}
	
	void FetchPSODescriptors(TDoubleLinkedList<FPipelineCacheFileFormatPSORead*>& Batch)
	{
		for (TDoubleLinkedList<FPipelineCacheFileFormatPSORead*>::TIterator It(Batch.GetHead()); It; ++It)
		{
			FPipelineCacheFileFormatPSORead* Entry = *It;
			FPipelineCacheFileFormatPSOMetaData const& Meta = TOC.MetaData.FindChecked(Entry->Hash);
			
			if((Meta.EngineFlags & FPipelineCacheFlagInvalidPSO) != 0)
			{
				// In reality we should not get to this case as GetOrderedPSOHashes() won't pass back PSOs that have this flag set
				UE_LOGF(LogRHI, Verbose, "Encountered a PSO entry %u marked invalid - ignoring", Entry->Hash);
				Entry->bValid = false;
				continue;
			}
		
			if (Meta.FileGuid == FileGuid)
			{
				FPipelineCacheFileFormatPSOMetaData const* GameMeta = TOC.MetaData.Find(Entry->Hash);

				if (GameMeta && ensure(AsyncFileHandle.IsValid()))
				{
					Entry->Data.SetNum(GameMeta->FileSize);
					Entry->ParentFileHandle = AsyncFileHandle;
					Entry->ReadRequest = MakeShareable(AsyncFileHandle->ReadRequest(GameMeta->FileOffset, GameMeta->FileSize, AIOP_Normal, nullptr, Entry->Data.GetData()));
				}
				else
				{
					UE_LOGF(LogRHI, Verbose, "Encountered a PSO entry %u that has been removed from the cache file: %ls ", Entry->Hash, *Meta.FileGuid.ToString());
					Entry->bValid = false;
					continue;
				}
			}
            else
            {
                UE_LOGF(LogRHI, Warning, "Encountered a PSO entry %u that references unknown file ID: %ls", Entry->Hash, *Meta.FileGuid.ToString());
                Entry->bValid = false;
                continue;
            }
			
            Entry->bValid = true;
			FExternalReadCallback ExternalReadCallback = [this, Entry](double ReaminingTime)
			{
				return this->OnExternalReadCallback(Entry, ReaminingTime);
			};
			
			if (!Entry->Ar || !Entry->Ar->AttachExternalReadDependency(ExternalReadCallback))
			{
				ExternalReadCallback(0.0);
				check(Entry->bReadCompleted);
			}
		}
	}
	
	FName GetPlatformName() const
	{
		return PlatformName;
	}

	const FString& GetRecordingFilename() const
	{
		return RecordingFilename;
	}

	const FString& GetCacheFilename() const
	{
		return Name;
	}

	const FGuid& GetFileGuid() const
	{
		return FileGuid;
	}

	const int32 GetTOCMetaDataSize() const
	{
		return TOC.MetaData.Num();
	}
};
uint32 FPipelineCacheFile::GameVersion = 0;

//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////


bool FPipelineFileCacheManager::IsBSSEquivalentPSOEntryCached(FPipelineCacheFileFormatPSO const& NewEntry)
{
	check(!IsPSOEntryCached(NewEntry)); // this routine should only be called after we have done the much faster test

	bool bFound = false;

	for (auto MapIt = FileCacheMap.CreateIterator(); !bFound && MapIt; ++MapIt)
	{
		bFound = MapIt->Value->IsBSSEquivalentPSOEntryCachedInternal(NewEntry);
	}

	return bFound;
}

// note: when EntryData is supplied it also performs a state update, we may need to verify all occurrences are coherent...
bool FPipelineFileCacheManager::IsPSOEntryCached(FPipelineCacheFileFormatPSO const& NewEntry, FPSOUsageData* EntryData)
{
	bool bFound = false;

	for (auto MapIt = FileCacheMap.CreateIterator(); !bFound && MapIt; ++MapIt)
	{
		bFound = MapIt->Value->IsPSOEntryCachedInternal(NewEntry, EntryData);
	}
	return bFound;
}

bool FPipelineFileCacheManager::IsPipelineFileCacheEnabled()
{
	static bool bOnce = false;
	static bool bCmdLineForce = false;
	if (!bOnce)
	{
		bOnce = true;
		bCmdLineForce = FParse::Param(FCommandLine::Get(), TEXT("psocache"));
		UE_CLOGF(bCmdLineForce, LogRHI, Warning, "****************************** Forcing PSO cache from command line");
	}
	return FileCacheEnabled && (bCmdLineForce || CVarPSOFileCacheEnabled.GetValueOnAnyThread() == 1);
}

bool FPipelineFileCacheManager::LogPSOtoFileCache()
{
	static bool bOnce = false;
	static bool bCmdLineForce = false;
	if (!bOnce)
	{
		bOnce = true;
		bCmdLineForce = FParse::Param(FCommandLine::Get(), TEXT("logpso"));
		UE_CLOGF(bCmdLineForce, LogRHI, Warning, "****************************** Forcing logging of PSOs from command line");
	}
	return (bCmdLineForce || CVarPSOFileCacheLogPSO.GetValueOnAnyThread() == 1);
}

bool FPipelineFileCacheManager::ReportNewPSOs()
{
    static bool bOnce = false;
    static bool bCmdLineForce = false;
    if (!bOnce)
    {
        bOnce = true;
        bCmdLineForce = FParse::Param(FCommandLine::Get(), TEXT("reportpso"));
        UE_CLOGF(bCmdLineForce, LogRHI, Warning, "****************************** Forcing reporting of new PSOs from command line");
    }
	return (bCmdLineForce || CVarPSOFileCacheReportPSO.GetValueOnAnyThread() == 1);
}

bool FPipelineFileCacheManager::LogPSODetails()
{
    static bool bOnce = false;
    static bool bCmdLineOption = false;
#if !UE_BUILD_SHIPPING
    if (!bOnce)
    {
        bOnce = true;
        bCmdLineOption = FParse::Param(FCommandLine::Get(), TEXT("logpsodetails"));
    }
#endif
	return bCmdLineOption;
}

void FPipelineFileCacheManager::Initialize(uint32 InGameVersion)
{
	ClearOSPipelineCache();
	
	// Make enabled explicit on a flag not the existence of "FileCache" object as we are using that behind a lock and in Open / Close operations
	FileCacheEnabled = ShouldEnableFileCache();
	FPipelineCacheFile::GameVersion = InGameVersion;
	if (FPipelineCacheFile::GameVersion == 0)
	{
		// Defaulting the CL is fine though
		FPipelineCacheFile::GameVersion = (uint32)FEngineVersion::Current().GetChangelist();
	}

	SET_MEMORY_STAT(STAT_NewCachedPSOMemory, 0);
	SET_MEMORY_STAT(STAT_PSOStatMemory, 0);
}

bool FPipelineFileCacheManager::ShouldEnableFileCache()
{
	if (!GRHISupportsPipelineFileCache)
	{
		return false;
	}

#if PLATFORM_IOS
	if (CVarAlwaysGeneratePOSSOFileCache.GetValueOnAnyThread() == 0)
	{
		struct stat FileInfo;
		static FString PrivateWritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
		FString Result = PrivateWritePathBase + FString([NSString stringWithFormat:@"/Caches/%@/com.apple.metal/functions.data", [NSBundle mainBundle].bundleIdentifier]);
		FString Result2 = PrivateWritePathBase + FString([NSString stringWithFormat:@"/Caches/%@/com.apple.metal/usecache.txt", [NSBundle mainBundle].bundleIdentifier]);
		if (stat(TCHAR_TO_UTF8(*Result), &FileInfo) != -1 && stat(TCHAR_TO_UTF8(*Result2), &FileInfo) != -1)
		{
			return false;
		}
	}
#endif

	return true;
}

void FPipelineFileCacheManager::PreCompileComplete()
{
#if PLATFORM_IOS
	// write out a file signifying we have completed a pre-compile of the PSO cache. Used on successive runs of the game to determine how much caching we need to still perform
	static FString PrivateWritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
	FString Result = PrivateWritePathBase + FString([NSString stringWithFormat:@"/Caches/%@/com.apple.metal/usecache.txt", [NSBundle mainBundle].bundleIdentifier]);
	int32 Handle = open(TCHAR_TO_UTF8(*Result), O_CREAT, S_IRUSR | S_IWUSR | S_IRGRP | S_IWGRP | S_IROTH | S_IWOTH);
	FString Version = FEngineVersion::Current().ToString();
	write(Handle, TCHAR_TO_ANSI(*Version), Version.Len());
	close(Handle);
#endif
}

void FPipelineFileCacheManager::ClearOSPipelineCache()
{
	UE_LOGF(LogTemp, Display, "Clearing the OS Cache");
	
	bool bCmdLineSkip = FParse::Param(FCommandLine::Get(), TEXT("skippsoclear"));
	if (CVarClearOSPSOFileCache.GetValueOnAnyThread() > 0 && !bCmdLineSkip)
	{
		// clear the PSO cache on IOS if the executable is newer
#if PLATFORM_IOS
		SCOPED_AUTORELEASE_POOL;

		static FString ExecutablePath = FString([[NSBundle mainBundle] bundlePath]) + TEXT("/") + FPlatformProcess::ExecutableName();
		struct stat FileInfo;
		if(stat(TCHAR_TO_UTF8(*ExecutablePath), &FileInfo) != -1)
		{
			// TODO: add ability to only do this change on major release as opposed to minor release (e.g. 10.30 -> 10.40 (delete) vs 10.40 -> 10.40.1 (don't delete)), this is very much game specific, so need a way to have games be able to modify this
			FTimespan ExecutableTime(0, 0, FileInfo.st_atime);
			static FString PrivateWritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
			FString Result = PrivateWritePathBase + FString([NSString stringWithFormat:@"/Caches/%@/com.apple.metal/functions.data", [NSBundle mainBundle].bundleIdentifier]);
			if (stat(TCHAR_TO_UTF8(*Result), &FileInfo) != -1)
			{
				FTimespan DataTime(0, 0, FileInfo.st_atime);
				if (ExecutableTime > DataTime)
				{
					UE_LOGF(LogTemp, Display, "Clearing functions.data");
					unlink(TCHAR_TO_UTF8(*Result));
				}
			}
			Result = PrivateWritePathBase + FString([NSString stringWithFormat:@"/Caches/%@/com.apple.metal/functions.maps", [NSBundle mainBundle].bundleIdentifier]);
			if (stat(TCHAR_TO_UTF8(*Result), &FileInfo) != -1)
			{
				FTimespan MapsTime(0, 0, FileInfo.st_atime);
				if (ExecutableTime > MapsTime)
				{
					UE_LOGF(LogTemp, Display, "Clearing functions.maps");
					unlink(TCHAR_TO_UTF8(*Result));
				}
			}
		}
#elif PLATFORM_MAC && (UE_BUILD_TEST || UE_BUILD_SHIPPING)
		if (!FPlatformProcess::IsSandboxedApplication())
		{
			SCOPED_AUTORELEASE_POOL;

			static FString ExecutablePath = FString([[NSBundle mainBundle] executablePath]);
			struct stat FileInfo;
			if (stat(TCHAR_TO_UTF8(*ExecutablePath), &FileInfo) != -1)
			{
				FTimespan ExecutableTime(0, 0, FileInfo.st_atime);
				FString CacheDir = FString([NSString stringWithFormat:@"%@/../C/%@/com.apple.metal", NSTemporaryDirectory(), [NSBundle mainBundle].bundleIdentifier]);
				TArray<FString> FoundFiles;
				IPlatformFile::GetPlatformPhysical().FindFilesRecursively(FoundFiles, *CacheDir, TEXT(".data"));

				// Find functions.data file in cache subfolders. If it's older than the executable, delete the whole cache.
				bool bIsCacheOutdated = false;
				for (FString& DataFile : FoundFiles)
				{
					if (FPaths::GetCleanFilename(DataFile) == TEXT("functions.data") && stat(TCHAR_TO_UTF8(*DataFile), &FileInfo) != -1)
					{
						FTimespan DataTime(0, 0, FileInfo.st_atime);
						if (ExecutableTime > DataTime)
						{
							bIsCacheOutdated = true;
						}
					}
				}

				if (bIsCacheOutdated)
				{
					IPlatformFile::GetPlatformPhysical().DeleteDirectoryRecursively(*CacheDir);
				}
			}
		}
#endif
	}
}

int32 FPipelineFileCacheManager::GetTotalPSOCount(const FString& PSOCacheKey)
{
	int32 TotalPSOs = 0;
	if (IsPipelineFileCacheEnabled())
	{
		FRWScopeLock Lock(FileCacheLock, SLT_ReadOnly);
		FPipelineCacheFile* Found = GetPipelineCacheFileFromKey(PSOCacheKey);
		TotalPSOs = Found ? Found->GetTOCMetaDataSize() : 0;
	}
	return TotalPSOs;
}

uint64 FPipelineFileCacheManager::SetGameUsageMaskWithComparison(uint64 InGameUsageMask, FPSOMaskComparisonFn InComparisonFnPtr)
{
	uint64 OldMask = 0;
	if(IsPipelineFileCacheEnabled())
	{
		FRWScopeLock Lock(FileCacheLock, SLT_Write);
		
		OldMask = FPipelineFileCacheManager::GameUsageMask;
		FPipelineFileCacheManager::GameUsageMask = InGameUsageMask;
		
		if(InComparisonFnPtr == nullptr)
		{
			InComparisonFnPtr = DefaultPSOMaskComparisonFunction;
		}
		
		FPipelineFileCacheManager::MaskComparisonFn = InComparisonFnPtr;
		FPipelineFileCacheManager::GameUsageMaskSet = true;
	}
	
	return OldMask;
}

void FPipelineFileCacheManager::Shutdown()
{
	if(IsPipelineFileCacheEnabled())
	{
		FRWScopeLock Lock(FileCacheLock, SLT_Write);
		for (auto const& Pair : Stats)
		{
			delete Pair.Value;
		}
		Stats.Empty();
		NewPSOs.Empty();
        NumNewPSOs = 0;
		
		FileCacheMap.Empty();
		FileCacheEnabled = false;
		
		SET_MEMORY_STAT(STAT_NewCachedPSOMemory, 0);
		SET_MEMORY_STAT(STAT_PSOStatMemory, 0);
	}
}

bool FPipelineFileCacheManager::HasPipelineFileCache(const FString & Key)
{
	FRWScopeLock Lock(FileCacheLock, SLT_ReadOnly);
	return FileCacheMap.Contains(Key);
}

bool FPipelineFileCacheManager::OpenPipelineFileCache(const FString& Key, const FString& CacheName, EShaderPlatform Platform, FGuid& OutGameFileGuid)
{
	bool bOk = false;
	OutGameFileGuid = FGuid();
	
	if(IsPipelineFileCacheEnabled())
	{
		FRWScopeLock Lock(FileCacheLock, SLT_Write);
		
		bool bFound = FileCacheMap.Contains(Key);

		if(!bFound)
		{
			TUniquePtr<FPipelineCacheFile> NewFileCache = MakeUnique<FPipelineCacheFile>();

			bOk = NewFileCache->OpenPipelineFileCache(CacheName, Platform, OutGameFileGuid);
			
			if (!bOk)
			{
				NewFileCache = nullptr;
			}
			else
			{
				UE_LOGF(LogRHI, Display, "FPipelineCacheFile[%ls] opened %ls, filename %ls, guid %ls. ", *Key, *CacheName, *NewFileCache->GetCacheFilename(), *NewFileCache->GetFileGuid().ToString());
				FileCacheMap.Add(Key, MoveTemp(NewFileCache));
				check(!GameGuidToCacheKey.Contains(OutGameFileGuid));
				GameGuidToCacheKey.Add(OutGameFileGuid, Key);
			}
		}
	}
	
	return bOk;
}

bool FPipelineFileCacheManager::OpenUserPipelineFileCache(const FString& Key, const FString& CacheName, EShaderPlatform Platform, FGuid& OutGameFileGuid)
{
	// close any existing cache and reset the user cache's PSO recording containers.
	CloseUserPipelineFileCache();

	bool bUserFileOpened = false;
	if (IsPipelineFileCacheEnabled())
	{
		FRWScopeLock Lock(FileCacheLock, SLT_Write);

		UserCacheKey = Key;
		FPipelineCacheFile* FileCache = GetPipelineCacheFileFromKey(UserCacheKey);
		if(ensure(!FileCache))
		{
			TUniquePtr<FPipelineCacheFile> NewFileCache = MakeUnique<FPipelineCacheFile>();
			FileCache = NewFileCache.Get();
			bUserFileOpened = NewFileCache->OpenUserPipelineFileCache(CacheName, Platform);

			// we always add the user cache, even if we did not open a file
			FileCacheMap.Add(UserCacheKey, MoveTemp(NewFileCache));
			check(!GameGuidToCacheKey.Contains(FileCache->GetFileGuid()));
			GameGuidToCacheKey.Add(FileCache->GetFileGuid(), UserCacheKey);
		}
		OutGameFileGuid = FileCache->GetFileGuid();
		UE_LOGF(LogRHI, Display, "FPipelineCacheFile User cache [key:%ls] opened '%ls'=%d, filename %ls, guid %ls. ", *UserCacheKey, *CacheName, (int)bUserFileOpened, *FileCache->GetCacheFilename(), *FileCache->GetFileGuid().ToString());

		// User Cache now exists - these caches should be empty for this file otherwise will have false positives from any previous file caching - if not something has been caching when it should not be
		check(NewPSOs.Num() == 0);
		check(RunTimeToPSOUsage.Num() == 0);
	}
	return bUserFileOpened;
}

void FPipelineFileCacheManager::CloseUserPipelineFileCache()
{
	if (IsPipelineFileCacheEnabled())
	{
		FRWScopeLock Lock(FileCacheLock, SLT_Write);
		if(GetPipelineCacheFileFromKey(UserCacheKey))
		{
			const FGuid& UserGuid = GetPipelineCacheFileFromKey(UserCacheKey)->GetFileGuid();
			GameGuidToCacheKey.Remove(UserGuid);
			FileCacheMap.Remove(UserCacheKey);
		}

		// Reset stats tracking for the next file.
		for (auto const& Pair : Stats)
		{
			FPlatformAtomics::InterlockedExchange((int64*)&Pair.Value->TotalBindCount, -1);
			FPlatformAtomics::InterlockedExchange((int64*)&Pair.Value->FirstFrameUsed, -1);
			FPlatformAtomics::InterlockedExchange((int64*)&Pair.Value->LastFrameUsed, -1);
		}

		// Reset serialized counts
		SET_DWORD_STAT(STAT_SerializedGraphicsPipelineStateCount, 0);
		SET_DWORD_STAT(STAT_SerializedComputePipelineStateCount, 0);

		// Not tracking when there is no file clear other stats as well
		SET_DWORD_STAT(STAT_TotalGraphicsPipelineStateCount, 0);
		SET_DWORD_STAT(STAT_TotalComputePipelineStateCount, 0);
		SET_DWORD_STAT(STAT_TotalRayTracingPipelineStateCount, 0);
		SET_DWORD_STAT(STAT_NewGraphicsPipelineStateCount, 0);
		SET_DWORD_STAT(STAT_NewComputePipelineStateCount, 0);
		SET_DWORD_STAT(STAT_NewRayTracingPipelineStateCount, 0);

		// Clear Runtime hashes otherwise we can't start adding newPSO's for a newly opened file
		RunTimeToPSOUsage.Empty();
		NewPSOUsage.Empty();
		NewPSOs.Empty();
		NumNewPSOs = 0;

		SET_MEMORY_STAT(STAT_NewCachedPSOMemory, 0);
	}
}

bool FPipelineFileCacheManager::SavePipelineFileCache(SaveMode Mode)
{
	bool bOk = false;

	if (IsPipelineFileCacheEnabled() && LogPSOtoFileCache())
	{
		CSV_EVENT(PSO, TEXT("Saving PSO cache"));
		FRWScopeLock Lock(FileCacheLock, SLT_Write);

		FPipelineCacheFile* UserCache = GetPipelineCacheFileFromKey(UserCacheKey);
		bOk = (UserCache != nullptr) && UserCache->SavePipelineFileCache(Mode, Stats, NewPSOs, RequestedOrder, NewPSOUsage);
		// If successful clear new PSO's as they should have been saved out
		// Leave everything else in-tact (e.g stats) for subsequent in place save operations
		if (bOk)
		{
			NumNewPSOs = NewPSOs.Num();
			SET_MEMORY_STAT(STAT_NewCachedPSOMemory, (NumNewPSOs * (sizeof(FPipelineCacheFileFormatPSO) + sizeof(uint32) + sizeof(uint32))));
		}
	}
	return bOk;
}

void FPipelineFileCacheManager::RegisterPSOUsageDataUpdateForNextSave(FPSOUsageData& UsageData)
{
	FPSOUsageData& CurrentEntry = NewPSOUsage.FindOrAdd(UsageData.PSOHash);
	CurrentEntry.PSOHash = UsageData.PSOHash;
	CurrentEntry.UsageMask |= UsageData.UsageMask;
	CurrentEntry.EngineFlags |= UsageData.EngineFlags;
}  

void FPipelineFileCacheManager::LogNewGraphicsPSOToConsoleAndCSV(FPipelineCacheFileFormatPSO& PSO, uint32 PSOHash, EPSOPrecacheResult PSOPrecacheResult)
{
	if (!LogNewPSOsToConsoleAndCSV)
	{
		return;
	}

	// Only print if not completed by PSO precaching
	if (PSOPrecacheResult != EPSOPrecacheResult::Complete)
	{
		CSV_EVENT(PSO, TEXT("Encountered new graphics PSO"));
		UE_LOGF(LogRHI, Display, "Encountered a new graphics PSO: %u (PSOPrecacheState: %ls)", PSOHash, LexToString(PSOPrecacheResult));
		int32 LogDetailLevel = LogPSODetails() ? 2 : GPSOFileCachePrintNewPSODescriptors;
		if (LogDetailLevel > 0)
		{
			UE_LOGF(LogRHI, Display, "New Graphics PSO (%u)", PSOHash);
			if (LogDetailLevel > 1)
			{
				UE_LOGF(LogRHI, Display, "%ls", *PSO.ToStringReadable());
			}
		}

	}
	else
	{
		UE_LOGF(LogRHI, Verbose, "Encountered a new graphics PSO for the file cache but it was already precached at runtime: %u", PSOHash);
	}
}

void FPipelineFileCacheManager::LogNewComputePSOToConsoleAndCSV(FPipelineCacheFileFormatPSO& PSO, uint32 PSOHash, EPSOPrecacheResult PSOPrecacheResult)
{
	if (!LogNewPSOsToConsoleAndCSV)
	{
		return;
	}

	if (PSOPrecacheResult != EPSOPrecacheResult::Complete)
	{
		CSV_EVENT(PSO, TEXT("Encountered new compute PSO"));
		UE_LOGF(LogRHI, Display, "Encountered a new compute PSO: %u (PSOPrecacheState: %ls)", PSOHash, LexToString(PSOPrecacheResult));
		if (GPSOFileCachePrintNewPSODescriptors > 0)
		{
			UE_LOGF(LogRHI, Display, "New compute PSO (%u) Description: %ls", PSOHash, *PSO.ComputeDesc.ComputeShader.ToString());
		}
	}
	else
	{
		UE_LOGF(LogRHI, Verbose, "Encountered a new compute PSO for the file cache but it was already precached at runtime: %u", PSOHash);
	}
}

void FPipelineFileCacheManager::LogNewRaytracingPSOToConsole(FPipelineCacheFileFormatPSO& PSO, uint32 PSOHash, bool bIsNonBlockingPSO)
{
	// When non-blocking creation is used, encountering a non-cached RTPSO is not likely to cause a hitch and so the logging is not useful/actionable.
	if (!LogNewPSOsToConsoleAndCSV || bIsNonBlockingPSO)
	{
		return;
	}

	UE_LOGF(LogRHI, Display, "Encountered a new ray tracing PSO: %u", PSOHash);
	if (GPSOFileCachePrintNewPSODescriptors > 0)
	{
		UE_LOGF(LogRHI, Display, "New ray tracing PSO (%u) Description: %ls", PSOHash, *PSO.RayTracingDesc.ToString());
	}
}

void FPipelineFileCacheManager::BroadcastNewPSOsDelegate()
{
	TArray<FPipelineCacheFileFormatPSO> PSOs;
	{
		FPipelineCacheFileFormatPSO PSO;
		while (NewPSOsToReport.Dequeue(PSO))
		{
			PSOs.Emplace(MoveTemp(PSO));
		}
	}

	if (!PSOs.IsEmpty() && ReportNewPSOs())
	{
		// It's not safe to touch UObjects-based delegates from the render thread.
		ExecuteOnGameThread(TEXT("OnPipelineStateLoggedBroadcastGT"), [InPSOs = MoveTemp(PSOs)]()
		{
			if (PSOLoggedEvent.IsBound())
			{
				for (const FPipelineCacheFileFormatPSO& PSO : InPSOs)
				{
					PSOLoggedEvent.Broadcast(PSO);
				}
			}
		});
	}
}

void FPipelineFileCacheManager::CacheGraphicsPSO(uint32 RunTimeHash, FGraphicsPipelineStateInitializer const& Initializer, EPSOPrecacheResult PSOPrecacheResult, FPipelineStateStats** OutStats)
{
	if(IsPipelineFileCacheEnabled() && (LogPSOtoFileCache() || ReportNewPSOs()))
	{
		FRWScopeLock Lock(FileCacheLock, SLT_ReadOnly);

		FPSOUsageData* PSOUsage = RunTimeToPSOUsage.Find(RunTimeHash);
		if(PSOUsage == nullptr || !IsReferenceMaskSet(FPipelineFileCacheManager::GameUsageMask, PSOUsage->UsageMask))
		{
			Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
			PSOUsage = RunTimeToPSOUsage.Find(RunTimeHash);
				
			if(PSOUsage == nullptr)
			{
				FPipelineCacheFileFormatPSO NewEntry;
				bool bOK = FPipelineCacheFileFormatPSO::Init(NewEntry, Initializer, PSOPrecacheResult);
				check(bOK);
					
				uint32 PSOHash = GetTypeHash(NewEntry);
				FPSOUsageData CurrentUsageData(PSOHash, 0, 0);

				if (!FPipelineFileCacheManager::IsPSOEntryCached(NewEntry, &CurrentUsageData))
				{
					bool bActuallyNewPSO = !NewPSOs.Contains(PSOHash);
					if (Initializer.bFromPSOFileCache)
					{
						// FIXME: this is a workaround. Needs proper investigation
						UE_LOGF(LogRHI, Warning, "PSO from the cache was not found in the cache! PSOHash: %u", PSOHash);
						bActuallyNewPSO = false;
					}

					if (bActuallyNewPSO && IsOpenGLPlatform(GMaxRHIShaderPlatform)) // OpenGL is a BSS platform and so we don't report BSS matches as missing.
					{
						bActuallyNewPSO = !FPipelineFileCacheManager::IsBSSEquivalentPSOEntryCached(NewEntry);
					}

					if (bActuallyNewPSO)
					{
						// Only add PSOs which are not fully precached yet if GPSOExcludePrecachePSOsInFileCache is set (includes miss, too late, unknown, NotSupported)
						if (PSOPrecacheResult == EPSOPrecacheResult::Complete)
						{
							bActuallyNewPSO = !GPSOExcludePrecachePSOsInFileCache;
						}
						// Only include precache-only PSOs if r.ShaderPipelineCache.IncludeAllPrecachedPSOs is enabled
						if (PSOPrecacheResult == EPSOPrecacheResult::Active)
						{
							bActuallyNewPSO = GPSOIncludeAllPrecachedPSOsInFileCache != 0;
						}
					}

					if (bActuallyNewPSO) 
					{						
						LogNewGraphicsPSOToConsoleAndCSV(NewEntry, PSOHash, PSOPrecacheResult);

						if (LogPSOtoFileCache())
						{
							NewPSOs.Add(PSOHash, NewEntry);
							INC_MEMORY_STAT_BY(STAT_NewCachedPSOMemory, sizeof(FPipelineCacheFileFormatPSO) + sizeof(uint32) + sizeof(uint32));
						}

						NumNewPSOs++;
						INC_DWORD_STAT(STAT_NewGraphicsPipelineStateCount);
						INC_DWORD_STAT(STAT_TotalGraphicsPipelineStateCount);

						if (ReportNewPSOs())
						{
							NewPSOsToReport.Enqueue(MoveTemp(NewEntry));
						}
					}
				}
					
				// Only set if the file cache doesn't have this Mask for the PSO - avoid making more entries and unnessary file saves
				if(!IsReferenceMaskSet(FPipelineFileCacheManager::GameUsageMask, CurrentUsageData.UsageMask))
				{
					CurrentUsageData.UsageMask |= FPipelineFileCacheManager::GameUsageMask;
					RegisterPSOUsageDataUpdateForNextSave(CurrentUsageData);
				}
					
				// Apply the existing file PSO Usage mask and current to our "fast" runtime check
				RunTimeToPSOUsage.Add(RunTimeHash, CurrentUsageData);
			}
			else if(!IsReferenceMaskSet(FPipelineFileCacheManager::GameUsageMask, PSOUsage->UsageMask))
			{
				PSOUsage->UsageMask |= FPipelineFileCacheManager::GameUsageMask;
				RegisterPSOUsageDataUpdateForNextSave(*PSOUsage);

				// If this PSO was previously logged as precache-only, upgrade its stored result to the
				// real runtime outcome. The Active state itself guarantees the CVar was set when logged.
				FPipelineCacheFileFormatPSO* Entry = NewPSOs.Find(PSOUsage->PSOHash);
				if (Entry && Entry->PSOPrecacheResult == EPSOPrecacheResult::Active)
				{
					Entry->PSOPrecacheResult = PSOPrecacheResult;
				}
			}
		}

		// Optionally supply stats to share the lock scope
		if (OutStats)
		{
			*OutStats = RegisterPSOStatsInternal(Lock, RunTimeHash);
		}
	}
}

void FPipelineFileCacheManager::CacheComputePSO(uint32 RunTimeHash, FComputePipelineStateInitializer const& Initializer, EPSOPrecacheResult PSOPrecacheResult, FPipelineStateStats** OutStats)
{
	if(IsPipelineFileCacheEnabled() && (LogPSOtoFileCache() || ReportNewPSOs()))
	{
		FRWScopeLock Lock(FileCacheLock, SLT_ReadOnly);
		
		{
			FPSOUsageData* PSOUsage = RunTimeToPSOUsage.Find(RunTimeHash);
			if(PSOUsage == nullptr || !IsReferenceMaskSet(FPipelineFileCacheManager::GameUsageMask, PSOUsage->UsageMask))
			{
				Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
				PSOUsage = RunTimeToPSOUsage.Find(RunTimeHash);
				
				if(PSOUsage == nullptr)
				{
					FPipelineCacheFileFormatPSO NewEntry;
					bool bOK = FPipelineCacheFileFormatPSO::Init(NewEntry, Initializer, PSOPrecacheResult);
					check(bOK);
					
					uint32 PSOHash = GetTypeHash(NewEntry);
					FPSOUsageData CurrentUsageData(PSOHash, 0, 0);
					
					if (!FPipelineFileCacheManager::IsPSOEntryCached(NewEntry, &CurrentUsageData))
					{
						bool bActuallyNewPSO = !NewPSOs.Contains(PSOHash);
						if (bActuallyNewPSO)
						{
							// Only add PSOs which are not fully precached yet if GPSOExcludePrecachePSOsInFileCache is set (includes miss, too late, unknown, NotSupported)
							if (PSOPrecacheResult == EPSOPrecacheResult::Complete)
							{
								bActuallyNewPSO = !GPSOExcludePrecachePSOsInFileCache;
							}
							// Only include precache-only PSOs if r.ShaderPipelineCache.IncludeAllPrecachedPSOs is enabled
							if (PSOPrecacheResult == EPSOPrecacheResult::Active)
							{
								bActuallyNewPSO = GPSOIncludeAllPrecachedPSOsInFileCache != 0;
							}
						}

						if (bActuallyNewPSO)
						{
							LogNewComputePSOToConsoleAndCSV(NewEntry, PSOHash, PSOPrecacheResult);

							if (LogPSOtoFileCache())
							{
								NewPSOs.Add(PSOHash, NewEntry);
								INC_MEMORY_STAT_BY(STAT_NewCachedPSOMemory, sizeof(FPipelineCacheFileFormatPSO) + sizeof(uint32) + sizeof(uint32));
							}

							NumNewPSOs++;
							INC_DWORD_STAT(STAT_NewComputePipelineStateCount);
							INC_DWORD_STAT(STAT_TotalComputePipelineStateCount);

							if (ReportNewPSOs())
							{
								NewPSOsToReport.Enqueue(MoveTemp(NewEntry));
							}
						}
					}
					
					// Only set if the file cache doesn't have this Mask for the PSO - avoid making more entries and unnessary file saves
					if(!IsReferenceMaskSet(FPipelineFileCacheManager::GameUsageMask, CurrentUsageData.UsageMask))
					{
						CurrentUsageData.UsageMask |= FPipelineFileCacheManager::GameUsageMask;
						RegisterPSOUsageDataUpdateForNextSave(CurrentUsageData);
					}
					
					// Apply the existing file PSO Usage mask and current to our "fast" runtime check
					RunTimeToPSOUsage.Add(RunTimeHash, CurrentUsageData);
				}
				else if(!IsReferenceMaskSet(FPipelineFileCacheManager::GameUsageMask, PSOUsage->UsageMask))
				{
					PSOUsage->UsageMask |= FPipelineFileCacheManager::GameUsageMask;
					RegisterPSOUsageDataUpdateForNextSave(*PSOUsage);

					// If this PSO was previously logged as precache-only, upgrade its stored result to the
					// real runtime outcome. The Active state itself guarantees the CVar was set when logged.
					FPipelineCacheFileFormatPSO* Entry = NewPSOs.Find(PSOUsage->PSOHash);
					if (Entry && Entry->PSOPrecacheResult == EPSOPrecacheResult::Active)
					{
						Entry->PSOPrecacheResult = PSOPrecacheResult;
					}
				}
			}
		}

		// Optionally supply stats to share the lock scope
		if (OutStats)
		{
			*OutStats = RegisterPSOStatsInternal(Lock, RunTimeHash);
		}
	}
}

void FPipelineFileCacheManager::CacheRayTracingPSO(const FRayTracingPipelineStateInitializer& Initializer, ERayTracingPipelineCacheFlags Flags)
{
	if (!IsPipelineFileCacheEnabled() || !(LogPSOtoFileCache() || ReportNewPSOs()))
	{
		return;
	}

	TArrayView<FRHIRayTracingShader*> ShaderTables[] =
	{
		Initializer.GetRayGenTable(),
		Initializer.GetMissTable(),
		Initializer.GetHitGroupTable(),
		Initializer.GetCallableTable()
	};

	const bool bIsNonBlocking = !EnumHasAnyFlags(Flags, ERayTracingPipelineCacheFlags::NonBlocking);

	FRWScopeLock Lock(FileCacheLock, SLT_ReadOnly);

	for (TArrayView<FRHIRayTracingShader*>& Table : ShaderTables)
	{
		for (FRHIRayTracingShader* Shader : Table)
		{
			FPipelineCacheFileFormatPSO::FPipelineFileCacheRayTracingDesc Desc(Initializer, Shader);
			uint32 RunTimeHash = GetTypeHash(Desc);

			FPSOUsageData* PSOUsage = RunTimeToPSOUsage.Find(RunTimeHash);
			if (PSOUsage == nullptr || !IsReferenceMaskSet(FPipelineFileCacheManager::GameUsageMask, PSOUsage->UsageMask))
			{
				Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
				PSOUsage = RunTimeToPSOUsage.Find(RunTimeHash);
				if (PSOUsage == nullptr)
				{
					FPipelineCacheFileFormatPSO NewEntry;
					bool bOK = FPipelineCacheFileFormatPSO::Init(NewEntry, Desc);
					check(bOK);

					uint32 PSOHash = GetTypeHash(NewEntry);
					FPSOUsageData CurrentUsageData(PSOHash, 0, 0);

					if (!FPipelineFileCacheManager::IsPSOEntryCached(NewEntry, &CurrentUsageData))
					{
						LogNewRaytracingPSOToConsole(NewEntry, PSOHash, bIsNonBlocking);

						if (LogPSOtoFileCache())
						{
							NewPSOs.Add(PSOHash, NewEntry);
							INC_MEMORY_STAT_BY(STAT_NewCachedPSOMemory, sizeof(FPipelineCacheFileFormatPSO) + sizeof(uint32) + sizeof(uint32));
						}

						NumNewPSOs++;
						INC_DWORD_STAT(STAT_NewRayTracingPipelineStateCount);
						INC_DWORD_STAT(STAT_TotalRayTracingPipelineStateCount);

						if (ReportNewPSOs())
						{
							NewPSOsToReport.Enqueue(MoveTemp(NewEntry));
						}
					}

					// Only set if the file cache doesn't have this Mask for the PSO - avoid making more entries and unnessary file saves
					if (!IsReferenceMaskSet(FPipelineFileCacheManager::GameUsageMask, CurrentUsageData.UsageMask))
					{
						CurrentUsageData.UsageMask |= FPipelineFileCacheManager::GameUsageMask;
						RegisterPSOUsageDataUpdateForNextSave(CurrentUsageData);
					}

					// Apply the existing file PSO Usage mask and current to our "fast" runtime check
					RunTimeToPSOUsage.Add(RunTimeHash, CurrentUsageData);

					// Immediately register usage of this ray tracing shader
					FPipelineStateStats* Stat = Stats.FindRef(PSOHash);
					if (Stat == nullptr)
					{
						Stat = new FPipelineStateStats;
						Stat->FirstFrameUsed = 0;
						Stat->LastFrameUsed = 0;
						Stat->CreateCount = 1;
						Stat->TotalBindCount = 1;
						Stat->PSOHash = PSOHash;
						Stats.Add(PSOHash, Stat);
						INC_MEMORY_STAT_BY(STAT_PSOStatMemory, sizeof(FPipelineStateStats) + sizeof(uint32));
					}
				}
			}
			else if (!IsReferenceMaskSet(FPipelineFileCacheManager::GameUsageMask, PSOUsage->UsageMask))
			{
				PSOUsage->UsageMask |= FPipelineFileCacheManager::GameUsageMask;
				RegisterPSOUsageDataUpdateForNextSave(*PSOUsage);
			}
		}
	}
}

void FPipelineFileCacheManager::RegisterPSOCompileFailure(uint32 RunTimeHash, FGraphicsPipelineStateInitializer const& Initializer)
{
	if(IsPipelineFileCacheEnabled() && (LogPSOtoFileCache() || ReportNewPSOs()) && Initializer.bFromPSOFileCache)
	{
		FRWScopeLock Lock(FileCacheLock, SLT_ReadOnly);

		FPSOUsageData* PSOUsage = RunTimeToPSOUsage.Find(RunTimeHash);
		if(PSOUsage == nullptr || !IsReferenceMaskSet(FPipelineCacheFlagInvalidPSO, PSOUsage->EngineFlags))
		{
			Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
			PSOUsage = RunTimeToPSOUsage.Find(RunTimeHash);
				
			if(PSOUsage == nullptr)
			{
				FPipelineCacheFileFormatPSO ShouldBeExistingEntry;
				bool bOK = FPipelineCacheFileFormatPSO::Init(ShouldBeExistingEntry, Initializer, EPSOPrecacheResult::Untracked);
				check(bOK);
					
				uint32 PSOHash = GetTypeHash(ShouldBeExistingEntry);
				FPSOUsageData CurrentUsageData(PSOHash, 0, 0);
					
				bool bCached = FPipelineFileCacheManager::IsPSOEntryCached(ShouldBeExistingEntry, &CurrentUsageData);
				check(bCached);	//bFromPSOFileCache was set but not in the cache something has gone wrong
				{
					CurrentUsageData.EngineFlags |= FPipelineCacheFlagInvalidPSO;
						
					RegisterPSOUsageDataUpdateForNextSave(CurrentUsageData);
					RunTimeToPSOUsage.Add(RunTimeHash, CurrentUsageData);
						
					UE_LOGF(LogRHI, Warning, "Graphics PSO (%u) compile failure registering to File Cache", PSOHash);
				}
			}
			else if(!IsReferenceMaskSet(FPipelineCacheFlagInvalidPSO, PSOUsage->EngineFlags))
			{
				PSOUsage->EngineFlags |= FPipelineCacheFlagInvalidPSO;
				RegisterPSOUsageDataUpdateForNextSave(*PSOUsage);
					
				UE_LOGF(LogRHI, Warning, "Graphics PSO (%u) compile failure registering to File Cache", PSOUsage->PSOHash);
			}
		}
	}
}

FPipelineStateStats* FPipelineFileCacheManager::RegisterPSOStatsInternal(FRWScopeLock& Lock, uint32 RunTimeHash)
{
	// May fail registration if the user cache has been closed
	FPSOUsageData* PSOUsage = RunTimeToPSOUsage.Find(RunTimeHash);
	if (!PSOUsage)
	{
		UE_LOGF(LogRHI, Display, "PSO (%u) missing from cache, please instead use the OutStats parameter in CacheGraphicsPSO/CacheComputePSO", RunTimeHash);
		return nullptr;
	}

	uint32 PSOHash = PSOUsage->PSOHash;
	FPipelineStateStats* Stat = Stats.FindRef(PSOHash);
	if (!Stat)
	{
		Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
		Stat = Stats.FindRef(PSOHash);
		if (!Stat)
		{
			Stat = new FPipelineStateStats;
			Stat->PSOHash = PSOHash;
			Stats.Add(PSOHash, Stat);

			INC_MEMORY_STAT_BY(STAT_PSOStatMemory, sizeof(FPipelineStateStats) + sizeof(uint32));
		}
	}
	Stat->CreateCount++;

	return Stat;
}

void FPipelineFileCacheManager::GetOrderedPSOHashes(const FString& PSOCacheKey, TArray<FPipelineCachePSOHeader>& PSOHashes, PSOOrder Order, int64 MinBindCount, TSet<uint32> const& AlreadyCompiledHashes)
{
	if(IsPipelineFileCacheEnabled())
	{
		FRWScopeLock Lock(FileCacheLock, SLT_Write);
		
		RequestedOrder = Order;
	
 		FPipelineCacheFile* FileCache = GetPipelineCacheFileFromKey(PSOCacheKey);

		if(FileCache)
		{
			FileCache->GetOrderedPSOHashes(PSOHashes, Order, MinBindCount, AlreadyCompiledHashes);
		}
	}
}

void FPipelineFileCacheManager::FetchPSODescriptors(const FString& PSOCacheKey, TDoubleLinkedList<FPipelineCacheFileFormatPSORead*>& Batch)
{
	if(IsPipelineFileCacheEnabled())
	{
		FRWScopeLock Lock(FileCacheLock, SLT_ReadOnly);
		FPipelineCacheFile* FileCache = GetPipelineCacheFileFromKey(PSOCacheKey);
		if(FileCache)
		{
			FileCache->FetchPSODescriptors(Batch);
		}
	}
}

struct FPipelineCacheFileData
{
	FPipelineCacheFileFormatHeader Header;
	TMap<uint32, FPipelineCacheFileFormatPSO> PSOs;
	FPipelineCacheFileFormatTOC TOC;
	
	static FPipelineCacheFileData Open(FString const& FilePath)
	{
		FPipelineCacheFileData Data;
		Data.Header.Magic = 0;
		FArchive* FileAReader = IFileManager::Get().CreateFileReader(*FilePath);
		if (FileAReader)
		{
			*FileAReader << Data.Header;
			if (Data.Header.Magic == FPipelineCacheFileFormatMagic && Data.Header.Version >= (uint32)EPipelineCacheFileFormatVersions::FirstWorking)
			{
                FileAReader->SetGameNetVer(Data.Header.Version);
				check(Data.Header.TableOffset > 0);
				FileAReader->Seek(Data.Header.TableOffset);
				
				*FileAReader << Data.TOC;
				if (!FileAReader->IsError())
				{
					for (auto& Entry : Data.TOC.MetaData)
					{
						if ( (Entry.Value.EngineFlags & FPipelineCacheFlagInvalidPSO) == 0 &&
                        	 Entry.Value.FileGuid == Data.Header.Guid &&
                        	 Entry.Value.FileSize > sizeof(FPipelineCacheFileFormatPSO::DescriptorType))
                        {
                            FPipelineCacheFileFormatPSO PSO;
                            FileAReader->Seek(Entry.Value.FileOffset);
                            *FileAReader << PSO;
							
#if PSO_COOKONLY_DATA
							// Tools get cook data populated into the PSO as the PSOs can be independant from Meta data
							if(Data.Header.Version >= (uint32)EPipelineCacheFileFormatVersions::PSOUsageMask)
							{
                            	PSO.UsageMask = Entry.Value.UsageMask;
							}
							if(Data.Header.Version >= (uint32)EPipelineCacheFileFormatVersions::PSOBindCount)
							{
								PSO.BindCount = Entry.Value.Stats.TotalBindCount;
							}
#endif
							Data.PSOs.Add(Entry.Key, PSO);
                        }
					}
				}
				
				if (FileAReader->IsError())
				{
                    UE_LOGF(LogRHI, Error, "Failed to read: %ls.", *FilePath);
					Data.Header.Magic = 0;
				}
				else
				{
					if (Data.Header.Version < (uint32)EPipelineCacheFileFormatVersions::ShaderMetaData)
					{
						for (auto& Entry : Data.TOC.MetaData)
						{
							FPipelineCacheFileFormatPSO& PSO = Data.PSOs.FindChecked(Entry.Key);
							switch(PSO.Type)
							{
								case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
									Entry.Value.Shaders.Add(PSO.ComputeDesc.ComputeShader);
									break;
								case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
									Entry.Value.Shaders.Add(PSO.GraphicsDesc.VertexShader);
									
									if (PSO.GraphicsDesc.FragmentShader != FShaderHash())
									{
										Entry.Value.Shaders.Add(PSO.GraphicsDesc.FragmentShader);
									}
									
									if (PSO.GraphicsDesc.GeometryShader != FShaderHash())
									{
										Entry.Value.Shaders.Add(PSO.GraphicsDesc.GeometryShader);
									}
									
									if (PSO.GraphicsDesc.MeshShader != FShaderHash())
									{
										Entry.Value.Shaders.Add(PSO.GraphicsDesc.MeshShader);
									}

									if (PSO.GraphicsDesc.AmplificationShader != FShaderHash())
									{
										Entry.Value.Shaders.Add(PSO.GraphicsDesc.AmplificationShader);
									}
									break;
								case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
									Entry.Value.Shaders.Add(PSO.RayTracingDesc.ShaderHash);
									break;
								default:
									check(false);
									break;
							}
						}
					}
					
					if (Data.Header.Version < (uint32)EPipelineCacheFileFormatVersions::SortedVertexDesc)
					{
						TMap<uint32, FPipelineCacheFileFormatPSOMetaData> MetaData;
						TMap<uint32, FPipelineCacheFileFormatPSO> PSOs;
						for (auto& Entry : Data.TOC.MetaData)
						{
							FPipelineCacheFileFormatPSO& PSO = Data.PSOs.FindChecked(Entry.Key);
							PSOs.Add(GetTypeHash(PSO), PSO);
							check(Entry.Value.FileGuid != FGuid());

							MetaData.Add(GetTypeHash(PSO), Entry.Value);
						}
						
						Data.TOC.MetaData = MetaData;
						Data.PSOs = PSOs;
					}
                    
                    Data.Header.Version = FPipelineCacheFileFormatCurrentVersion;
                }
			}
			
			FileAReader->Close();
			
			delete FileAReader;
		}
        else
        {
            UE_LOGF(LogRHI, Error, "Failed to open: %ls.", *FilePath);
        }
		return Data;
	}
};
		 
uint32 FPipelineFileCacheManager::NumPSOsLogged()
{
	uint32 Result = 0;
	if(IsPipelineFileCacheEnabled() && LogPSOtoFileCache())
	{
		// Only count PSOs that are both new and have at least one bind or have been marked invalid (compile failure) otherwise we can ignore them
		FRWScopeLock Lock(FileCacheLock, SLT_ReadOnly);
		
		// We now need to know if the number of usage masks changes - this number should be as least the same as before but could be conceptually more if an existing PSO has an extra usage mask applied
		if(NewPSOUsage.Num() > 0)
		{
			for(auto& MaskEntry : NewPSOUsage)
			{
				FPipelineStateStats const* Stat = Stats.FindRef(MaskEntry.Key);
				if ((Stat && Stat->TotalBindCount > 0) || (MaskEntry.Value.EngineFlags & FPipelineCacheFlagInvalidPSO) != 0)
				{
					Result++;
				}
			}
		}
		
		if(Result == 0 && NumNewPSOs > 0)
		{
			// This can happen if the Mask was zero at some point
			
			for (auto& PSOPair : NewPSOs)
			{
				FPipelineStateStats const* Stat = Stats.FindRef(PSOPair.Key);
				if (Stat && Stat->TotalBindCount > 0)
				{
					Result++;
				}
			}
		}
	}
	return Result;
}

FPipelineFileCacheManager::FPipelineStateLoggedEvent& FPipelineFileCacheManager::OnPipelineStateLogged()
{
	return PSOLoggedEvent;
}

bool FPipelineFileCacheManager::LoadPipelineFileCacheInto(FString const& Path, TSet<FPipelineCacheFileFormatPSO>& PSOs)
{
	FPipelineCacheFileData A = FPipelineCacheFileData::Open(Path);
	bool bAny = false;
	for (const auto& Pair : A.PSOs)
	{
		PSOs.Add(Pair.Value);
		bAny = true;
	}
	return bAny;
}

bool FPipelineFileCacheManager::SavePipelineFileCacheFrom(uint32 GameVersion, EShaderPlatform Platform, FString const& Path, const TSet<FPipelineCacheFileFormatPSO>& PSOs)
{
	FPipelineCacheFileData Output;
	Output.Header.Magic = FPipelineCacheFileFormatMagic;
	Output.Header.Version = FPipelineCacheFileFormatCurrentVersion;
	Output.Header.GameVersion = GameVersion;
	Output.Header.Platform = Platform;
	Output.Header.TableOffset = 0;
	Output.Header.Guid = FGuid::NewGuid();

	Output.TOC.MetaData.Reserve(PSOs.Num());

	for (const FPipelineCacheFileFormatPSO& Item : PSOs)
	{
		FPipelineCacheFileFormatPSOMetaData Meta;
		Meta.Stats.PSOHash = GetTypeHash(Item);
		Meta.FileGuid = Output.Header.Guid;
		Meta.FileSize = 0;
#if PSO_COOKONLY_DATA
		Meta.UsageMask = Item.UsageMask;
		Meta.Stats.TotalBindCount = Item.BindCount;
#endif
		switch (Item.Type)
		{
			case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
			{
				INC_DWORD_STAT(STAT_SerializedComputePipelineStateCount);
				Meta.Shaders.Add(Item.ComputeDesc.ComputeShader);
				break;
			}
			case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
			{
				INC_DWORD_STAT(STAT_SerializedGraphicsPipelineStateCount);

				if (Item.GraphicsDesc.VertexShader != FShaderHash())
					Meta.Shaders.Add(Item.GraphicsDesc.VertexShader);

				if (Item.GraphicsDesc.FragmentShader != FShaderHash())
					Meta.Shaders.Add(Item.GraphicsDesc.FragmentShader);

				if (Item.GraphicsDesc.GeometryShader != FShaderHash())
					Meta.Shaders.Add(Item.GraphicsDesc.GeometryShader);

				if (Item.GraphicsDesc.MeshShader != FShaderHash())
					Meta.Shaders.Add(Item.GraphicsDesc.MeshShader);

				if (Item.GraphicsDesc.AmplificationShader != FShaderHash())
					Meta.Shaders.Add(Item.GraphicsDesc.AmplificationShader);

				break;
			}
			case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
			{
				INC_DWORD_STAT(STAT_SerializedRayTracingPipelineStateCount);
				Meta.Shaders.Add(Item.RayTracingDesc.ShaderHash);
				break;
			}
			default:
			{
				check(false);
				break;
			}
		}

		check(Meta.FileGuid != FGuid());
		Output.TOC.MetaData.Add(Meta.Stats.PSOHash, Meta);
		Output.PSOs.Add(Meta.Stats.PSOHash, Item);
	}

	FArchive* FileWriter = IFileManager::Get().CreateFileWriter(*Path);
	if (!FileWriter)
	{
		return false;
	}
	FileWriter->SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);
	*FileWriter << Output.Header;

	uint64 PSOOffset = (uint64)FileWriter->Tell();

	for (auto& Entry : Output.TOC.MetaData)
	{
		FPipelineCacheFileFormatPSO& PSO = Output.PSOs.FindChecked(Entry.Key);

		uint32 PSOHash = Entry.Key;

		Entry.Value.FileOffset = PSOOffset;
		Entry.Value.FileGuid = Output.Header.Guid;

		TArray<uint8> Bytes;
		FMemoryWriter Wr(Bytes);
		Wr.SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);
		Wr << PSO;

		FileWriter->Serialize(Bytes.GetData(), Wr.TotalSize());

		Entry.Value.FileSize = Wr.TotalSize();
		PSOOffset += Entry.Value.FileSize;
	}

	FileWriter->Seek(0);

	Output.Header.TableOffset = PSOOffset;
	*FileWriter << Output.Header;

	FileWriter->Seek(PSOOffset);
	*FileWriter << Output.TOC;

	FileWriter->Flush();

	bool bOK = !FileWriter->IsError();

	FileWriter->Close();

	delete FileWriter;
	return bOK;
}


bool FPipelineFileCacheManager::MergePipelineFileCaches(FString const& PathA, FString const& PathB, FPipelineFileCacheManager::PSOOrder Order, FString const& OutputPath)
{
	bool bOK = false;
	
	FPipelineCacheFileData A = FPipelineCacheFileData::Open(PathA);
	FPipelineCacheFileData B = FPipelineCacheFileData::Open(PathB);
	
	if (A.Header.Magic == FPipelineCacheFileFormatMagic && B.Header.Magic == FPipelineCacheFileFormatMagic && A.Header.GameVersion == B.Header.GameVersion && A.Header.Platform == B.Header.Platform && A.Header.Version == FPipelineCacheFileFormatCurrentVersion && B.Header.Version == FPipelineCacheFileFormatCurrentVersion)
	{
		FPipelineCacheFileData Output;
		Output.Header.Magic = FPipelineCacheFileFormatMagic;
		Output.Header.Version = FPipelineCacheFileFormatCurrentVersion;
		Output.Header.GameVersion = A.Header.GameVersion;
		Output.Header.Platform = A.Header.Platform;
		Output.Header.TableOffset = 0;
		Output.Header.Guid = FGuid::NewGuid();
		
		uint32 MergeCount = 0;
		for (auto const& Entry : A.TOC.MetaData)
		{
			// Don't merge PSOs that have the invalid bit set
			if((Entry.Value.EngineFlags & FPipelineCacheFlagInvalidPSO) != 0)
			{
				continue;
			}

			Output.TOC.MetaData.Add(Entry.Key, Entry.Value);
		}
		for (auto const& Entry : B.TOC.MetaData)
		{
			// Don't merge PSOs that have the invalid bit set
			if((Entry.Value.EngineFlags & FPipelineCacheFlagInvalidPSO) != 0)
			{
				continue;
			}

			// Make sure these usage masks for the same PSOHash find their way in
			auto* ExistingMetaEntry = Output.TOC.MetaData.Find(Entry.Key);
			if(ExistingMetaEntry != nullptr)
			{
				ExistingMetaEntry->UsageMask |=  Entry.Value.UsageMask;
				ExistingMetaEntry->EngineFlags |= Entry.Value.EngineFlags;
				++MergeCount;
			}
			else
			{
				Output.TOC.MetaData.Add(Entry.Key, Entry.Value);
			}
		}
		
		FPipelineCacheFile::SortMetaData(Output.TOC.MetaData, Order);
		Output.TOC.SortedOrder = Order;

		FArchive* FileWriter = IFileManager::Get().CreateFileWriter(*OutputPath);
		if (FileWriter)
		{
            FileWriter->SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);
			FileWriter->Seek(0);
			*FileWriter << Output.Header;
			
			uint64 PSOOffset = (uint64)FileWriter->Tell();
			
            TSet<uint32> HashesToRemove;
            
			for (auto& Entry : Output.TOC.MetaData)
			{
				FPipelineCacheFileFormatPSO PSO;
				if (Entry.Value.FileGuid == A.Header.Guid)
				{
					PSO = A.PSOs.FindChecked(Entry.Key);
				}
				else if (Entry.Value.FileGuid == B.Header.Guid)
				{
					PSO = B.PSOs.FindChecked(Entry.Key);
				}
                else
                {
                    HashesToRemove.Add(Entry.Key);
                    continue;
                }
				
				uint32 PSOHash = Entry.Key;
				
				Entry.Value.FileOffset = PSOOffset;
				Entry.Value.FileGuid = Output.Header.Guid;
				
				TArray<uint8> Bytes;
				FMemoryWriter Wr(Bytes);
				Wr.SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);
				Wr << PSO;
				
				FileWriter->Serialize(Bytes.GetData(), Wr.TotalSize());
				
				Entry.Value.FileSize = Wr.TotalSize();
				PSOOffset += Entry.Value.FileSize;
			}
            
            for (uint32 Key : HashesToRemove)
            {
                Output.TOC.MetaData.Remove(Key);
            }
			
			FileWriter->Seek(0);
			
			Output.Header.TableOffset = PSOOffset;
			*FileWriter << Output.Header;
			
			FileWriter->Seek(PSOOffset);
			*FileWriter << Output.TOC;
			
			FileWriter->Flush();
			
			bOK = !FileWriter->IsError();
            
            UE_CLOGF(!bOK, LogRHI, Error, "Failed to write output file: %ls.", *OutputPath);
			
			FileWriter->Close();
			
			delete FileWriter;
		}
        else
        {
            UE_LOGF(LogRHI, Error, "Failed to open output file: %ls.", *OutputPath);
        }
	}
    else if (A.Header.GameVersion != B.Header.GameVersion)
    {
        UE_LOGF(LogRHI, Error, "Incompatible game versions: %u vs. %u.", A.Header.GameVersion, B.Header.GameVersion);
    }
    else if (A.Header.Platform != B.Header.Platform)
    {
        UE_LOGF(LogRHI, Error, "Incompatible shader platforms: %ls vs. %ls.", *LegacyShaderPlatformToShaderFormat(A.Header.Platform).ToString(), *LegacyShaderPlatformToShaderFormat(B.Header.Platform).ToString());
    }
    else if (A.Header.Version != B.Header.Version)
    {
        UE_LOGF(LogRHI, Error, "Incompatible file versions: %u vs. %u.", A.Header.Version, B.Header.Version);
    }
    else
    {
        UE_LOG(LogRHI, Error, TEXT("Incompatible file headers: %" UINT64_FMT " vs. %" UINT64_FMT ": expected %" UINT64_FMT "."), A.Header.Magic, B.Header.Magic, FPipelineCacheFileFormatMagic);
    }
	return bOK;
}

FPipelineCacheFileFormatPSO::FPipelineFileCacheRayTracingDesc::FPipelineFileCacheRayTracingDesc(const FRayTracingPipelineStateInitializer& Initializer, const FRHIRayTracingShader* ShaderRHI)
: ShaderHash(ShaderRHI->GetHash())
, Frequency(ShaderRHI->GetFrequency())
, ShaderBindingLayout(Initializer.ShaderBindingLayout ? *Initializer.ShaderBindingLayout : FRHIShaderBindingLayout())
{
}

FString FPipelineCacheFileFormatPSO::FPipelineFileCacheRayTracingDesc::HeaderLine() const
{
	FString Result("RayTracingShader,DeprecatedMaxPayloadSizeInBytes,Frequency,ShaderBindingLayout-Hash,ShaderBindingLayout-Flags,ShaderBindingLayout-NumUBEntries");
	for (int32 Index = 0; Index < FRHIShaderBindingLayout::MaxUniformBufferEntries; Index++)
	{
		Result += FString::Printf(TEXT(",%s%d,%s%d,%s%d,%s%d,%s%d,%s%d")
			, TEXT("ShaderBindingLayout-UBEntry-Name"), Index
			, TEXT("ShaderBindingLayout-UBEntry-RegisterSpace"), Index
			, TEXT("ShaderBindingLayout-UBEntry-CBVResourceIndex"), Index
			, TEXT("ShaderBindingLayout-UBEntry-BaseSRVResourceIndex"), Index
			, TEXT("ShaderBindingLayout-UBEntry-BaseUAVResourceIndex"), Index
			, TEXT("ShaderBindingLayout-UBEntry-BaseSamplerResourceIndex"), Index
		);
	}

	return Result;
}

FString FPipelineCacheFileFormatPSO::FPipelineFileCacheRayTracingDesc::ToString() const
{
	FString Result = FString::Printf(TEXT("%s,%d,%d,%d,%d,%d")
		, *ShaderHash.ToString()
		, DeprecatedMaxPayloadSizeInBytes
		, uint32(Frequency)
		, ShaderBindingLayout.GetHash()
		, uint32{EnumToUnderlyingType(ShaderBindingLayout.GetFlags())}
		, ShaderBindingLayout.GetNumUniformBufferEntries()
	);

	for (uint32 Index = 0; Index < FRHIShaderBindingLayout::MaxUniformBufferEntries; Index++)
	{
		FRHIUniformBufferShaderBindingLayout Entry = Index < ShaderBindingLayout.GetNumUniformBufferEntries() ? ShaderBindingLayout.GetUniformBufferEntry(Index) : FRHIUniformBufferShaderBindingLayout();
		Result += FString::Printf(TEXT(",%s,%d,%d,%d,%d,%d")
			, Entry.LayoutName.Len() > 0 ? *Entry.LayoutName : TEXT("None")
			, Entry.RegisterSpace
			, Entry.CBVResourceIndex
			, Entry.BaseSRVResourceIndex
			, Entry.BaseUAVResourceIndex
			, Entry.BaseSamplerResourceIndex
		);
	}
	return Result;
}

void FPipelineCacheFileFormatPSO::FPipelineFileCacheRayTracingDesc::AddToReadableString(TReadableStringBuilder& OutBuilder) const
{
	// TODO: probably needs a better implementation once we get to this
	switch (Frequency)
	{
		case SF_RayGen:
			OutBuilder << TEXT(" RGS:");
			break;
		case SF_RayCallable:
			OutBuilder << TEXT(" RCS:");
			break;
		case SF_RayHitGroup:
			OutBuilder << TEXT(" RHGS:");
			break;
		case SF_RayMiss:
			OutBuilder << TEXT(" RMS:");
			break;
	}
	OutBuilder << ShaderHash.ToString();
	OutBuilder << TEXT(" AHGI ");
}

void FPipelineCacheFileFormatPSO::FPipelineFileCacheRayTracingDesc::FromString(const FString& Src)
{
	TArray<FString> Parts;
	Src.TrimStartAndEnd().ParseIntoArray(Parts, TEXT(","));

	// make sure we have required number of parts
	if (Parts.Num() != (6 + 6 * FRHIShaderBindingLayout::MaxUniformBufferEntries))
	{
		return;
	}

	ShaderHash.FromString(Parts[0]);

	// Not used, but kept for back-compatibility
	LexFromString(DeprecatedMaxPayloadSizeInBytes, Parts[1]);

	uint32 Temp = 0;
	LexFromString(Temp, Parts[2]);
	Frequency = EShaderFrequency(Temp);

	uint32 ShaderBiningLayoutHash;
	LexFromString(ShaderBiningLayoutHash, Parts[3]);

	LexFromString(Temp, Parts[4]);
	EShaderBindingLayoutFlags ShaderBindingLayoutFlags = (EShaderBindingLayoutFlags) Temp;

	uint32 ShaderBindingLayoutNumUniformBuffers;
	LexFromString(ShaderBindingLayoutNumUniformBuffers, Parts[5]);

	uint32 CurrentPartIndex = 6;
	TArray<FRHIUniformBufferShaderBindingLayout> ShaderBindingLayoutUBEntries;
	ShaderBindingLayoutUBEntries.SetNum(ShaderBindingLayoutNumUniformBuffers);
	for (uint32 UBEntryIndex = 0; UBEntryIndex < ShaderBindingLayoutNumUniformBuffers; ++UBEntryIndex)
	{
		FRHIUniformBufferShaderBindingLayout& UBEntry = ShaderBindingLayoutUBEntries[UBEntryIndex];
		UBEntry.LayoutName = Parts[CurrentPartIndex++];
		LexFromString(Temp, Parts[CurrentPartIndex++]);
		UBEntry.RegisterSpace = Temp;
		LexFromString(Temp, Parts[CurrentPartIndex++]);
		UBEntry.CBVResourceIndex = Temp;
		LexFromString(Temp, Parts[CurrentPartIndex++]);
		UBEntry.BaseSRVResourceIndex = Temp;
		LexFromString(Temp, Parts[CurrentPartIndex++]);
		UBEntry.BaseUAVResourceIndex = Temp;
		LexFromString(Temp, Parts[CurrentPartIndex++]);
		UBEntry.BaseSamplerResourceIndex = Temp;
	}	

	ShaderBindingLayout = FRHIShaderBindingLayout(ShaderBindingLayoutFlags, ShaderBindingLayoutUBEntries);
	check(ShaderBindingLayout.GetHash() == ShaderBiningLayoutHash);
}

bool FPipelineCacheFileFormatPSO::Init(FPipelineCacheFileFormatPSO& PSO, FPipelineCacheFileFormatPSO::FPipelineFileCacheRayTracingDesc const& Desc)
{
	PSO.Type = DescriptorType::RayTracing;

#if PSO_COOKONLY_DATA
	PSO.UsageMask = 0;
	PSO.BindCount = 0;
#endif

	PSO.RayTracingDesc = Desc;

	return true;
}

