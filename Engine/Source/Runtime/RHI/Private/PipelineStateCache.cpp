// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
PipelineStateCache.cpp: Pipeline state cache implementation.
=============================================================================*/

#include "PipelineStateCache.h"
#include "Async/AsyncWork.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Deque.h"
#include "PipelineFileCache.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/App.h"
#include "Misc/TimeGuard.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RHICommandList.h"
#include "RHIFwd.h"
#include "RHIBreadcrumbs.h"
#include "Stats/StatsTrace.h"
#include "Templates/TypeHash.h"
#include "Misc/QueuedThreadPoolWrapper.h"

// 5.4.2 local change to avoid modifying public headers
namespace PipelineStateCache
{
	// Waits for any pending tasks to complete.
	extern RHI_API void WaitForAllTasks();

}

// perform cache eviction each frame, used to stress the system and flush out bugs
#define PSO_DO_CACHE_EVICT_EACH_FRAME 0

// Log event and info about cache eviction
#define PSO_LOG_CACHE_EVICT 0

// Stat tracking
#define PSO_TRACK_CACHE_STATS 0

#define PIPELINESTATECACHE_VERIFYTHREADSAFE (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

CSV_DECLARE_CATEGORY_EXTERN(PSO);
CSV_DEFINE_CATEGORY_MODULE(RHI_API, PSOPrecacheCompiling, false);

DEFINE_LOG_CATEGORY_STATIC(LogPSOHitching, Log, All);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Runtime Graphics PSO Hitch Count"), STAT_RuntimeGraphicsPSOHitchCount, STATGROUP_PipelineStateCache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Runtime Compute PSO Hitch Count"), STAT_RuntimeComputePSOHitchCount, STATGROUP_PipelineStateCache);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total Precached Graphics PSOs"), STAT_TotalGraphicsPrecachedPSOs, STATGROUP_PipelineStateCache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total Precached Compute PSOs"), STAT_TotalComputePrecachedPSOs, STATGROUP_PipelineStateCache);

DECLARE_STATS_GROUP(TEXT("PSOPrecache InFlight"), STATGROUP_PSOPrecacheInFlight, STATCAT_Advanced);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Active Graphics PSO Precache Requests"), STAT_ActiveGraphicsPSOPrecacheRequests, STATGROUP_PSOPrecacheInFlight);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Active Compute PSO Precache Requests"), STAT_ActiveComputePSOPrecacheRequests, STATGROUP_PSOPrecacheInFlight);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Highest Priority Graphics PSO Precache Requests"), STAT_HighestPriorityGraphicsPSOPrecacheRequests, STATGROUP_PSOPrecacheInFlight);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Highest Priority Compute PSO Precache Requests"), STAT_HighestPriorityComputePSOPrecacheRequests, STATGROUP_PSOPrecacheInFlight);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("High Priority Graphics PSO Precache Requests"), STAT_HighPriorityGraphicsPSOPrecacheRequests, STATGROUP_PSOPrecacheInFlight);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("High Priority Compute PSO Precache Requests"), STAT_HighPriorityComputePSOPrecacheRequests, STATGROUP_PSOPrecacheInFlight);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Medium Priority Graphics PSO Precache Requests"), STAT_MediumPriorityGraphicsPSOPrecacheRequests, STATGROUP_PSOPrecacheInFlight);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Medium Priority Compute PSO Precache Requests"), STAT_MediumPriorityComputePSOPrecacheRequests, STATGROUP_PSOPrecacheInFlight);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Low Priority Graphics PSO Precache Requests"), STAT_LowPriorityGraphicsPSOPrecacheRequests, STATGROUP_PSOPrecacheInFlight);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Low Priority Compute PSO Precache Requests"), STAT_LowPriorityComputePSOPrecacheRequests, STATGROUP_PSOPrecacheInFlight);

DECLARE_STATS_GROUP(TEXT("PSOPrecache InMemory"), STATGROUP_PSOPrecacheInMemory, STATCAT_Advanced);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Precached Graphics PSOs InMemory"), STAT_PrecachedGraphicsPrecachedPSOsInMemory, STATGROUP_PSOPrecacheInMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Precached Compute PSOs InMemory"), STAT_PrecachedComputePrecachedPSOsInMemory, STATGROUP_PSOPrecacheInMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Runtime Graphics PSOs"), STAT_ActiveRuntimeGraphicsPSOs, STATGROUP_PSOPrecacheInMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Runtime Compute PSOs"), STAT_ActiveRuntimeComputePSOs, STATGROUP_PSOPrecacheInMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Runtime Graphics PSOs From InMemoryCache"), STAT_RuntimeGraphicsPSOsFromInMemoryPSOCache, STATGROUP_PSOPrecacheInMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Runtime Compute PSOs From InMemoryCache"), STAT_RuntimeComputePSOsFromInMemoryPSOCache, STATGROUP_PSOPrecacheInMemory);

static inline uint32 GetTypeHash(const FBoundShaderStateInput& Input)
{
	uint32 Hash = GetTypeHash(Input.VertexDeclarationRHI);
	Hash = HashCombineFast(Hash, GetTypeHash(Input.VertexShaderRHI));
	Hash = HashCombineFast(Hash, GetTypeHash(Input.VertexShaderSpecializationConstants));
	Hash = HashCombineFast(Hash, GetTypeHash(Input.PixelShaderRHI));
	Hash = HashCombineFast(Hash, GetTypeHash(Input.PixelShaderSpecializationConstants));
#if PLATFORM_SUPPORTS_MESH_SHADERS
	Hash = HashCombineFast(Hash, GetTypeHash(Input.GetMeshShader()));
	Hash = HashCombineFast(Hash, GetTypeHash(Input.MeshShaderSpecializationConstants));
	Hash = HashCombineFast(Hash, GetTypeHash(Input.GetAmplificationShader()));
	Hash = HashCombineFast(Hash, GetTypeHash(Input.AmplificationShaderSpecializationConstants));
#endif
#if PLATFORM_SUPPORTS_WORKGRAPH_SHADERS
	Hash = HashCombineFast(Hash, GetTypeHash(Input.GetWorkGraphShader()));
#endif
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	Hash = HashCombineFast(Hash, GetTypeHash(Input.GetGeometryShader()));
	Hash = HashCombineFast(Hash, GetTypeHash(Input.GeometryShaderSpecializationConstants));
#endif
	return Hash;
}

inline uint32 GetTypeHash(const FExclusiveDepthStencil& Ds)
{
	return GetTypeHash(Ds.Value);
}

static inline uint32 GetTypeHash(const FComputePipelineStateInitializer& Initializer)
{
	uint32 Hash = GetTypeHash(Initializer.ComputeShader);
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.SpecializationConstants));
	return Hash;
}

static inline uint32 GetTypeHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	uint32 Hash = GetTypeHash(Initializer.BoundShaderState);
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.BlendState));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.RasterizerState));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.DepthStencilState));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.PrimitiveType));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.RenderTargetsEnabled));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.RenderTargetFormats));
	for (int32 Index = 0; Index < Initializer.RenderTargetFlags.Num(); ++Index)
	{
		Hash = HashCombineFast(Hash, GetTypeHash(Initializer.RenderTargetFlags[Index] & FGraphicsPipelineStateInitializer::RelevantRenderTargetFlagMask));
	}
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.DepthStencilTargetFormat));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.DepthStencilTargetFlag & FGraphicsPipelineStateInitializer::RelevantDepthStencilFlagMask));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.DepthTargetLoadAction));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.DepthTargetStoreAction));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.StencilTargetLoadAction));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.StencilTargetStoreAction));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.DepthStencilAccess));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.NumSamples));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.SubpassHint));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.SubpassIndex));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.ConservativeRasterization));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.bDepthBounds));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.MultiViewCount));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.bHasFragmentDensityAttachment));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.bAllowVariableRateShading));
	Hash = HashCombineFast(Hash, GetTypeHash(Initializer.ShadingRate));
	return Hash;
}

constexpr int32 PSO_MISS_FRAME_HISTORY_SIZE = 3;
static TAtomic<uint32> GraphicsPipelineCacheMisses;
static TArray<uint32> GraphicsPipelineCacheMissesHistory;
static TAtomic<uint32> ComputePipelineCacheMisses;
static TArray<uint32> ComputePipelineCacheMissesHistory;
static bool	ReportFrameHitchThisFrame;

enum class EPSOCompileAsyncMode
{
	None = 0,
	All = 1,
	Precompile = 2,
	NonPrecompiled = 3,
};

static TAutoConsoleVariable<int32> GCVarAsyncPipelineCompile(
	TEXT("r.AsyncPipelineCompile"),
	(int32)EPSOCompileAsyncMode::All,
	TEXT("0 to Create PSOs at the moment they are requested\n")
	TEXT("1 to Create Pipeline State Objects asynchronously(default)\n")
	TEXT("2 to Create Only precompile PSOs asynchronously\n")
	TEXT("3 to Create Only non-precompile PSOs asynchronously")
	,
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

bool GRunPSOCreateTasksOnRHIT = false;
static FAutoConsoleVariableRef CVarCreatePSOsOnRHIThread(
	TEXT("r.pso.CreateOnRHIThread"),
	GRunPSOCreateTasksOnRHIT,
	TEXT("0: Run PSO creation on task threads\n")
	TEXT("1: Run PSO creation on RHI thread."),
	ECVF_RenderThreadSafe
);

bool GEnablePSOAsyncCacheConsolidation = true;
static FAutoConsoleVariableRef CVarEnablePSOAsyncCacheConsolidation(
	TEXT("r.pso.EnableAsyncCacheConsolidation"),
	GEnablePSOAsyncCacheConsolidation,
	TEXT("0: Require Render Thread and RHI Thread to synchronize before flushing the PSO cache.")
	TEXT("1: Flush the PSO cache without synchronizing the Render Thread with the RHI Thread.\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarPSOEvictionTime(
	TEXT("r.pso.evictiontime"),
	60,
	TEXT("Time between checks to remove stale objects from the cache. 0 = no eviction (which may eventually OOM...)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarPSORuntimeCreationHitchThreshold(
	TEXT("r.PSO.RuntimeCreationHitchThreshold"),
	20,
	TEXT("Threshold for runtime PSO creation to count as a hitch (in msec) (default 20)"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

#if RHI_RAYTRACING
static TAutoConsoleVariable<int32> CVarRTPSOCacheSize(
	TEXT("r.RayTracing.PSOCacheSize"),
	50,
	TEXT("Number of ray tracing pipelines to keep in the cache (default = 50). Set to 0 to disable eviction.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);
#endif // RHI_RAYTRACING

int32 GPSOPrecaching = 1;
static FAutoConsoleVariableRef CVarPSOPrecaching(
	TEXT("r.PSOPrecaching"),
	GPSOPrecaching,
	TEXT("0 to Disable PSOs precaching\n")
	TEXT("1 to Enable PSO precaching\n"),
	ECVF_Default
);

int32 GPSOPrecacheMinPriorityForActivePrecacheRequests = 0;
static FAutoConsoleVariableRef CVarPSOWaitForHighPriorityRequestsOnly(
	TEXT("r.PSOPrecaching.MinPriorityForActivePrecacheRequests"),
	GPSOPrecacheMinPriorityForActivePrecacheRequests,
	TEXT("Priority of active request to take into account when computing the number of active PSOs (can be used in loading screen waits for example).\n")
	TEXT("0: Add all pending PSO precache requests (default)\n")
	TEXT("1: Medium priority and above only\n")
	TEXT("2: High priority and above only\n")
	TEXT("3: Highest priority only\n"),
	ECVF_Default
);

bool GPSOPrecachePermitPriorityEscalation = true;
static FAutoConsoleVariableRef CVarPSOPrecachePermitPriorityEscalation(
	TEXT("r.PSOPrecaching.PermitPriorityEscalation"),
	GPSOPrecachePermitPriorityEscalation,
	TEXT("Whether to permit requests to increase high pri PSO precaching tasks to highest.\n")
	TEXT("1: High priority tasks can be escalated to highest if requested. (default)\n")
	TEXT("0: High priority tasks will remain unchanged."),
	ECVF_Default
);

extern void DumpPipelineCacheStats();

static FAutoConsoleCommand DumpPipelineCmd(
	TEXT("r.DumpPipelineCache"),
	TEXT("Dump current cache stats."),
	FConsoleCommandDelegate::CreateStatic(DumpPipelineCacheStats)
);

int32 GPSOPrecacheProcessPriorityThreshold = 0;
static FAutoConsoleVariableRef CVarPSOOnlyPrecacheMaxPriority(
	TEXT("r.PSOPrecaching.TaskPriorityThreshold"),
	GPSOPrecacheProcessPriorityThreshold,
	TEXT("The precache task priority threshold below which tasks are not processed.\n")
	TEXT("0: All compile tasks are processed. (default)\n")
	TEXT("1: Medium priority compile tasks and above are processed.\n")
	TEXT("2: High priority compile tasks and above are processed.\n")
	TEXT("3: Only highest priority compile tasks are processed."),
	ECVF_Default
);
static_assert((int)FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType::NumTypes == 4, "EPSOPrecacheCompileType has changed, 'r.PSOPrecaching.TaskPriorityThreshold' must be updated.");

int32 GPSOPrecacheUnhealthyCacheHitchThresholdMs = 80;
static FAutoConsoleVariableRef CVarPSOPrecacheUnhealthyCacheHitchThresholdMs(
	TEXT("r.PSOPrecache.UnhealthyCacheHitchThresholdMs"),
	GPSOPrecacheUnhealthyCacheHitchThresholdMs,
	TEXT("Threshold for runtime previously-precached PSO creation to count towards suspected unhealthy driver cache behavior, in milliseconds (default 80)."),
	ECVF_Default
);

int32 GPSOPrecacheUnhealthyCacheMaxHitches = 100;
static FAutoConsoleVariableRef CVarPSOPrecacheUnhealthyCacheMaxHitches(
	TEXT("r.PSOPrecache.UnhealthyCacheMaxHitches"),
	GPSOPrecacheUnhealthyCacheMaxHitches,
	TEXT("Threshold for the number of runtime previously-precached PSO creation hitches to set a flag indicating a suspected unhealthy driver cache (default 100)."),
	ECVF_Default
);

static std::atomic<uint32> RuntimePSOCreationCount = 0;
static std::atomic<uint32> TotalPSOCreationHitchCount = 0;
static std::atomic<uint32> GraphicsPSOCreationHitchCount = 0;
static std::atomic<uint32> ComputePSOCreationHitchCount = 0;
static std::atomic<uint32> PrecachedPSOCreationHitchCount = 0;
static std::atomic<uint32> UnhealthyDriverCachePSOCreationHitchCount = 0;
static std::atomic<bool> bDriverCacheSuspectedUnhealthy = false;

struct FPSOCompilationDebugData
{
	const TCHAR* PassName = nullptr;
	FString ResourceName;
#if WITH_RHI_BREADCRUMBS
	FRHIBreadcrumbNode const* BreadcrumbRoot = nullptr;
	FRHIBreadcrumbNode const* BreadcrumbNode = nullptr;

	// Adding Shared ref to BreadcrumbNodeAllocator to keep it alive. During the warming of the PSO cache at launch, the RHICmdLis doesn't wait for the PSO to finish compiling.
	TSharedPtr<const FRHIBreadcrumbAllocator> BreadcrumbNodeAllocator;
#endif // WITH_RHI_BREADCRUMBS
};

static inline uint64 CheckAndUpdateHitchCountStat(FPSOPrecacheRequestID::EType PSOType, bool bIsRuntimePSO, const FPSOCompilationDebugData& PSOCompilationDebugData, uint64 StartTime, EPSOPrecacheResult PSOPrecacheResult)
{
	uint64 PSOCompilationTime = FPlatformTime::Cycles64() - StartTime;
	if (!bIsRuntimePSO)
	{
		return PSOCompilationTime;
	}

	RuntimePSOCreationCount.fetch_add(1, std::memory_order_relaxed);

	int32 RuntimePSOCreationHitchThreshold = CVarPSORuntimeCreationHitchThreshold.GetValueOnAnyThread();
	double PSOCreationTimeMs = FPlatformTime::ToMilliseconds64(PSOCompilationTime);
	if (PSOCreationTimeMs > RuntimePSOCreationHitchThreshold)
	{
		if (PSOType == FPSOPrecacheRequestID::EType::Graphics)
		{
#if WITH_RHI_BREADCRUMBS
			UE_LOGF(LogPSOHitching, Verbose, "Runtime graphics PSO creation hitch (%.2f msec) for Resource %ls in Pass %ls (precache status: %ls) - Breadcrumbs: %ls", 
				   PSOCreationTimeMs, *PSOCompilationDebugData.ResourceName, PSOCompilationDebugData.PassName ? PSOCompilationDebugData.PassName : TEXT("Unknown"), LexToString(PSOPrecacheResult), PSOCompilationDebugData.BreadcrumbNode ? *PSOCompilationDebugData.BreadcrumbNode->GetFullPath() : TEXT("Unknown"));
#else
			UE_LOGF(LogPSOHitching, Verbose, "Runtime graphics PSO creation hitch (%.2f msec) for Resource %ls in Pass %ls (precache status: %ls)", 
				   PSOCreationTimeMs, *PSOCompilationDebugData.ResourceName, PSOCompilationDebugData.PassName ? PSOCompilationDebugData.PassName : TEXT("Unknown"), LexToString(PSOPrecacheResult));
#endif // WITH_RHI_BREADCRUMBS
			INC_DWORD_STAT(STAT_RuntimeGraphicsPSOHitchCount);
			CSV_CUSTOM_STAT(PSO, GraphicsPSOHitch, 1, ECsvCustomStatOp::Accumulate);
			CSV_CUSTOM_STAT(PSO, GraphicsPSOHitchTime, PSOCreationTimeMs, ECsvCustomStatOp::Accumulate);
			GraphicsPSOCreationHitchCount.fetch_add(1, std::memory_order_relaxed);
		}
		else if (PSOType == FPSOPrecacheRequestID::EType::Compute)
		{
#if WITH_RHI_BREADCRUMBS
			UE_LOGF(LogPSOHitching, Verbose, "Runtime compute PSO creation hitch (%.2f msec) for Resource %ls in Pass %ls (precache status: %ls) - Breadcrumbs: %ls", 
				   PSOCreationTimeMs, *PSOCompilationDebugData.ResourceName, PSOCompilationDebugData.PassName ? PSOCompilationDebugData.PassName : TEXT("Unknown"), LexToString(PSOPrecacheResult), PSOCompilationDebugData.BreadcrumbNode ? *PSOCompilationDebugData.BreadcrumbNode->GetFullPath() : TEXT("Unknown"));
#else
			UE_LOGF(LogPSOHitching, Verbose, "Runtime compute PSO creation hitch (%.2f msec) for Resource %ls in Pass %ls (precache status: %ls)", 
				   PSOCreationTimeMs, *PSOCompilationDebugData.ResourceName, PSOCompilationDebugData.PassName ? PSOCompilationDebugData.PassName : TEXT("Unknown"), LexToString(PSOPrecacheResult));
#endif // WITH_RHI_BREADCRUMBS
			INC_DWORD_STAT(STAT_RuntimeComputePSOHitchCount);
			CSV_CUSTOM_STAT(PSO, ComputePSOHitch, 1, ECsvCustomStatOp::Accumulate);
			CSV_CUSTOM_STAT(PSO, ComputePSOHitchTime, PSOCreationTimeMs, ECsvCustomStatOp::Accumulate);
			ComputePSOCreationHitchCount.fetch_add(1, std::memory_order_relaxed);
		}

		// Hitches coming from precached PSOs are more concerning. Try to detect whether
		// the underlying driver cache is unhealthy.
		if (PSOPrecacheResult == EPSOPrecacheResult::Complete)
		{
			PrecachedPSOCreationHitchCount.fetch_add(1, std::memory_order_relaxed);

			if (PSOCreationTimeMs > GPSOPrecacheUnhealthyCacheHitchThresholdMs)
			{
				int32 CurrentDriverCacheHitchCount = UnhealthyDriverCachePSOCreationHitchCount.fetch_add(1, std::memory_order_relaxed) + 1;

				if (CurrentDriverCacheHitchCount == GPSOPrecacheUnhealthyCacheMaxHitches)
				{
					UE_LOGF(LogPSOHitching, Warning, "Encountered %d PSO creation hitches (threshold: %dms) from PSOs that were precached. This could be an indication of an unhealthy driver cache.", CurrentDriverCacheHitchCount, GPSOPrecacheUnhealthyCacheHitchThresholdMs);
					bDriverCacheSuspectedUnhealthy.store(true, std::memory_order_relaxed);
				}
			}
		}

		uint32 TotalHitches = TotalPSOCreationHitchCount.fetch_add(1, std::memory_order_relaxed) + 1;
		if (TotalHitches > 0 && TotalHitches % 50 == 0)
		{
			// The numbers logged here might be slightly out-of-sync but it's good enough for a general picture.
			UE_LOGF(LogPSOHitching, Log, "Encountered %d PSO creation hitches so far (%d graphics, %d compute). %d of them were precached.", TotalHitches, 
				GraphicsPSOCreationHitchCount.load(std::memory_order_relaxed), ComputePSOCreationHitchCount.load(std::memory_order_relaxed), PrecachedPSOCreationHitchCount.load(std::memory_order_relaxed));
		}
	}

	return PSOCompilationTime;
}

static int32 GPSOPrecompileThreadPoolSize = 0;
static FAutoConsoleVariableRef GPSOPrecompileThreadPoolSizeVar(
	TEXT("r.pso.PrecompileThreadPoolSize"),
	GPSOPrecompileThreadPoolSize,
	TEXT("The number of threads available for concurrent PSO Precompiling.\n")
	TEXT("0 to disable threadpool usage when precompiling PSOs. (default)")
	,
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

static int32 GPSOPrecompileThreadPoolPercentOfHardwareThreads = 75;
static FAutoConsoleVariableRef GPSOPrecompileThreadPoolPercentOfHardwareThreadsVar(
	TEXT("r.pso.PrecompileThreadPoolPercentOfHardwareThreads"),
	GPSOPrecompileThreadPoolPercentOfHardwareThreads,
	TEXT("If > 0, use this percentage of cores (rounded up) for the PSO precompile thread pool\n")
	TEXT("Use this as an alternative to r.pso.PrecompileThreadPoolSize\n")
	TEXT("0 to disable threadpool usage when precompiling PSOs. (default 75%)")
	,
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

static int32 GPSOPrecompileThreadPoolSizeMin = 2;
static FAutoConsoleVariableRef GPSOPrecompileThreadPoolSizeMinVar(
	TEXT("r.pso.PrecompileThreadPoolSizeMin"),
	GPSOPrecompileThreadPoolSizeMin,
	TEXT("The minimum number of threads available for concurrent PSO Precompiling.\n")
	TEXT("Ignored unless r.pso.PrecompileThreadPoolPercentOfHardwareThreads is specified\n")
	TEXT("0 = no minimum (default 2)")
	,
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

static int32 GPSOPrecompileThreadPoolSizeMax = INT_MAX;
static FAutoConsoleVariableRef GPSOPrecompileThreadPoolSizeMaxVar(
	TEXT("r.pso.PrecompileThreadPoolSizeMax"),
	GPSOPrecompileThreadPoolSizeMax,
	TEXT("The maximum number of threads available for concurrent PSO Precompiling.\n")
	TEXT("Ignored unless r.pso.PrecompileThreadPoolPercentOfHardwareThreads is specified\n")
	TEXT("Default is no maximum (INT_MAX)")
	,
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

int32 GPSOPrecompileThreadPoolThreadPriority = (int32)EThreadPriority::TPri_BelowNormal;
static FAutoConsoleVariableRef CVarPrecompileThreadPoolThreadPriority(
	TEXT("r.pso.PrecompileThreadPoolThreadPriority"),
	GPSOPrecompileThreadPoolThreadPriority,
	TEXT("Thread priority for the PSO precompile pool"),
	ECVF_RenderThreadSafe);

int32 GPSOPrecacheBoostThreadPriorityForHighestPriorityRequests = 0;
static FAutoConsoleVariableRef CVarPSOPrecacheBoostThreadPriorityForHighestPriorityRequests(
	TEXT("r.PSOPrecache.BoostThreadPriorityForHighestPriorityRequests"),
	GPSOPrecacheBoostThreadPriorityForHighestPriorityRequests,
	TEXT("Boost compilation thread priority when compiling Highest priority PSO precaching requests to reduce fallback rendering time (can have influence on foreground tasks then)"),
	ECVF_RenderThreadSafe);

int32 GPSOPrecacheKeepInMemoryUntilUsed = 0;
static FAutoConsoleVariableRef CVarPSOPrecacheKeepInMemoryUntilUsed(
	TEXT("r.PSOPrecache.KeepInMemoryUntilUsed"),
	GPSOPrecacheKeepInMemoryUntilUsed,
	TEXT("If enabled and if the underlying GPU vendor is NVIDIA or Qualcomm, precached PSOs will be kept in memory instead of being deleted immediately after creation, and will only be deleted once they are actually used for rendering or when the maximum limit of in-memory PSOs is reached.\n")
	TEXT("This can speed up the re-creation of precached PSOs for NVIDIA and Qualcomm drivers and avoid small hitches, at the cost of memory.\n")
	TEXT("It's recommended to set r.PSOPrecache.KeepInMemoryGraphicsMaxNum and r.PSOPrecache.KeepInMemoryComputeMaxNum to a non-zero value to ensure the number of in-memory PSOs is bounded.\n")
	TEXT("Valid options:\n")
	TEXT("0 = off (default)\n")
	TEXT("1 = PSOs are kept in memory after precaching but not deleted immediately when used by the renderer. They are instead only evicted when the limit of in-memory PSOs is reached\n")
	TEXT("2 = PSOs are kept in memory after precaching and deleted when used by the renderer"),
	ECVF_ReadOnly);

int32 GPSOPrecacheKeepInMemoryGraphicsMaxNum = 2000;
static FAutoConsoleVariableRef CVarPSOPrecacheKeepInMemoryGraphicsMaxNum(
	TEXT("r.PSOPrecache.KeepInMemoryGraphicsMaxNum"),
	GPSOPrecacheKeepInMemoryGraphicsMaxNum,
	TEXT("If r.PSOPrecache.KeepInMemoryUntilUsed is enabled, this value will control the maximum number of precached graphics PSOs that are kept in memory at a time.\n")
	TEXT("If set to 0, no limit will be applied (not recommended outside of testing, as it can cause unbounded memory usage)."),
	ECVF_RenderThreadSafe);

int32 GPSOPrecacheKeepInMemoryComputeMaxNum = 1000;
static FAutoConsoleVariableRef CVarPSOPrecacheKeepInMemoryComputeMaxNum(
	TEXT("r.PSOPrecache.KeepInMemoryComputeMaxNum"),
	GPSOPrecacheKeepInMemoryComputeMaxNum,
	TEXT("If r.PSOPrecache.KeepInMemoryUntilUsed is enabled, this value will control the maximum number of precached compute PSOs that are kept in memory at a time.\n")
	TEXT("If set to 0, no limit will be applied (not recommended outside of testing, as it can cause unbounded memory usage)."),
	ECVF_RenderThreadSafe);

bool ShouldKeepPrecachedPSOsForIHV()
{
	return (IsRHIDeviceNVIDIA() || IsRHIDeviceQualcomm());
}

bool ShouldKeepPrecachedPSOsInMemory()
{
	return GPSOPrecacheKeepInMemoryUntilUsed && ShouldKeepPrecachedPSOsForIHV();
}

bool ShouldTrackUsedPrecachedPSOs()
{
	return ShouldKeepPrecachedPSOsInMemory() && GPSOPrecacheKeepInMemoryUntilUsed == 2;
}

const TCHAR* LexToString(EPSOPrecacheResult Result)
{
	switch (Result)
	{
		case EPSOPrecacheResult::Active:       return TEXT("Precaching");
		case EPSOPrecacheResult::Complete:     return TEXT("Precached");
		case EPSOPrecacheResult::Missed:       return TEXT("Missed");
		case EPSOPrecacheResult::TooLate:      return TEXT("Too Late");
		case EPSOPrecacheResult::NotSupported: return TEXT("Not Supported");
		case EPSOPrecacheResult::Untracked:    return TEXT("Untracked");
		case EPSOPrecacheResult::Unknown:
			[[fallthrough]];
		default:                               return TEXT("Unknown");
	}
}

static FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType GetPSOCompileTypeFromPrecachePriority(EPSOPrecachePriority PrecachePriority)
{
	switch (PrecachePriority)
	{
	case EPSOPrecachePriority::Highest:
		return FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType::MaxPri;
	case EPSOPrecachePriority::High:
	case EPSOPrecachePriority::Medium:
		return FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType::NormalPri;
	default: checkNoEntry(); [[fallthrough]];
	case EPSOPrecachePriority::Low:
		return FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType::MinPri;
	}
}

static EQueuedWorkPriority GetQueuedWorkPriorityFromPSOPrecachePriority(EPSOPrecachePriority PrecachePriority)
{
	switch (PrecachePriority)
	{
		case EPSOPrecachePriority::Highest:
			return EQueuedWorkPriority::Highest;
		case EPSOPrecachePriority::High:
			return EQueuedWorkPriority::High;
		case EPSOPrecachePriority::Low:
			return EQueuedWorkPriority::Low;
		default: checkNoEntry(); [[fallthrough]];
		case EPSOPrecachePriority::Medium:
			return EQueuedWorkPriority::Normal;
	}
}

static EPSOPrecachePriority GetPSOPrecachePriorityFromQueuePri(EQueuedWorkPriority QueuePri)
{
	switch (QueuePri)
	{
		case EQueuedWorkPriority::Blocking:
		case EQueuedWorkPriority::Highest:
			return EPSOPrecachePriority::Highest;
		case EQueuedWorkPriority::High:
			return EPSOPrecachePriority::High;
		case EQueuedWorkPriority::Low:
		case EQueuedWorkPriority::Lowest:
			return EPSOPrecachePriority::Low;			
		default: checkNoEntry(); [[fallthrough]];
		case EQueuedWorkPriority::Normal:
			return EPSOPrecachePriority::Medium;
	}
}

// PSO thread pool wrapper allows us to control scheduling based on the priority of tasks.
// when r.PSOPrecaching.TaskPriorityThreshold is set only jobs >= priority will be processed.
// note, the remaining tasks are required to stick around as they are found and boosted when called for.
class FPSOQueuedThreadPoolWrapper : public FQueuedThreadPoolWrapper
{
public:
	FPSOQueuedThreadPoolWrapper(FQueuedThreadPool* InWrappedQueuedThreadPool)
		: FQueuedThreadPoolWrapper(InWrappedQueuedThreadPool)
	{}
protected:

	virtual bool CanSchedule(EQueuedWorkPriority QueuePriority) const override
	{
		if(FQueuedThreadPoolWrapper::CanSchedule(QueuePriority))
		{
			const EPSOPrecachePriority PrecachePriority = GetPSOPrecachePriorityFromQueuePri(QueuePriority);
			return ((int32)PrecachePriority >= GPSOPrecacheProcessPriorityThreshold);
		}
		return false;
	}
};

class FPSOPrecacheThreadPool
{
public:
	~FPSOPrecacheThreadPool()
	{
		// Thread pool needs to be shutdown before the global object is deleted
		check(PSOPrecompileCompileThreadPool.load() == nullptr);
	}

	FQueuedThreadPool& Get()
	{
		if (PSOPrecompileCompileThreadPool.load() == nullptr)
		{
			FScopeLock lock(&LockCS);
			if (PSOPrecompileCompileThreadPool.load() == nullptr)
			{
				check(UsePool());

				FQueuedThreadPool* PSOPrecompileCompileThreadPoolLocal = FPSOQueuedThreadPoolWrapper::Allocate();
				PSOPrecompileCompileThreadPoolLocal->Create(GetDesiredPoolSize(), 512 * 1024, (EThreadPriority)GPSOPrecompileThreadPoolThreadPriority, TEXT("PSOPrecompilePool"));
				PSOPrecompileCompileThreadPool = new FPSOQueuedThreadPoolWrapper(PSOPrecompileCompileThreadPoolLocal);
			}
		}
		return *PSOPrecompileCompileThreadPool;
	}

	void ShutdownThreadPool()
	{
		FQueuedThreadPool* LocalPool = PSOPrecompileCompileThreadPool.exchange(nullptr);
		if (LocalPool != nullptr)
		{
			delete LocalPool;
		}
	}

	static int32 GetDesiredPoolSize()
	{
		if (GPSOPrecompileThreadPoolSize > 0)
		{
			ensure(GPSOPrecompileThreadPoolPercentOfHardwareThreads == 0); // These settings are mutually exclusive
			return GPSOPrecompileThreadPoolSize;
		}
		if (GPSOPrecompileThreadPoolPercentOfHardwareThreads > 0)
		{
			int32 NumThreads = FMath::CeilToInt((float)FPlatformMisc::NumberOfCoresIncludingHyperthreads() * (float)GPSOPrecompileThreadPoolPercentOfHardwareThreads / 100.0f);
			return FMath::Clamp(NumThreads, GPSOPrecompileThreadPoolSizeMin, GPSOPrecompileThreadPoolSizeMax);
		}
		return 0;
	}

	static bool UsePool()
	{
		return GetDesiredPoolSize() > 0;
	}

private:
	FCriticalSection LockCS;
	std::atomic<FQueuedThreadPool*> PSOPrecompileCompileThreadPool {};
};

static FPSOPrecacheThreadPool GPSOPrecacheThreadPool;

void PipelineStateCache::PreCompileComplete()
{
	// free up our threads when the precompile completes and don't have precaching enabled (otherwise the thread are used during gameplay as well)
	if (!PipelineStateCache::IsPSOPrecachingEnabled())
	{
		GPSOPrecacheThreadPool.ShutdownThreadPool();
	}
}

extern RHI_API FComputePipelineState* FindComputePipelineState(const FComputePipelineStateInitializer& Initializer, bool bVerifyUse);
extern RHI_API FComputePipelineState* GetComputePipelineState(FRHIComputeCommandList& RHICmdList, const FComputePipelineStateInitializer& Initializer, bool bVerifyUse);
extern RHI_API FRHIComputePipelineState* ExecuteSetComputePipelineState(FComputePipelineState* ComputePipelineState);
extern RHI_API FRHIGraphicsPipelineState* ExecuteSetGraphicsPipelineState(FGraphicsPipelineState* GraphicsPipelineState);

// Prints out information about a failed compilation from Init.
// This is fatal unless the compilation request is coming from the precaching system.
static void HandlePipelineCreationFailure(const FGraphicsPipelineStateInitializer& Init)
{
	uint64 CombinedHash = 0;
	FString ShaderHashList;

	const auto AddShaderHash = [&CombinedHash, &ShaderHashList](const FRHIShader* Shader)
	{
		FShaderHash ShaderHash;
		if (Shader)
		{
			ShaderHash = Shader->GetHash();
			ShaderHashList.Appendf(TEXT("%s: %s, "), GetShaderFrequencyString(Shader->GetFrequency(), false), *ShaderHash.ToString());
		}
		CombinedHash += ShaderHash.Hash;
	};

	// Log the shader and pipeline hashes, so we can look them up in the stable keys (SHK) file. Please note that NeedsShaderStableKeys must be set to
	// true in the [DevOptions.Shaders] section of *Engine.ini in order for the cook process to produce SHK files for the shader libraries. The contents
	// of those files can be extracted as text using the ShaderPipelineCacheTools commandlet, like this:
	//		UnrealEditor-Cmd.exe ProjectName -run=ShaderPipelineCacheTools dump File.shk
	// The pipeline hash is created by hashing together the individual shader hashes, see FShaderCodeLibraryPipeline::GetPipelineHash for details.
	AddShaderHash(Init.BoundShaderState.GetVertexShader());
	AddShaderHash(Init.BoundShaderState.GetMeshShader());
	AddShaderHash(Init.BoundShaderState.GetAmplificationShader());
	AddShaderHash(Init.BoundShaderState.GetPixelShader());
	AddShaderHash(Init.BoundShaderState.GetGeometryShader());

	FShaderHash PipelineHash;
	PipelineHash.Hash = CombinedHash;

	UE_LOGF(LogRHI, Error, "Failed to create graphics pipeline, hashes: %lsPipeline: %ls.", *ShaderHashList, *PipelineHash.ToString());

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if(Init.BoundShaderState.VertexShaderRHI)
	{
		UE_LOGF(LogRHI, Error, "Vertex: %ls", Init.BoundShaderState.VertexShaderRHI->GetShaderName());
	}
	if (Init.BoundShaderState.GetMeshShader())
	{
		UE_LOGF(LogRHI, Error, "Mesh: %ls", Init.BoundShaderState.GetMeshShader()->GetShaderName());
	}
	if (Init.BoundShaderState.GetAmplificationShader())
	{
		UE_LOGF(LogRHI, Error, "Amplification: %ls", Init.BoundShaderState.GetAmplificationShader()->GetShaderName());
	}
	if(Init.BoundShaderState.GetGeometryShader())
	{
		UE_LOGF(LogRHI, Error, "Geometry: %ls", Init.BoundShaderState.GetGeometryShader()->GetShaderName());
	}
	if(Init.BoundShaderState.PixelShaderRHI)
	{
		UE_LOGF(LogRHI, Error, "Pixel: %ls", Init.BoundShaderState.PixelShaderRHI->GetShaderName());
	}

	UE_LOGF(LogRHI, Error, "Render Targets: (%u)", Init.RenderTargetFormats.Num());
	for(int32 i = 0; i < Init.RenderTargetFormats.Num(); ++i)
	{
		//#todo-mattc GetPixelFormatString is not available in scw. Need to move it so we can print more info here.
		UE_LOGF(LogRHI, Error, "0x%x", (uint32)Init.RenderTargetFormats[i]);
	}

	UE_LOGF(LogRHI, Error, "Depth Stencil Format:");
	UE_LOGF(LogRHI, Error, "0x%x", Init.DepthStencilTargetFormat);
#endif

	if(Init.bFromPSOFileCache)
	{
		// Let the cache know so it hopefully won't give out this one again
		FPipelineFileCacheManager::RegisterPSOCompileFailure(GetTypeHash(Init), Init);
	}
	else if(!Init.bPSOPrecache)
	{
		// Precache requests are allowed to fail, but if the PSO is needed by a draw/dispatch command, we cannot continue.
		UE_LOGF(LogRHI, Fatal, "Shader compilation failures are Fatal.");
	}
}

// Prints out information about a failed compute pipeline compilation.
// This is fatal unless the compilation request is coming from the precaching system.
static void HandlePipelineCreationFailure(const FComputePipelineStateInitializer& Init)
{
	// Dump the shader hash so it can be looked up in the SHK data. See the previous function for details.
	UE_LOGF(LogRHI, Error, "Failed to create compute pipeline with hash %ls.", *Init.ComputeShader->GetHash().ToString());

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_LOGF(LogRHI, Error, "Shader: %ls", Init.ComputeShader->GetShaderName());
#endif

	if (!Init.bPSOPrecache && !Init.bFromPSOFileCache)
	{
		// Precache requests are allowed to fail, but if the PSO is needed by a draw/dispatch command, we cannot continue.
		UE_LOGF(LogRHI, Fatal, "Shader compilation failures are Fatal.");
	}
}

#if PSO_TRACK_CACHE_STATS

static std::atomic_uint64_t TotalPrecompileCompleteTime[(int)EPSOPrecachePriority::Num];
static std::atomic_uint64_t TotalPrecompileCompileTime[(int)EPSOPrecachePriority::Num];
static std::atomic_uint64_t TotalPrecompileTimeToBegin[(int)EPSOPrecachePriority::Num];

static std::atomic_int64_t TotalNumPrecompileJobs[(int)EPSOPrecachePriority::Num];
static std::atomic_int64_t TotalNumPrecompileJobsCompleted[(int)EPSOPrecachePriority::Num];

static std::atomic_int64_t MaxPrecompileJobTime[(int)EPSOPrecachePriority::Num];
static std::atomic_int64_t MaxPrecompileTimeToCompile[(int)EPSOPrecachePriority::Num];
static std::atomic_int64_t MaxPrecompileTimeToBegin[(int)EPSOPrecachePriority::Num];

void ResetPrecompileStats()
{
	for (int i = 0; i < (int)EPSOPrecachePriority::Num; i++)
	{
		TotalPrecompileCompleteTime[i] = 0;
		TotalPrecompileCompileTime[i] = 0;
		TotalPrecompileTimeToBegin[i] = 0;

		TotalNumPrecompileJobs[i] = 0;
		TotalNumPrecompileJobsCompleted[i] = 0;

		MaxPrecompileJobTime[i] = 0;
		MaxPrecompileTimeToCompile[i] = 0;
		MaxPrecompileTimeToBegin[i] = 0;
	}
}

void StatsEndPrecompile(uint64 CreateTime, uint64 RescheduleTime, uint64 TaskBeginTime, uint64 EndTime, EPSOPrecachePriority PrecachePriority)
{
	uint64 TaskIssueTime = FMath::Max(CreateTime, RescheduleTime);
	uint64 TimeToComplete = EndTime - TaskIssueTime;
	uint64 TimeToCompile = EndTime - TaskBeginTime;
	uint64 TimeToBegin = TaskBeginTime - TaskIssueTime;
	check (TaskBeginTime > TaskIssueTime);

	TotalPrecompileCompleteTime[(int)PrecachePriority] += TimeToComplete;
	TotalPrecompileCompileTime[(int)PrecachePriority] += TimeToCompile;
	TotalPrecompileTimeToBegin[(int)PrecachePriority] += TimeToBegin;

	MaxPrecompileJobTime[(int)PrecachePriority] = FMath::Max((uint64)MaxPrecompileJobTime[(int)TaskPri].load(), (uint64)TimeToComplete);
	MaxPrecompileTimeToCompile[(int)PrecachePriority] = FMath::Max((uint64)MaxPrecompileTimeToCompile[(int)TaskPri].load(), (uint64)TimeToCompile);
	MaxPrecompileTimeToBegin[(int)PrecachePriority] = FMath::Max((uint64)MaxPrecompileTimeToBegin[(int)TaskPri].load(), (uint64)TimeToBegin);

	TotalNumPrecompileJobsCompleted[(int)PrecachePriority]++;
}
#endif

class FPSOPrecacheAsyncTask
	: public FAsyncTaskBase
{
	TUniqueFunction<void(const FPSOPrecacheAsyncTask*)> AsyncTaskFunc;
public:

	FPSOPrecacheAsyncTask(TUniqueFunction<void(const FPSOPrecacheAsyncTask*)> InFunc) : AsyncTaskFunc(MoveTemp(InFunc))
	{
#if PSO_TRACK_CACHE_STATS
		CreateTime = FPlatformTime::Cycles64();
#endif
		// Cache the StatId to remain backward compatible with TTask that declare GetStatId as non-const.
		Init(GetStatId());
	}
	bool TryAbandonTask() final 	{ 	return false;	}

	bool Reschedule(FQueuedThreadPool* InQueuedPool, EQueuedWorkPriority InQueuedWorkPriority)
	{
		uint64 RescheduleAttemptTime = FPlatformTime::Cycles64();
		bool bSuccess = FAsyncTaskBase::Reschedule(InQueuedPool, InQueuedWorkPriority);
#if PSO_TRACK_CACHE_STATS
		if (bSuccess)
		{
			RescheduleTime = RescheduleAttemptTime;
		}
#endif
		return bSuccess;
	}

	void DoTaskWork() final
	{
#if PSO_TRACK_CACHE_STATS
		TaskBeginTime = FPlatformTime::Cycles64();
#endif
		AsyncTaskFunc(this);
#if PSO_TRACK_CACHE_STATS
		StatsEndPrecompile(CreateTime, RescheduleTime, TaskBeginTime, FPlatformTime::Cycles64(), GetPriority());
#endif
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FPSOPrecacheAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
	}

#if PSO_TRACK_CACHE_STATS
	uint64 CreateTime = 0;
	uint64 RescheduleTime = 0;
	uint64 TaskBeginTime = 0;
#endif
};

/**
 * Base class to hold pipeline state (and optionally stats)
 */
class FPipelineState
{
public:

	FPipelineState()
	: Stats(nullptr)
	{
		InitStats();
	}

	virtual ~FPipelineState() = default;

	virtual bool IsCompute() const = 0;

	inline void AddUse()
	{
		FPipelineStateStats::UpdateStats(Stats);
	}
	
#if PSO_TRACK_CACHE_STATS
	
	void InitStats()
	{
		FirstUsedTime = LastUsedTime = FPlatformTime::Seconds();
		FirstFrameUsed = LastFrameUsed = 0;
		Hits = HitsAcrossFrames = 0;
	}
	
	void AddHit()
	{
		LastUsedTime = FPlatformTime::Seconds();
		Hits++;

		if (LastFrameUsed != GFrameCounter)
		{
			LastFrameUsed = GFrameCounter;
			HitsAcrossFrames++;
		}
	}

	double			FirstUsedTime;
	double			LastUsedTime;
	uint64			FirstFrameUsed;
	uint64			LastFrameUsed;
	int				Hits;
	int				HitsAcrossFrames;

#else
	void InitStats() {}
	void AddHit() {}
#endif // PSO_TRACK_CACHE_STATS

	FPipelineStateStats* Stats;
};

namespace PipelineStateCache
{
	constexpr int32 RenderThreadIndex = 0;
	constexpr int32 RHIThreadIndex = 1;

	int32 GetCacheIndexForCurrentThread()
	{
		return IsInParallelRHIThread() || !GEnablePSOAsyncCacheConsolidation;
	}
}

#if CSV_PROFILER_STATS
class FPSOCSVStatTracker
{
private:
	
	struct FPersistentStats
	{
		// PSO compiles can take place over multiple frames, use the persistent stats.
		TCsvPersistentCustomStat<int>* TotalCompletedPSOPrecompiles			= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("TotalCompletedPSOPrecompiles"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
		TCsvPersistentCustomStat<int>* PSOPrecompileQueueLength				= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("PSOPrecompileQueueLength"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
		TCsvPersistentCustomStat<int>* PSOGraphicsPrecompilesInProgress		= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("PSOGraphicsPrecompilesInProgress"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
		TCsvPersistentCustomStat<int>* PSOComputePrecompilesInProgress		= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("PSOComputePrecompilesInProgress"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));

		TCsvPersistentCustomStat<int>* LowPriorityPrecacheInFlight			= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("LowPriorityPrecacheInFlight"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
		TCsvPersistentCustomStat<int>* MediumPriorityPrecacheInFlight		= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("MediumPriorityPrecacheInFlight"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
		TCsvPersistentCustomStat<int>* HighPriorityPrecacheInFlight			= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("HighPriorityPrecacheInFlight"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
		TCsvPersistentCustomStat<int>* HighestPriorityPrecacheInFlight		= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("HighestPriorityPrecacheInFlight"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));

		TCsvPersistentCustomStat<int>* LowPriorityPSOPrecompileQueueLength	= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("LowPriorityPSOPrecompileQueueLength"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
		TCsvPersistentCustomStat<int>* MediumPriorityPSOPrecompileQueueLength	= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("MediumPriorityPSOPrecompileQueueLength"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
		TCsvPersistentCustomStat<int>* HighPriorityPSOPrecompileQueueLength	= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("HighPriorityPSOPrecompileQueueLength"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
		TCsvPersistentCustomStat<int>* HighestPriorityPSOPrecompileQueueLength = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("HighestPriorityPSOPrecompileQueueLength"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
	};

	static FPersistentStats& GetPersistentStats()
	{
		static FPersistentStats PersistentStats;
		return PersistentStats;
	}
	static bool AreStatsEnabled()
	{
		return bEnableStatTracker && FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
	}

	enum class EStatType
	{
		InFlight,
		Queued,
	};

	std::atomic_int PSOPrecachePriority = (int)EPSOPrecachePriority::NotSet;

	uint8 bPrecache : 1 = 0;
	uint8 bCompute : 1 = 0;
	uint8 bProcessedFinished : 1 = 0;

public:

	static inline bool bEnableStatTracker = true;

	~FPSOCSVStatTracker()
	{
		check(!AreStatsEnabled() || bProcessedFinished || !bPrecache);
	}

	void SetState(bool bInPrecache, bool bInCompute, EPSOPrecachePriority InPriority)
	{
		if (!AreStatsEnabled())
		{
			return;
		}

		bPrecache = bInPrecache;
		bCompute = bInCompute;

		if (bPrecache)
		{
			FPersistentStats& PersistentStats = GetPersistentStats();
			PersistentStats.PSOPrecompileQueueLength->Add(1);

			if (bCompute)
			{
				PersistentStats.PSOComputePrecompilesInProgress->Add(1);
			}
			else
			{
				PersistentStats.PSOGraphicsPrecompilesInProgress->Add(1);
			}

			SetQueuedPriority(InPriority);
		}
	}
	
	void SetQueuedPriority(EPSOPrecachePriority InNewPriority)
	{
		if (!AreStatsEnabled())
		{
			return;
		}

		EPSOPrecachePriority PriorValue = (EPSOPrecachePriority)PSOPrecachePriority.exchange((int)InNewPriority);
		if (bPrecache && PriorValue != InNewPriority)
		{
			// unset the previous priority
			if (PriorValue != EPSOPrecachePriority::NotSet)
			{
				GetPriorityStat(PriorValue, EStatType::Queued)->Add(-1);
			}
			GetPriorityStat(InNewPriority, EStatType::Queued)->Add(1);
		}
	}

	void SetInFlightPriority(EPSOPrecachePriority InFlightPriority)
	{
		if (!AreStatsEnabled())
		{
			return;
		}

		check(bPrecache);

		// This is called during the task execution where the priority is final.
		// We set both the priority and the inflight state.
		check(InFlightPriority != EPSOPrecachePriority::NotSet);
		SetQueuedPriority(InFlightPriority);
		GetPriorityStat(InFlightPriority, EStatType::InFlight)->Add(1);
	}

	void PrecacheFinished()
	{
		if (!AreStatsEnabled())
		{
			return;
		}

		check(bPrecache)

		FPersistentStats& PersistentStats = GetPersistentStats();

		PersistentStats.TotalCompletedPSOPrecompiles->Add(1);
		PersistentStats.PSOPrecompileQueueLength->Add(-1);

		if (bCompute)
		{
			PersistentStats.PSOComputePrecompilesInProgress->Add(-1);
		}
		else
		{
			PersistentStats.PSOGraphicsPrecompilesInProgress->Add(-1);
		}

		EPSOPrecachePriority CurrentPriority = (EPSOPrecachePriority)PSOPrecachePriority.load();
		if (ensure(CurrentPriority != EPSOPrecachePriority::NotSet))
		{
			GetPriorityStat(CurrentPriority, EStatType::Queued)->Add(-1);
			GetPriorityStat(CurrentPriority, EStatType::InFlight)->Add(-1);
		}

		bProcessedFinished = true;
	}

private:

	TCsvPersistentCustomStat < int > * GetPriorityStat(EPSOPrecachePriority InPriority, EStatType StatType) const
	{
		FPersistentStats& PersistentStats = GetPersistentStats();

		switch (InPriority)
		{
			case EPSOPrecachePriority::Low:
				return StatType == EStatType::InFlight ? PersistentStats.LowPriorityPrecacheInFlight : PersistentStats.LowPriorityPSOPrecompileQueueLength;				
			case EPSOPrecachePriority::Medium:
				return StatType == EStatType::InFlight ? PersistentStats.MediumPriorityPrecacheInFlight : PersistentStats.MediumPriorityPSOPrecompileQueueLength;				
			case EPSOPrecachePriority::High:
				return StatType == EStatType::InFlight ? PersistentStats.HighPriorityPrecacheInFlight : PersistentStats.HighPriorityPSOPrecompileQueueLength;				
			case EPSOPrecachePriority::Highest:
				return StatType == EStatType::InFlight ? PersistentStats.HighestPriorityPrecacheInFlight : PersistentStats.HighestPriorityPSOPrecompileQueueLength;
			default:
				checkNoEntry();
				return nullptr;
		}
	}
};

static FAutoConsoleVariableRef CVarPSOPrecacheEnableCompileStats(
	TEXT("r.PSOPrecache.EnableCompileStats"),
	FPSOCSVStatTracker::bEnableStatTracker,
	TEXT("Enable CSV stat tracking of precache compile progress. (default enabled)"),
	ECVF_RenderThreadSafe
);

#else
class FPSOCSVStatTracker
{
public:
	FPSOCSVStatTracker() {}
	void SetState(bool bInPrecache, bool bInCompute, EPSOPrecachePriority InPriority) {}
	void SetQueuedPriority(EPSOPrecachePriority InPriority) {}
	void SetInFlightPriority(EPSOPrecachePriority InPriority) {}
	void PrecacheFinished() {}
};
#endif


/**
 * Base class for pipeline state intended to be stored in a TSharedPipelineStateCache,
 * with state double buffering for Render and RHI Threads.
 */
class FPipelineStateAsync : public FPipelineState
{
private:

	struct FCompletionState
	{
		FGraphEventRef CompletionEvent;
		TUniquePtr<FPSOPrecacheAsyncTask> PrecompileTask;
	};
	TSharedPtr<FCompletionState> CompletionStates[2];

	// GetCompletionState is thread safe on Render or RHI Threads.
	const TSharedPtr<FCompletionState>& GetCompletionState() const
	{
		return CompletionStates[PipelineStateCache::GetCacheIndexForCurrentThread()];
	}

	// MakeCompletionState is not thread safe.
	const TSharedPtr<FCompletionState>& MakeCompletionState()
	{
		const TSharedPtr<FCompletionState>& CompletionState = GetCompletionState();
		if (!CompletionState)
		{
			CompletionStates[1] = CompletionStates[0] = MakeShared<FCompletionState>();
		}
		return CompletionState;
	}

	// ClearCompletionState can be called safely by a Render and RHI task when there are no parallel tasks of the same type.
	void ClearCompletionState()
	{
		int32 CacheIndex = PipelineStateCache::GetCacheIndexForCurrentThread();
		CompletionStates[CacheIndex] = nullptr;

		// Clear both references if asynchronous pipeline state cache is disabled.
		if (!IsInParallelRHIThread())
		{
			// Accessing GEnablePSOAsyncCacheConsolidation is safe on the Render Thread
			if (!GEnablePSOAsyncCacheConsolidation)
			{
				CompletionStates[!CacheIndex] = nullptr;
			}
		}
	}

public:

	FPSOCSVStatTracker CSVStat;

	virtual ~FPipelineStateAsync() override
	{
		check(IsComplete());
		verify(!WaitCompletion());
	}

	// GetCompletionEvent is thread safe on Render or RHI Threads as long as ClearCompletionState is not called on a parallel task.
	FGraphEvent* GetCompletionEvent() const
	{
		const TSharedPtr<FCompletionState>& CompletionState = GetCompletionState();
		return CompletionState ? GetCompletionState()->CompletionEvent.GetReference() : nullptr;
	}

	// SetCompletionEvent is not thread safe and should be called before adding this state to the cache.
	void SetCompletionEvent(FGraphEventRef InCompletionEvent)
	{
		MakeCompletionState()->CompletionEvent = MoveTemp(InCompletionEvent);
	}

	// GetPrecompileTask is thread safe on Render or RHI Threads as long as ClearCompletionState is not called on a parallel task.
	FPSOPrecacheAsyncTask* GetPrecompileTask() const
	{
		const TSharedPtr<FCompletionState>& CompletionState = GetCompletionState();
		return CompletionState ? CompletionState->PrecompileTask.Get() : nullptr;
	}

	// SetPrecompileTask is not thread safe and should be called before adding this state to the cache.
	void SetPrecompileTask(TUniquePtr<FPSOPrecacheAsyncTask> InPrecompileTask)
	{
		MakeCompletionState()->PrecompileTask = MoveTemp(InPrecompileTask);
	}

	// IsComplete is thread safe on Render or RHI Threads as long as ClearCompletionState is not called on a parallel task.
	bool IsComplete()
	{
		const TSharedPtr<FCompletionState>& CompletionState = GetCompletionState();
		return CompletionState == nullptr || ((!CompletionState->CompletionEvent.IsValid() || CompletionState->CompletionEvent->IsComplete()) && (!CompletionState->PrecompileTask.IsValid() || CompletionState->PrecompileTask->IsDone()));
	}

	// WaitCompletion can be called safely by a Render and RHI task when there are no parallel tasks of the same type.
	// return true if we actually waited on the task
	bool WaitCompletion()
	{
		bool bNeedsToWait = false;

		FPSOPrecacheAsyncTask* PrecompileTask = GetPrecompileTask();
		if (PrecompileTask)
		{
			bNeedsToWait = bNeedsToWait || !PrecompileTask->IsDone();
			PrecompileTask->EnsureCompletion();
		}

		FGraphEvent* CompletionEvent = GetCompletionEvent();
		if (CompletionEvent && !CompletionEvent->IsComplete())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FPipelineState_WaitCompletion);
#if PSO_TRACK_CACHE_STATS
			UE_LOGF(LogRHI, Log, "FTaskGraphInterface Waiting on FPipelineState completionEvent");
#endif
			bNeedsToWait = true;
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(CompletionEvent);
		}

		ClearCompletionState();
		return bNeedsToWait;
	}
};

/**
 * Base class for pipeline state that doesn't need state double buffering.
 */
class FPipelineStateSync : public FPipelineState
{
public:

	virtual ~FPipelineStateSync() 
	{
		check(IsComplete());
		verify(!WaitCompletion());
		check(!PrecompileTask.IsValid());
	}

	virtual bool IsCompute() const = 0;

	FGraphEventRef CompletionEvent;
	TUniquePtr<FPSOPrecacheAsyncTask> PrecompileTask;

	bool IsComplete()
	{
		return (!CompletionEvent.IsValid() || CompletionEvent->IsComplete()) && (!PrecompileTask.IsValid() || PrecompileTask->IsDone());
	}

	// return true if we actually waited on the task
	bool WaitCompletion()
	{
		bool bNeedsToWait = false;

		if (PrecompileTask.IsValid())
		{
			bNeedsToWait = bNeedsToWait || !PrecompileTask->IsDone();
			PrecompileTask->EnsureCompletion();
			PrecompileTask = nullptr;
		}

		if(CompletionEvent.IsValid() && !CompletionEvent->IsComplete())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FPipelineState_WaitCompletion);
#if PSO_TRACK_CACHE_STATS
			UE_LOGF(LogRHI, Log, "FTaskGraphInterface Waiting on FPipelineState completionEvent");
#endif
			bNeedsToWait = true;
			FTaskGraphInterface::Get().WaitUntilTaskCompletes( CompletionEvent );
		}
		CompletionEvent = nullptr;

		return bNeedsToWait;
	}
};


/* State for compute  */
class FComputePipelineState : public FPipelineStateAsync
{
public:
	FComputePipelineState(FRHIComputeShader* InComputeShader, bool bInRuntimePSO, bool bInFromInMemoryPSOCache)
		: ComputeShader(InComputeShader)
	{
		ComputeShader->AddRef();
#if STATS
		bRuntimePSO = bInRuntimePSO; 
		bFromInMemoryPSOCache = bInFromInMemoryPSOCache;
		if (bRuntimePSO)
		{
			INC_DWORD_STAT(STAT_ActiveRuntimeComputePSOs);
			if (bFromInMemoryPSOCache)
			{
				INC_DWORD_STAT(STAT_RuntimeComputePSOsFromInMemoryPSOCache);
			}
		}
#endif // STATS
	}
	
	~FComputePipelineState()
	{
		ComputeShader->Release();
#if STATS		
		if (bRuntimePSO)
		{
			DEC_DWORD_STAT(STAT_ActiveRuntimeComputePSOs);
			if (bFromInMemoryPSOCache)
			{
				DEC_DWORD_STAT(STAT_RuntimeComputePSOsFromInMemoryPSOCache);
			}
		}
#endif // STATS
	}

	virtual bool IsCompute() const
	{
		return true;
	}

	inline void Verify_IncUse()
	{
	#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		int32 Result = InUseCount.Increment();
		check(Result >= 1);
	#endif
	}

	inline void Verify_DecUse()
	{
	#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		int32 Result = InUseCount.Decrement();
		check(Result >= 0);
	#endif
	}

	inline void Verify_NoUse()
	{
	#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		check(InUseCount.GetValue() == 0);
	#endif
	}

	FRHIComputeShader* ComputeShader;
	TRefCountPtr<FRHIComputePipelineState> RHIPipeline;
#if STATS
	bool bRuntimePSO = true;
	bool bFromInMemoryPSOCache = false;
#endif // STATS
#if PIPELINESTATECACHE_VERIFYTHREADSAFE
	FThreadSafeCounter InUseCount;
#endif
};

/* State for work graphs  */
class FWorkGraphPipelineState : public FPipelineStateAsync
{
public:
	FWorkGraphPipelineState(FRHIWorkGraphShader* InWorkGraphShader)
		: WorkGraphShader(InWorkGraphShader)
	{
		WorkGraphShader->AddRef();
	}

	~FWorkGraphPipelineState()
	{
		WorkGraphShader->Release();
	}

	virtual bool IsCompute() const
	{
		return true;
	}

	bool IsCompilationComplete() const
	{
		FGraphEvent* CompletionEvent = GetCompletionEvent();
		return !CompletionEvent || CompletionEvent->IsComplete();
	}

	inline void Verify_IncUse()
	{
#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		int32 Result = InUseCount.Increment();
		check(Result >= 1);
#endif
	}

	inline void Verify_DecUse()
	{
#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		int32 Result = InUseCount.Decrement();
		check(Result >= 0);
#endif
	}

	inline void Verify_NoUse()
	{
#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		check(InUseCount.GetValue() == 0);
#endif
	}

	FWorkGraphShaderRHIRef WorkGraphShader;
	FWorkGraphPipelineStateRHIRef RHIPipeline;
#if PIPELINESTATECACHE_VERIFYTHREADSAFE
	FThreadSafeCounter InUseCount;
#endif
};


/* State for graphics */
class FGraphicsPipelineState : public FPipelineStateAsync
{
public:
	FGraphicsPipelineState(bool bInRuntimePSO, bool bInFromInMemoryPSOCache)
	{
	#if STATS
		bRuntimePSO = bInRuntimePSO;
		bFromInMemoryPSOCache = bInFromInMemoryPSOCache;
		if (bRuntimePSO)
		{
			INC_DWORD_STAT(STAT_ActiveRuntimeGraphicsPSOs);
			if (bFromInMemoryPSOCache)
			{
				INC_DWORD_STAT(STAT_RuntimeGraphicsPSOsFromInMemoryPSOCache);
			}
		}
	#endif // STATS
	}

	~FGraphicsPipelineState()
	{
	#if STATS
		if (bRuntimePSO)
		{
			DEC_DWORD_STAT(STAT_ActiveRuntimeGraphicsPSOs);
			if (bFromInMemoryPSOCache)
			{
				DEC_DWORD_STAT(STAT_RuntimeGraphicsPSOsFromInMemoryPSOCache);
			}
		}
	#endif // STATS
	}

	virtual bool IsCompute() const
	{
		return false;
	}

	inline void Verify_IncUse()
	{
	#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		int32 Result = InUseCount.Increment();
		check(Result >= 1);
	#endif
	}

	inline void Verify_DecUse()
	{
	#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		int32 Result = InUseCount.Decrement();
		check(Result >= 0);
	#endif
	}

	inline void Verify_NoUse()
	{
	#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		check(InUseCount.GetValue() == 0);
	#endif
	}

	TRefCountPtr<FRHIGraphicsPipelineState> RHIPipeline;
#if PIPELINESTATECACHE_VERIFYTHREADSAFE
	FThreadSafeCounter InUseCount;
#endif
	uint64 SortKey = 0;
#if STATS
	bool bRuntimePSO = true;
	bool bFromInMemoryPSOCache = false;
#endif // STATS
};

FRHIComputePipelineState* GetRHIComputePipelineState(FComputePipelineState* PipelineState)
{
	ensure(PipelineState->RHIPipeline);
	return PipelineState->RHIPipeline;
}

FRHIWorkGraphPipelineState* GetRHIWorkGraphPipelineState(FWorkGraphPipelineState* PipelineState)
{
	ensure(PipelineState->RHIPipeline);
	return PipelineState->RHIPipeline;
}

/* State for ray tracing */
class FRayTracingPipelineState : public FPipelineStateSync
{
public:
	FRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer)
	{
		{
			int32 Index = 0;
			for (FRHIRayTracingShader* Shader : Initializer.GetHitGroupTable())
			{
				HitGroupShaderMap.Add(Shader->GetHash(), Index++);
			}
		}

		{
			int32 Index = 0;
			for (FRHIRayTracingShader* Shader : Initializer.GetCallableTable())
			{
				CallableShaderMap.Add(Shader->GetHash(), Index++);
			}
		}

		{
			int32 Index = 0;
			for (FRHIRayTracingShader* Shader : Initializer.GetMissTable())
			{
				MissShaderMap.Add(Shader->GetHash(), Index++);
			}
		}
	}

	virtual bool IsCompute() const
	{
		return false;
	}

	inline void AddHit()
	{
		if (LastFrameHit != GFrameCounter)
		{
			LastFrameHit = GFrameCounter;
			HitsAcrossFrames++;
		}

		FPipelineStateSync::AddHit();
	}

	bool operator < (const FRayTracingPipelineState& Other)
	{
		if (LastFrameHit != Other.LastFrameHit)
		{
			return LastFrameHit < Other.LastFrameHit;
		}
		return HitsAcrossFrames < Other.HitsAcrossFrames;
	}

	bool IsCompilationComplete() const 
	{
		return !CompletionEvent.IsValid() || CompletionEvent->IsComplete();
	}

	inline void Verify_NoUse()
	{
	#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		check(InUseCount.GetValue() == 0);
	#endif
	}

	FRayTracingPipelineStateRHIRef RHIPipeline;

	uint32 MaxLocalBindingSize = 0;

	uint64 HitsAcrossFrames = 0;
	uint64 LastFrameHit = 0;

	TMap<FShaderHash, int32> HitGroupShaderMap;
	TMap<FShaderHash, int32> CallableShaderMap;
	TMap<FShaderHash, int32> MissShaderMap;

#if PIPELINESTATECACHE_VERIFYTHREADSAFE
	FThreadSafeCounter InUseCount;
#endif
};

FRHIRayTracingPipelineState* GetRHIRayTracingPipelineState(FRayTracingPipelineState* PipelineState)
{
	if (PipelineState)
	{
		ensure(PipelineState->RHIPipeline);
		PipelineState->CompletionEvent = nullptr;
		return PipelineState->RHIPipeline;
	}
	return nullptr;
}

uint32 GetRHIRayTracingPipelineStateMaxLocalBindingDataSize(FRayTracingPipelineState* PipelineState)
{
	if (PipelineState)
	{
		return PipelineState->MaxLocalBindingSize;
	}
	return 0;
}

int32 FindRayTracingHitGroupIndex(FRayTracingPipelineState* Pipeline, FRHIRayTracingShader* HitGroupShader, bool bRequired)
{
#if RHI_RAYTRACING
	if (int32* FoundIndex = Pipeline->HitGroupShaderMap.Find(HitGroupShader->GetHash()))
	{
		return *FoundIndex;
	}
	checkf(!bRequired, TEXT("Required hit group shader was not found in the ray tracing pipeline."));
#endif // RHI_RAYTRACING

	return INDEX_NONE;
}

int32 FindRayTracingCallableShaderIndex(FRayTracingPipelineState* Pipeline, FRHIRayTracingShader* CallableShader, bool bRequired)
{
#if RHI_RAYTRACING
	if (int32* FoundIndex = Pipeline->CallableShaderMap.Find(CallableShader->GetHash()))
	{
		return *FoundIndex;
	}
	checkf(!bRequired, TEXT("Required callable shader was not found in the ray tracing pipeline."));
#endif // RHI_RAYTRACING

	return INDEX_NONE;
}

int32 FindRayTracingMissShaderIndex(FRayTracingPipelineState* Pipeline, FRHIRayTracingShader* MissShader, bool bRequired)
{
#if RHI_RAYTRACING
	if (int32* FoundIndex = Pipeline->MissShaderMap.Find(MissShader->GetHash()))
	{
		return *FoundIndex;
	}
	checkf(!bRequired, TEXT("Required miss shader was not found in the ray tracing pipeline."));
#endif // RHI_RAYTRACING

	return INDEX_NONE;
}

template <typename TPipelineInitializer>
bool IsPrecachedPSO(const TPipelineInitializer& Initializer)
{
	return Initializer.bFromPSOFileCache || Initializer.bPSOPrecache;
}

FComputePipelineState* FindComputePipelineState(const FComputePipelineStateInitializer& Initializer, bool bVerifyUse)
{
	return PipelineStateCache::FindComputePipelineState(Initializer, bVerifyUse);
}

FComputePipelineState* GetComputePipelineState(FRHIComputeCommandList& RHICmdList, const FComputePipelineStateInitializer& Initializer, bool bVerifyUse)
{
	FComputePipelineState* PipelineState = PipelineStateCache::GetAndOrCreateComputePipelineState(RHICmdList, Initializer);
	if (PipelineState && bVerifyUse)
	{
		PipelineState->Verify_IncUse();
	}
	return PipelineState;
}

void SetComputePipelineState(FRHIComputeCommandList& RHICmdList, const FComputePipelineStateInitializer& Initializer)
{
	FComputePipelineState* PipelineState = GetComputePipelineState(RHICmdList, Initializer);
	RHICmdList.SetComputePipelineState(PipelineState, Initializer.ComputeShader);
}

FGraphicsPipelineState* FindGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer, bool bVerifyUse)
{
	return PipelineStateCache::FindGraphicsPipelineState(Initializer, bVerifyUse);
}

FGraphicsPipelineState* GetGraphicsPipelineState(FRHICommandList& RHICmdList, const FGraphicsPipelineStateInitializer& Initializer, EApplyRendertargetOption ApplyFlags, bool bVerifyUse)
{
#if PLATFORM_USE_FALLBACK_PSO
	checkNoEntry();
	return nullptr;
#else
	FGraphicsPipelineState* PipelineState = PipelineStateCache::GetAndOrCreateGraphicsPipelineState(RHICmdList, Initializer, ApplyFlags);
	if (PipelineState && bVerifyUse && !Initializer.bFromPSOFileCache)
	{
		PipelineState->Verify_IncUse();
	}
	return PipelineState;
#endif
}

FGraphicsPipelineState* GetGraphicsPipelineState(FRHICommandList& RHICmdList, const FGraphicsPipelineStateInitializer& Initializer, bool bVerifyUse)
{
	return GetGraphicsPipelineState(RHICmdList, Initializer, EApplyRendertargetOption::CheckApply, bVerifyUse);
}

void SetGraphicsPipelineState(FRHICommandList& RHICmdList, const FGraphicsPipelineStateInitializer& Initializer, uint32 StencilRef, EApplyRendertargetOption ApplyFlags, bool bApplyAdditionalState)
{
#if PLATFORM_USE_FALLBACK_PSO
	RHICmdList.SetGraphicsPipelineState(Initializer, StencilRef, bApplyAdditionalState);
#else
	FGraphicsPipelineState* PipelineState = GetGraphicsPipelineState(RHICmdList, Initializer, ApplyFlags, true /* bVerifyUse */);
	if (PipelineState && !Initializer.bFromPSOFileCache)
	{
		check(IsInRenderingThread() || IsInParallelRenderingThread());
		RHICmdList.SetGraphicsPipelineState(PipelineState, Initializer.BoundShaderState, StencilRef, bApplyAdditionalState);
	}
#endif
}

void SetGraphicsPipelineStateCheckApply(FRHICommandList& RHICmdList, const FGraphicsPipelineStateInitializer& Initializer, uint32 StencilRef, bool bApplyAdditionalState)
{
#if PLATFORM_USE_FALLBACK_PSO
	RHICmdList.SetGraphicsPipelineState(Initializer, StencilRef, bApplyAdditionalState);
#else
	FGraphicsPipelineState* PipelineState = GetGraphicsPipelineState(RHICmdList, Initializer, true /* bVerifyUse */);
	if (PipelineState && !Initializer.bFromPSOFileCache)
	{
		check(IsInRenderingThread() || IsInParallelRenderingThread());
		RHICmdList.SetGraphicsPipelineState(PipelineState, Initializer.BoundShaderState, StencilRef, bApplyAdditionalState);
	}
#endif
}

/* TSharedPipelineStateCache
 * This is a cache of the * pipeline states
 * there is a local thread cache which is consolidated with the global thread cache
 * global thread cache is read only until the end of the frame when the local thread caches are consolidated
 */
template<class TMyKey,class TMyValue>
class TSharedPipelineStateCache
{
private:

	TMap<TMyKey, TMyValue>& GetLocalCache()
	{
		// Find or create storage for two PipelineStateCacheTypes for this thread.
		TOptional<FPipelineStateCacheType>* PipelineStateCaches = static_cast<TOptional<FPipelineStateCacheType>*>(FPlatformTLS::GetTlsValue(TLSSlot));
		if (!PipelineStateCaches)
		{
			PipelineStateCaches = new TOptional<FPipelineStateCacheType>[2];
			FPlatformTLS::SetTlsValue(TLSSlot, PipelineStateCaches);
		}

		// Select the cache to use, based on whether or not this thread is processing RHI tasks.
		int32 CacheIndex = PipelineStateCache::GetCacheIndexForCurrentThread();
		TOptional<FPipelineStateCacheType>& PipelineStateCache = PipelineStateCaches[CacheIndex];
		if (!PipelineStateCache)
		{
			// If the cache doesn't exist, create it and register it with the appropriate cache directories.
			PipelineStateCache.Emplace();

			FScopeLock S(&AllThreadsLock);
			AllThreadsPipelineStateCache.Add(&PipelineStateCache.GetValue());
			if (CacheIndex == PipelineStateCache::RHIThreadIndex)
			{
				RHIThreadsPipelineStateCache.Add(&PipelineStateCache.GetValue());
			}
			else
			{
				RenderThreadsPipelineStateCache.Add(&PipelineStateCache.GetValue());
			}
		}

		// Return the selected cache.
		return PipelineStateCache.GetValue();
	}

#if PIPELINESTATECACHE_VERIFYTHREADSAFE
	struct FScopeVerifyIncrement
	{
		volatile int32& VerifyMutex;
		FScopeVerifyIncrement(volatile int32(&InVerifyMutex)[2]) : VerifyMutex(InVerifyMutex[PipelineStateCache::GetCacheIndexForCurrentThread()])
		{
			int32 Result = FPlatformAtomics::InterlockedIncrement(&VerifyMutex);
			if (Result <= 0)
			{
				UE_LOGF(LogRHI, Fatal, "Find was hit while Consolidate was running");
			}
		}

		~FScopeVerifyIncrement()
		{
			int32 Result = FPlatformAtomics::InterlockedDecrement(&VerifyMutex);
			if (Result < 0)
			{
				UE_LOGF(LogRHI, Fatal, "Find was hit while Consolidate was running");
			}
		}
	};

	struct FScopeVerifyDecrement
	{
		volatile int32& VerifyMutex;
		FScopeVerifyDecrement(volatile int32(&InVerifyMutex)[2]) : VerifyMutex(InVerifyMutex[PipelineStateCache::GetCacheIndexForCurrentThread()])
		{
			int32 Result = FPlatformAtomics::InterlockedDecrement(&VerifyMutex);
			if (Result >= 0)
			{
				UE_LOGF(LogRHI, Fatal, "Consolidate was hit while Get/SetPSO was running");
			}
		}

		~FScopeVerifyDecrement()
		{
			int32 Result = FPlatformAtomics::InterlockedIncrement(&VerifyMutex);
			if (Result != 0)
			{
				UE_LOGF(LogRHI, Fatal, "Consolidate was hit while Get/SetPSO was running");
			}
		}
	};
#endif

public:
	typedef TMap<TMyKey, TMyValue> FPipelineStateCacheType;

	TSharedPipelineStateCache() = default;

	bool Find(const TMyKey& InKey, TMyValue& OutResult)
	{
#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		FScopeVerifyIncrement S(VerifyMutex);
#endif

		FRWScopeLock InterruptGuard(InterruptLock, SLT_ReadOnly);

		// Select the maps to use.
		FPipelineStateCacheType* LocalCurrentMap = CurrentMap;
		FPipelineStateCacheType* LocalBackfillMap = BackfillMap;
		if (!IsInParallelRHIThread() && !IsInRHIThread())
		{
			if (GEnablePSOAsyncCacheConsolidation)
			{
				LocalCurrentMap = CurrentMap_RenderThread;
				LocalBackfillMap = BackfillMap_RenderThread;
			}
		}

		// safe because we only ever find when we don't add
		TMyValue* Result = LocalCurrentMap->Find(InKey);

		if (Result)
		{
			OutResult = *Result;
			return true;
		}

		// check the local cahce which is safe because only this thread adds to it
		TMap<TMyKey, TMyValue>& LocalCache = GetLocalCache();
		// if it's not in the local cache then it will rebuild
		Result = LocalCache.Find(InKey);
		if (Result)
		{
			OutResult = *Result;
			return true;
		}

		Result = LocalBackfillMap->Find(InKey);

		if (Result)
		{
			LocalCache.Add(InKey, *Result);
			OutResult = *Result;
			return true;
		}


		return false;
	}

	bool Add(const TMyKey& InKey, const TMyValue& InValue)
	{
#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		FScopeVerifyIncrement S(VerifyMutex);
#endif

		FRWScopeLock InterruptGuard(InterruptLock, SLT_ReadOnly);
		
		// everything is added to the local cache then at end of frame we consolidate them all
		TMap<TMyKey, TMyValue>& LocalCache = GetLocalCache();

		check(LocalCache.Contains(InKey) == false);
		LocalCache.Add(InKey, InValue);
		checkfSlow(LocalCache.Contains(InKey), TEXT("PSO not found immediately after adding.  Likely cause is an uninitialized field in a constructor or copy constructor"));
		return true;
	}

	void GetResources(TArray<TRefCountPtr<FRHIResource>>& OutResources, bool bConsolidateWithInterrupt, UE::FTimeout ConsolidationTimeout)
	{
		FRWScopeLock InterruptGuard(InterruptLock, SLT_Write);

		// Wait for any in-flight consolidation.
		// Consolidation is predicated on command context completion, allow for a timeout in case it's blocked
		if (bConsolidateWithInterrupt && WaitAndFinishAsyncCacheConsolidation(ConsolidationTimeout))
		{
			bIsInterrupt = true;
			
			// Kick off a new one
			FlushResources(false);
			WaitAndFinishAsyncCacheConsolidation(ConsolidationTimeout);

			bIsInterrupt = false;
		}
		
		for (auto&& [Desc, State] : *CurrentMap)
		{
			OutResources.Add(TRefCountPtr<FRHIResource>(State->RHIPipeline));
		} 
	}

	// Call from the Render Thread
	void FlushResources(bool bInDiscardAndSwap)
	{
		SCOPED_NAMED_EVENT(ConsolidateThreadedCaches, FColor::Turquoise);

		if (!bIsInterrupt)
		{
			InterruptLock.ReadLock();
		}
		
		ON_SCOPE_EXIT
		{
			if (!bIsInterrupt)
			{
				InterruptLock.ReadUnlock();
			}
		};

		bPendingDiscardAndSwap |= bInDiscardAndSwap;

#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		FScopeVerifyDecrement S(VerifyMutex);
#endif

		// Determine if the RHI THread is still consolidating its cache.
		if (RHICompletionEvent)
		{
			if (!RHICompletionEvent->IsComplete())
			{
				return;
			}
			RHICompletionEvent = nullptr;

			// Finish asynchronous cache consolidation.
			FinishAsyncCacheConsolidation();
		}

		bDiscardAndSwap = bPendingDiscardAndSwap;
		bPendingDiscardAndSwap = false;

		if (GEnablePSOAsyncCacheConsolidation)
		{
			// Determine if asynchronous cache consolidation was just enabled.
			if (CurrentMap_RenderThread->IsEmpty() && BackfillMap_RenderThread->IsEmpty())
			{
				// If the maps just happen to be empty, this will be cheap.
				OnAsyncConsolidationEnabled();
			}

			// Initiate an asynchronous cache consolidation.
			StartAsyncCacheConsolidation();
		}
		else
		{
			// Determine if asynchronous cache consolidation was just disabled.
			if (!CurrentMap_RenderThread->IsEmpty() || !BackfillMap_RenderThread->IsEmpty())
			{
				OnAsyncConsolidationDisabled();
			}

			// Synchronously consolidate all caches.
			ConsolidateThreadedCaches();
			ProcessDelayedCleanup();
			ReleasedEntries = 0;
			if (bDiscardAndSwap)
			{
				ReleasedEntries = DiscardAndSwap(CurrentMap, BackfillMap);
				bDiscardAndSwap = false;
			}
		}
	}

	void Shutdown()
	{
#if PIPELINESTATECACHE_VERIFYTHREADSAFE
		FScopeVerifyDecrement S(VerifyMutex);
#endif

		if (RHICompletionEvent)
		{
			RHICompletionEvent->Wait();
			RHICompletionEvent = nullptr;

			// Finish asynchronous cache consolidation.
			FinishAsyncCacheConsolidation();
		}

		// Determine if asynchronous cache consolidation was just disabled.
		if (!CurrentMap_RenderThread->IsEmpty() || !BackfillMap_RenderThread->IsEmpty())
		{
			OnAsyncConsolidationDisabled();
		}

		// Synchronously consolidate all caches.
		ConsolidateThreadedCaches();
		ProcessDelayedCleanup();

		// call discard twice to clear both the backing and main caches
		ReleasedEntries = DiscardAndSwap(CurrentMap, BackfillMap);
		ReleasedEntries += DiscardAndSwap(CurrentMap, BackfillMap);

		bDiscardAndSwap = false;
	}

	void WaitTasksComplete()
	{
		FScopeLock S(&AllThreadsLock);
		
		for (FPipelineStateCacheType* PipelineStateCache : AllThreadsPipelineStateCache)
		{
			WaitTasksComplete(PipelineStateCache);
		}
		
		WaitTasksComplete(BackfillMap);
		WaitTasksComplete(CurrentMap);

		WaitTasksComplete(BackfillMap_RenderThread);
		WaitTasksComplete(CurrentMap_RenderThread);
	}

	int32 NumReleasedEntries() const
	{
		return ReleasedEntries;
	}

private:
	bool WaitAndFinishAsyncCacheConsolidation(UE::FTimeout Timeout)
	{
		if (!RHICompletionEvent)
		{
			return true;
		}

		if (!RHICompletionEvent->FTaskBase::Wait(Timeout))
		{
			return false;
		}

		RHICompletionEvent = nullptr;
		FinishAsyncCacheConsolidation();
		return true;
	}

	void WaitTasksComplete(FPipelineStateCacheType* PipelineStateCache)
	{
		FScopeLock S(&AllThreadsLock);
		for (auto PipelineStateCacheIterator = PipelineStateCache->CreateIterator(); PipelineStateCacheIterator; ++PipelineStateCacheIterator)
		{
			auto PipelineState = PipelineStateCacheIterator->Value;
			if (PipelineState != nullptr)
			{
				PipelineState->WaitCompletion();
			}
		}
	}

	void OnAsyncConsolidationEnabled()
	{
		*CurrentMap_RenderThread = *CurrentMap;
		*BackfillMap_RenderThread = *BackfillMap;
	}

	void OnAsyncConsolidationDisabled()
	{
		// The render thread caches are the most up-to-date.
		Swap(CurrentMap, CurrentMap_RenderThread);
		Swap(BackfillMap, BackfillMap_RenderThread);
		CurrentMap_RenderThread->Reset();
		BackfillMap_RenderThread->Reset();

		// New Render Thread pipeline states have already been
		// consolidated into the Render Thread's maps.
		NewRenderThreadPipelineStates.Reset();
	}

	void ConsolidateThreadedCaches()
	{
		SCOPE_TIME_GUARD_MS(TEXT("ConsolidatePipelineCache"), 0.1);
		check(IsInRenderingThread());
		
		// consolidate all the local threads keys with the current thread
		// No one is allowed to call GetLocalCache while this is running
		// this is verified by the VerifyMutex.
		for (FPipelineStateCacheType* PipelineStateCache : AllThreadsPipelineStateCache)
		{
			for (auto PipelineStateCacheIterator = PipelineStateCache->CreateIterator(); PipelineStateCacheIterator; ++PipelineStateCacheIterator)
			{
				const TMyKey& ThreadKey = PipelineStateCacheIterator->Key;
				const TMyValue& ThreadValue = PipelineStateCacheIterator->Value;

				{
					TMyValue* CurrentValue = CurrentMap->Find(ThreadKey);
					if (CurrentValue)
					{
						check(!BackfillMap->Contains(ThreadKey));

						// if two threads get from the backfill map then we might just be dealing with one pipelinestate, in which case we have already added it to the currentmap and don't need to do anything else
						if (*CurrentValue != ThreadValue)
						{
							// otherwise we need to discard the duplicate.
							++DuplicateStateGenerated;
							DeleteArray.Add(ThreadValue);
						}
					}
					else
					{
						check(!BackfillMap->Contains(ThreadKey) || *BackfillMap->Find(ThreadKey) == ThreadValue);

						BackfillMap->Remove(ThreadKey);
						CurrentMap->Add(ThreadKey, ThreadValue);
						Uncompleted.Add(TTuple<TMyKey, TMyValue>(ThreadKey, ThreadValue));
					}
					PipelineStateCacheIterator.RemoveCurrent();
				}
			}
		}

		// tick and complete any uncompleted PSO tasks (we free up precompile tasks here).
		for (int32 i = Uncompleted.Num() - 1; i >= 0; i--)
		{
			checkSlow(CurrentMap->Find(Uncompleted[i].Key));
			if (Uncompleted[i].Value->IsComplete())
			{
				Uncompleted[i].Value->WaitCompletion();
				Uncompleted.RemoveAtSwap(i, EAllowShrinking::No);
			}
		}
	}

	void StartAsyncCacheConsolidation()
	{
		SCOPED_NAMED_EVENT(StartAsyncCacheConsolidation, FColor::Magenta);

		// Create an event to signal when the RHI cache conslidation completes.
		RHICompletionEvent = FGraphEvent::CreateGraphEvent();
		RHICompletionEvent->SetDebugName(TEXT("AsyncCacheConsolidation"));

		FGraphEventArray Prerequisites;
		if (!bIsInterrupt)
		{
			// Add the completion of the RHI cache consolidation as a prerequisite for the next RHI dispatch.
			GRHICommandList.AddNextDispatchPrerequisite(RHICompletionEvent);
			Prerequisites.Add(GRHICommandList.GetCompletionEvent());
		}

		// Flush Render Thread local caches and consolidate them into a single map.
		ConsolidatePipelineStates(NewRenderThreadPipelineStates, RenderThreadsPipelineStateCache);

		// Add new Render Thread pipeline states to the consolidated maps for the Render Thread.
		ConsolidateThreadCache(*CurrentMap_RenderThread, *BackfillMap_RenderThread, NewRenderThreadPipelineStates, true);

		// Enqueue an RHI cache consolidation task to execute when the last RHI submit completes.
		FFunctionGraphTask::CreateAndDispatchWhenReady(
			[this]()
			{
				FTaskTagScope TaskTagScope(ETaskTag::EParallelRhiThread);

				RHIAsyncCacheConsolidation();
			},
			TStatId(),
			&Prerequisites,
			ENamedThreads::AnyHiPriThreadHiPriTask);
	}

	void RHIAsyncCacheConsolidation()
	{
		SCOPED_NAMED_EVENT(RHIAsyncCacheConsolidation, FColor::Purple);
		{
#if PIPELINESTATECACHE_VERIFYTHREADSAFE
			FScopeVerifyDecrement S(VerifyMutex);
#endif

			// Add new Render Thread pipeline states to the consolidated maps for the RHI Thread.
			ConsolidateThreadCache(*CurrentMap, *BackfillMap, NewRenderThreadPipelineStates, false);

			// New Render Thread pipeline states have already been consolidated on the Render Thread.
			NewRenderThreadPipelineStates.Reset();

			// Flush RHI Thread local caches and consolidate them into a single map.
			ConsolidatePipelineStates(NewRHIThreadPipelineStates, RHIThreadsPipelineStateCache);

			// Add new RHI Thread pipeline states to the consolidated maps for the RHI Thread.
			ConsolidateThreadCache(*CurrentMap, *BackfillMap, NewRHIThreadPipelineStates, true);

			// Check for completed tasks.
			ManageIncompleteTasks();

			if (bDiscardAndSwap)
			{
				// The Render Thread will discard the contents of the backfill map.
				BackfillMap->Reset();

				DiscardAndSwap(CurrentMap, BackfillMap);
			}
		}

		// Signal that the RHI cache consolidation is complete.
		RHICompletionEvent->DispatchSubsequents();
	}

	void FinishAsyncCacheConsolidation()
	{
		SCOPED_NAMED_EVENT(FinishAsyncCacheConsolidation, FColor::Orange);

		// Add new RHI Thread pipeline states to the consolidated maps for the Render Thread.
		ConsolidateThreadCache(*CurrentMap_RenderThread, *BackfillMap_RenderThread, NewRHIThreadPipelineStates, false);

		// New RHI Thread pipeline states have already been consolidated on the RHI Thread.
		NewRHIThreadPipelineStates.Reset();

		// Flush Render Thread local caches and consolidate them into a single map.
		ConsolidatePipelineStates(NewRenderThreadPipelineStates, RenderThreadsPipelineStateCache);

		// Add new Render Thread pipeline states to the consolidated maps for the Render Thread.
		ConsolidateThreadCache(*CurrentMap_RenderThread, *BackfillMap_RenderThread, NewRenderThreadPipelineStates, true);

		// Check for completed tasks.
		ManageCompleteTasks();

		// Clean up duplicate tasks.
		ProcessDelayedCleanup();

		ReleasedEntries = 0;
		if (bDiscardAndSwap)
		{
			ReleasedEntries = DiscardAndSwap(CurrentMap_RenderThread, BackfillMap_RenderThread);

			bDiscardAndSwap = false;
		}
	}

	void ConsolidatePipelineStates(FPipelineStateCacheType& PipelineStates, TArray<FPipelineStateCacheType*>& ThreadsPipelineStateCache)
	{
		SCOPE_TIME_GUARD_MS(TEXT("ConsolidatePipelineStateCache"), 0.1);

		// Gather pipeline states generated in Render Thread tasks into a single map.
		// No Render Thread task is allowed to call GetLocalCache while this is running
		// this is verified by the VerifyMutex.
		for (FPipelineStateCacheType* PipelineStateCache : ThreadsPipelineStateCache)
		{
			for (auto PipelineStateCacheIterator = PipelineStateCache->CreateIterator(); PipelineStateCacheIterator; ++PipelineStateCacheIterator)
			{
				const TMyKey& ThreadKey = PipelineStateCacheIterator->Key;
				const TMyValue& ThreadValue = PipelineStateCacheIterator->Value;

				{
					TMyValue* CurrentValue = PipelineStates.Find(ThreadKey);
					if (CurrentValue)
					{
						// if two threads get from the backfill map then we might just be dealing with one pipelinestate,
						// in which case we have already added it to the map and don't need to do anything else
						if (*CurrentValue != ThreadValue)
						{
							// otherwise we need to discard the duplicate.
							++DuplicateStateGenerated;
							DeleteArray.Add(ThreadValue);
						}
					}
					else
					{
						PipelineStates.Add(ThreadKey, ThreadValue);
					}

					PipelineStateCacheIterator.RemoveCurrent();
				}
			}
		}
	}

	void ConsolidateThreadCache(FPipelineStateCacheType& CurrentPipelineStateMap, FPipelineStateCacheType& BackfillPipelineStateMap, FPipelineStateCacheType& NewPipelineStates, bool bCacheNewTasks)
	{
		SCOPE_TIME_GUARD_MS(TEXT("ConsolidateThreadCache"), 0.1);

		// consolidate all the new pipeline states with the state maps.
		// No one is allowed to call Add or Find while this is running
		// this is verified by the VerifyMutex.
		{
			for (auto PipelineStateCacheIterator = NewPipelineStates.CreateIterator(); PipelineStateCacheIterator; ++PipelineStateCacheIterator)
			{
				const TMyKey& ThreadKey = PipelineStateCacheIterator->Key;
				const TMyValue& ThreadValue = PipelineStateCacheIterator->Value;

				{
					TMyValue* CurrentValue = CurrentPipelineStateMap.Find(ThreadKey);
					if (CurrentValue)
					{
						check(!BackfillPipelineStateMap.Contains(ThreadKey));

						// if two threads get from the backfill map then we might just be dealing with one pipelinestate, in which case we have already added it to the currentmap and don't need to do anything else
						if (*CurrentValue != ThreadValue)
						{
							// otherwise we need to discard the duplicate.
							++DuplicateStateGenerated;
							DeleteArray.Add(ThreadValue);
							PipelineStateCacheIterator.RemoveCurrent();
						}
					}
					else
					{
						check(!BackfillPipelineStateMap.Contains(ThreadKey) || *BackfillPipelineStateMap.Find(ThreadKey) == ThreadValue);

						CurrentPipelineStateMap.Add(ThreadKey, ThreadValue);
						int32 Removed = BackfillPipelineStateMap.Remove(ThreadKey);
						if (Removed == 0 && bCacheNewTasks)
						{
							Uncompleted.Add(*PipelineStateCacheIterator);
						}
					}
				}
			}
		}
	}

	void ManageIncompleteTasks()
	{
		// tick and complete any uncompleted PSO tasks (we free up precompile tasks here).
		for (int32 i = Uncompleted.Num() - 1; i >= 0; i--)
		{
			checkSlow(CurrentMap->Find(Uncompleted[i].Key));
			if (Uncompleted[i].Value->IsComplete())
			{
				Uncompleted[i].Value->WaitCompletion();
				Completed.Add(Uncompleted[i]); // WaitCompletion must also be called on the Render Thread to ensure the CompletionState is destroyed.
				Uncompleted.RemoveAtSwap(i, EAllowShrinking::No);
			}
		}
	}

	void ManageCompleteTasks()
	{
		// tick Completed PSO tasks (we free up precompile tasks here).
		for (int32 i = Completed.Num() - 1; i >= 0; i--)
		{
			checkSlow(CurrentMap_RenderThread->Find(Completed[i].Key));
			checkSlow(Completed[i].Value->IsComplete());
			Completed[i].Value->WaitCompletion();
		}
		Completed.Reset();
	}

	template<typename F>
	void ExecuteImmediateCommand(F&& Functor)
	{
		if (IsInRenderingThread())
		{
			FRHICommandListImmediate::Get().EnqueueLambda([Functor](FRHICommandListImmediate& RHICmdList) mutable
			{
				Functor(RHICmdList);
			});
		}
		else
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady([Functor]() mutable
			{
				FRHICommandListImmediate::Get().EnqueueLambda([Functor = MoveTemp(Functor)](FRHICommandListImmediate& RHICmdList) mutable
				{
					Functor(RHICmdList);
				});
			}, TStatId{}, nullptr, ENamedThreads::ActualRenderingThread);
		}
	}

	void ProcessDelayedCleanup()
	{
		if (DeleteArray.IsEmpty())
		{
			return;
		}

		ExecuteImmediateCommand([DeleteArray = MoveTemp(DeleteArray)](FRHICommandListImmediate& RHICmdList) mutable
		{
			for (TMyValue& OldPipelineState : DeleteArray)
			{
				// Once in the delayed list this object should not be findable anymore, so the 0 should remain, making this safe
				OldPipelineState->Verify_NoUse();

				// Duplicate entries must wait for in progress compiles to complete.
				// inprogress tasks could also remain in this container and deferred for the next tick.
				bool bWaited = OldPipelineState->WaitCompletion();
				UE_CLOGF(bWaited, LogRHI, Log, "Waited on a pipeline compile task while discarding duplicate.");
				delete OldPipelineState;
			}
		});
	}

	int32 DiscardAndSwap(FPipelineStateCacheType*& InOutCurrentMap, FPipelineStateCacheType*& InOutBackfillMap)
	{
		// This should be very fast, if not it's likely eviction time is too high and too 
		// many items are building up.
		SCOPE_TIME_GUARD_MS(TEXT("TrimPiplelineCache"), 0.1);

		// the consolidate should always be run before the DiscardAndSwap.
		// there should be no inuse pipeline states in the backfill map (because they should have been moved into the CurrentMap).
		int32 Discarded = InOutBackfillMap->Num();
		if (Discarded > 0)
		{
			ExecuteImmediateCommand([DiscardMap = MoveTemp(*InOutBackfillMap)](FRHICommandListImmediate& RHICmdList) mutable
			{
				for (const auto& DiscardIterator : DiscardMap)
				{
					DiscardIterator.Value->Verify_NoUse();

					// Incomplete tasks should be put back to the current map. There should be no incomplete tasks encountered here.
					bool bWaited = DiscardIterator.Value->WaitCompletion();
					UE_CLOGF(bWaited, LogRHI, Error, "Waited on a pipeline compile task while discarding retired PSOs.");
					delete DiscardIterator.Value;
				}
			});
		}

		Swap(InOutBackfillMap, InOutCurrentMap);

		// keep alive incomplete tasks by moving them back to the current map.
		for (int32 i = Uncompleted.Num() - 1; i >= 0; i--)
		{
			int32 Removed = InOutBackfillMap->Remove(Uncompleted[i].Key);
			checkSlow(Removed);
			if (Removed)
			{
				InOutCurrentMap->Add(Uncompleted[i].Key, Uncompleted[i].Value);
			}
		}

		return Discarded;
	}

private:
	
	// The list of tasks that are still in progress.
	TArray<TTuple<TMyKey, TMyValue>> Uncompleted;
	TArray<TTuple<TMyKey, TMyValue>> Completed;

	uint32 TLSSlot{ FPlatformTLS::AllocTlsSlot() };

	FPipelineStateCacheType NewRenderThreadPipelineStates;
	FPipelineStateCacheType NewRHIThreadPipelineStates;

	FPipelineStateCacheType Maps[4];

	FPipelineStateCacheType* CurrentMap{ &Maps[0] };
	FPipelineStateCacheType* BackfillMap{ &Maps[1] };

	FPipelineStateCacheType* CurrentMap_RenderThread{ &Maps[2] };
	FPipelineStateCacheType* BackfillMap_RenderThread{ &Maps[3] };

	TArray<TMyValue> DeleteArray;

	FCriticalSection AllThreadsLock;
	TArray<FPipelineStateCacheType*> AllThreadsPipelineStateCache;
	TArray<FPipelineStateCacheType*> RenderThreadsPipelineStateCache;
	TArray<FPipelineStateCacheType*> RHIThreadsPipelineStateCache;

	/** Is an interrupt in progress? */
	volatile bool bIsInterrupt = false;

	/** While the pipeline cache activities are thread safe and mostly lock-free,
	 *  we may need to interrupt/lock the work to properly consolidate the pending caches. */
	FRWLock InterruptLock;

	FGraphEventRef RHICompletionEvent;

	int32 ReleasedEntries = 0;
	uint32 DuplicateStateGenerated = 0;

	bool bPendingDiscardAndSwap = false;
	bool bDiscardAndSwap = false;

#if PSO_TRACK_CACHE_STATS
	friend void DumpPipelineCacheStats();
#endif

#if PIPELINESTATECACHE_VERIFYTHREADSAFE
	volatile int32 VerifyMutex[2];
#endif
};

// Request state
enum class EPSOPrecacheStateMask : uint8
{
	None = 0,
	Compiling = 1 << 0, 		// once the start is scheduled
	Cached = 1 << 1, 			// PSO is cached in memory
	Uncached = 1 << 2,  		// PSO is released
	Failed = 1 << 3,    		// PSO failed to compile
	UsedForRendering = 1 << 4, 	// Set once this precached PSO is actually needed and used for rendering.
};

ENUM_CLASS_FLAGS(EPSOPrecacheStateMask)
static bool GForceHighToHighestPri = false;

template<class TPrecachePipelineCacheDerived, class TPrecachedPSOInitializer, class TPipelineState>
class TPrecachePipelineCacheBase
{
public:
	TPrecachePipelineCacheBase(FPSOPrecacheRequestID::EType InType, uint32 InMaxInMemoryPSOs)
		: PSOType(InType),
		MaxInMemoryPSOs(InMaxInMemoryPSOs)
	{
		if (ShouldKeepPrecachedPSOsInMemory() && InMaxInMemoryPSOs > 0)
		{
			InMemoryPSOIndices.Reserve(InMaxInMemoryPSOs);
		}
		PrecachedPSOsToCleanup.Reserve(100);
	}

	~TPrecachePipelineCacheBase()
	{
		// Wait for all precache tasks to have finished.
		WaitTasksComplete();
	}

	// Sets a new maximum number of precached PSOs kept in memory. FIFO order of currently tracked PSOs is maintained.
	// If the new maximum is smaller than the current maximum, the oldest PSOs are released.
	void SetMaxInMemoryPSOs(uint32 NewMaxInMemoryPSOs)
	{
		if (!ShouldKeepPrecachedPSOsInMemory() || MaxInMemoryPSOs == NewMaxInMemoryPSOs)
		{
			return;
		}

		FRWScopeLock WriteLock(PrecachePSOsRWLock, SLT_Write);
		MaxInMemoryPSOs = NewMaxInMemoryPSOs;
		InMemoryPSOIndices.Reserve(MaxInMemoryPSOs);

		// Release oldest PSOs.
		while (static_cast<uint32>(InMemoryPSOIndices.Num()) > MaxInMemoryPSOs)
		{
			uint32 PSOIndex = InMemoryPSOIndices.First();
			InMemoryPSOIndices.PopFirst();

			// Enqueue the corresponding PSO for cleanup.
			PrecachedPSOsToCleanup.Add(PrecachedPSOInitializers[PSOIndex]);
		}
	}
	
protected:

	void RescheduleTaskToNewPriority(EPSOPrecacheStateMask CurrentState, EPSOPrecachePriority NewPriority, EPSOPrecachePriority PrevPriority, TPipelineState* PipelineState)
	{
		check(PrevPriority != EPSOPrecachePriority::Highest);
		check((int32)NewPriority > (int32)PrevPriority);

		bool bCompleted = EnumHasAnyFlags(CurrentState, EPSOPrecacheStateMask::Failed | EPSOPrecacheStateMask::Cached);
		UE_CLOGF(bCompleted, LogRHI, Error, "pso request has completed? prev %x, new %x", (uint32)PrevPriority, (uint32)NewPriority);

		if (FPSOPrecacheThreadPool::UsePool())
		{
			check(PipelineState->GetPrecompileTask());
			if(PipelineState->GetPrecompileTask())
			{				
				EQueuedWorkPriority NewQueuePriority = GetQueuedWorkPriorityFromPSOPrecachePriority(NewPriority);
				if( PipelineState->GetPrecompileTask()->Reschedule(&GPSOPrecacheThreadPool.Get(), NewQueuePriority) )
				{
					PipelineState->CSVStat.SetQueuedPriority(NewPriority);
				}
			}
		}
	}

	FPSOPrecacheRequestResult TryAddNewState(const TPrecachedPSOInitializer& Initializer, const FPSOPrecacheRequestParams& RequestParams, bool bDoAsyncCompile)
	{
		// Refcounted and max in memory PSOs can't be used together
		EPSOPrecacheInMemoryMode InMemoryMode = ShouldKeepPrecachedPSOsInMemory() ? EPSOPrecacheInMemoryMode::Uncached : RequestParams.InMemoryMode;
		if (!ShouldKeepPrecachedPSOsForIHV())
		{
			InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
		}

		FPSOPrecacheRequestResult Result;
		uint64 InitializerHash = TPrecachePipelineCacheDerived::PipelineStateInitializerHash(Initializer);

		// Fast check first with read lock
		{
			FRWScopeLock ReadLock(PrecachePSOsRWLock, SLT_ReadOnly);
			if (IsCachedOrCompiling(Initializer, InitializerHash, InMemoryMode, Result))
			{
				return Result;
			}
		}

		// Now try and add with write lock
		TPipelineState* NewPipelineState = nullptr;
		{
			FRWScopeLock WriteLock(PrecachePSOsRWLock, SLT_Write);
			if (IsCachedOrCompiling(Initializer, InitializerHash, InMemoryMode, Result))
			{
				return Result;
			}

			// Check if it's uncached
			FPrecacheTask* FindResult = PrecachedPSOInitializerData.Find(InitializerHash);
			if (FindResult)
			{			
				check(!ShouldKeepPrecachedPSOsInMemory());
				check(InMemoryMode == EPSOPrecacheInMemoryMode::Cached);				

				// If no refs then it should be in the clean up list when it's not cached yet
				// (if it's still referenced then it goes from uncached to cached
				bool bInCleanupListWhenNotCached = FindResult->RefCount == 0;
				if (bInCleanupListWhenNotCached)
				{
					FPlatformAtomics::InterlockedIncrement(&FindResult->RefCount);
				}
				else
				{
					check(FindResult->InMemoryMode == EPSOPrecacheInMemoryMode::Uncached);
				}

				FindResult->InMemoryMode = EPSOPrecacheInMemoryMode::Cached;
					
				check(FindResult->RefCount == 1);
				Result.RequestID = FindResult->RequestID;
				check(Result.RequestID.IsValid());

				// If it's not uncached then it's still async compiling but requested for cleanup when done
				// Remove from the cleanup list again.
				EPSOPrecacheStateMask CurrentState = FindResult->ReadPSOPrecacheState();
				if (!EnumHasAnyFlags(CurrentState, EPSOPrecacheStateMask::Uncached))
				{	
					if (bInCleanupListWhenNotCached)
					{
						verify(PrecachedPSOsToCleanup.Remove(InitializerHash) == 1);
					}
					else
					{
						check(!PrecachedPSOsToCleanup.Contains(InitializerHash));
					}

					// Add async compile event if it's still compiling
					if (!IsCompilationDone(CurrentState))
					{
						Result.AsyncCompileEvent = FindResult->PipelineState->GetCompletionEvent();
					}
					return Result;
				}

				// Uncached - reset the state and kick the compilation again
				FindResult->ResetStateMask();
				FindResult->ResetPrecachePriority();

				check(!PrecachedPSOsToCleanup.Contains(InitializerHash));
			}
			else
			{
				// Add to array to get the new RequestID
				Result.RequestID.Type = (uint32)PSOType;
				Result.RequestID.RequestID = PrecachedPSOInitializers.Add(InitializerHash);

				// Add data to map
				FPrecacheTask PrecacheTask;
				PrecacheTask.RequestID = Result.RequestID;
				PrecacheTask.RefCount = ShouldKeepPrecachedPSOsInMemory() ? 0 : 1;
				PrecachedPSOInitializerData.Add(InitializerHash, PrecacheTask);

				FindResult = PrecachedPSOInitializerData.Find(InitializerHash);
				check(FindResult);

				// Increment total count
				if (PSOType == FPSOPrecacheRequestID::EType::Graphics)
				{
					INC_DWORD_STAT(STAT_TotalGraphicsPrecachedPSOs);
				}
				else
				{
					INC_DWORD_STAT(STAT_TotalComputePrecachedPSOs);
				}
			}
						
			// create new graphics state
			NewPipelineState = TPrecachePipelineCacheDerived::CreateNewPSO(Initializer);			
			check(FindResult->PipelineState == nullptr);
			FindResult->PipelineState = NewPipelineState;
			FindResult->InMemoryMode = InMemoryMode;
			verify(FindResult->AddPrecachePriority(RequestParams.PrecachePriority) == EPSOPrecachePriority::NotSet);
						
			if (bDoAsyncCompile)
			{
				// Assign the event at this point because we need to release the lock before calling OnNewPipelineStateCreated which
				// might call PrecacheFinished directly (The background task might get abandoned) and FRWLock can't be acquired recursively
				// Note that calling IsComplete will return false until we link it somehow like we do below
				NewPipelineState->SetCompletionEvent(FGraphEvent::CreateGraphEvent());
				Result.AsyncCompileEvent = NewPipelineState->GetCompletionEvent();

				UpdateActiveCompileCount(true /*Increment*/);
				UpdatePerPriorityCompileCount(RequestParams.PrecachePriority, true /*Increment*/);
			}

			if (ShouldKeepPrecachedPSOsInMemory())
			{
				if (MaxInMemoryPSOs > 0)
				{
					check(static_cast<uint32>(InMemoryPSOIndices.Num()) <= MaxInMemoryPSOs);

					// Evict the oldest PSO if we're at maximum capacity.
					if (InMemoryPSOIndices.Num() == MaxInMemoryPSOs)
					{
						uint32 PSOIndex = InMemoryPSOIndices.First();
						InMemoryPSOIndices.PopFirst();

						// Enqueue the corresponding PSO for cleanup.
						PrecachedPSOsToCleanup.Add(PrecachedPSOInitializers[PSOIndex]);
					}
					
					InMemoryPSOIndices.PushLast(Result.RequestID.RequestID);
				}
			}
		}

		TPrecachePipelineCacheDerived::OnNewPipelineStateCreated(Initializer, RequestParams, NewPipelineState, bDoAsyncCompile);

		// A boost request might have been issued while we were kicking the task, need to check it here
		{
			FRWScopeLock ReadLock(PrecachePSOsRWLock, SLT_ReadOnly);
			FPrecacheTask* FindResult = PrecachedPSOInitializerData.Find(InitializerHash);
			check(FindResult != nullptr);
			if (FindResult != nullptr)
			{
				EPSOPrecacheStateMask PreviousStateMask = FindResult->AddPSOPrecacheState(EPSOPrecacheStateMask::Compiling);

				// The current priority is the potentially new (higher) priority while RequestParams.PrecachePriority was the original requested priority
				// The priority might have changed via BoostPriority but not processed yet because the task might not have been kicked yet, but the internal
				// tracked priority if the precache task is already updated and checked here
				// NOTE: Priority can only modified via write lock in SetPrecachePriorityInternal so safe to read here with read lock
				EPSOPrecachePriority CurrentPriority = FindResult->GetPrecachePriority();

				// by the time we're here, PrecacheFinished might already have been called, so only update priority if it will still be processed
				if (!IsCompilationDone(PreviousStateMask) && CurrentPriority != RequestParams.PrecachePriority)
				{
					RescheduleTaskToNewPriority(PreviousStateMask, CurrentPriority, RequestParams.PrecachePriority, FindResult->PipelineState);
				}
			}
		}

		// Make sure that we don't access NewPipelineState here as the task might have already been finished, ProcessDelayedCleanup may have been called
		// and NewPipelineState already been deleted
		
		return Result;
	}
public:

	void WaitTasksComplete()
	{
		// We hold the lock to observe task state, releasing it if tasks are still in flight
		// precache tasks may also attempt to lock PrecachePSOsRWLock (TPrecachePipelineCacheBase::PrecacheFinished).
		// TODO: Replace all of this spin wait.
		bool bTasksWaiting = true;
		bool bFirstPass = true;
		while(bTasksWaiting)
		{
			bTasksWaiting = false;
			{
				FRWScopeLock WriteLock(PrecachePSOsRWLock, SLT_Write);
				for (auto Iterator = PrecachedPSOInitializerData.CreateIterator(); Iterator; ++Iterator)
				{
					FPrecacheTask& PrecacheTask = Iterator->Value;
					if (PrecacheTask.PipelineState && !PrecacheTask.PipelineState->IsComplete())
					{
						bTasksWaiting = true;
						if (bFirstPass)
						{
							// boost the priority of anything we're waiting for to improve throughput.
							// ensurecomplete is guaranteed to complete, but We're not explicitly waiting for precache tasks here.
							// So lower pri tasks may never complete when r.PSOPrecaching.TaskPriorityThreshold is in use.
							if (PrecacheTask.GetPrecachePriority() != EPSOPrecachePriority::Highest)
							{
								SetPrecachePriorityInternal(PrecacheTask, EPSOPrecachePriority::Highest);
							}
						}
						else
						{
							// everything should be boosted on the second pass.
							check(PrecacheTask.GetPrecachePriority() == EPSOPrecachePriority::Highest);
							break; // release PrecachePSOsRWLock so as to avoid any further blocking of in-progress tasks.
						}
					}
					else if (PrecacheTask.PipelineState)
					{
						check(EnumHasAnyFlags(PrecacheTask.ReadPSOPrecacheState(), (EPSOPrecacheStateMask::Cached | EPSOPrecacheStateMask::Failed)));
						delete PrecacheTask.PipelineState;
						PrecacheTask.PipelineState = nullptr;
					}
				}
				bFirstPass = false;
				if (!bTasksWaiting)
				{
					PrecachedPSOsToCleanup.Empty();
				}
			}
			if (bTasksWaiting)
			{
				// Yield while we wait.
				FPlatformProcess::Sleep(0.01f);
			}
		}
	}

	bool ReleasePrecachedPSO(const FPSOPrecacheRequestID& RequestID)
	{
		check(RequestID.GetType() == PSOType);
		check(!ShouldKeepPrecachedPSOsInMemory());

		FRWScopeLock WriteLock(PrecachePSOsRWLock, SLT_Write);		
		if ((int32)RequestID.RequestID >= PrecachedPSOInitializers.Num())
		{
			return false;
		}

		uint64 InitializerHash = PrecachedPSOInitializers[RequestID.RequestID];
		FPrecacheTask* FindResult = PrecachedPSOInitializerData.Find(InitializerHash);
		check(FindResult);
		if (FindResult->InMemoryMode == EPSOPrecacheInMemoryMode::Cached)
		{
			EPSOPrecacheStateMask CurrentState = FindResult->ReadPSOPrecacheState();
			if (!EnumHasAnyFlags(CurrentState, EPSOPrecacheStateMask::Uncached))
			{
				check(FindResult->RefCount > 0);
				int32 NewRefCount = FPlatformAtomics::InterlockedDecrement(&FindResult->RefCount);
				if (NewRefCount == 0)
				{
					PrecachedPSOsToCleanup.Add(InitializerHash);
					return true;
				}
			}
			else
			{
				check(FindResult->RefCount == 0);
			}
		}
		
		return false;
	}

	EPSOPrecacheResult GetPrecachingState(const FPSOPrecacheRequestID& RequestID)
	{
		check(RequestID.GetType() == PSOType);

		uint64 InitializerHash = 0u;
		{
			FRWScopeLock ReadLock(PrecachePSOsRWLock, SLT_ReadOnly);
			InitializerHash = PrecachedPSOInitializers[RequestID.RequestID];
		}
		bool bMarkUsed = false;
		bool bOutMarkedUsed = false;
		FGraphEventRef CompletionEvent;
		FPSOPrecacheRequestID PSOPrecacheRequestID;
		EPSOPrecacheInMemoryMode InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
		return GetPrecachingStateInternal(InitializerHash, bMarkUsed, bOutMarkedUsed, CompletionEvent, PSOPrecacheRequestID, InMemoryMode);
	}

	EPSOPrecacheResult GetPrecachingState(const TPrecachedPSOInitializer& Initializer, FGraphEventRef& OutCompletionEvent, FPSOPrecacheRequestID& OutPSOPrecacheRequestID)
	{
		uint64 InitializerHash = TPrecachePipelineCacheDerived::PipelineStateInitializerHash(Initializer);
		bool bMarkUsed = false;
		bool bOutMarkedUsed = false;
		EPSOPrecacheInMemoryMode InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
		return GetPrecachingStateInternal(InitializerHash, bMarkUsed, bOutMarkedUsed, OutCompletionEvent, OutPSOPrecacheRequestID, InMemoryMode);
	}

	uint64 GetPSOCompilationTime(uint64 PSOPrecacheHash)
	{
#if !UE_BUILD_SHIPPING
		FRWScopeLock ReadLock(PrecachePSOsRWLock, SLT_ReadOnly);
		FPrecacheTask* FindResult = PrecachedPSOInitializerData.Find(PSOPrecacheHash);
		return FindResult ? FindResult->PSOCompilationTime : 0;
#else
		return 0;
#endif
	}

	// Retrieves the precaching state for the given PSO, and marks it as used if not already marked.
	EPSOPrecacheResult GetPrecachingStateAndMarkUsed(const TPrecachedPSOInitializer& Initializer, bool& bOutMarkedUsed)
	{
		uint64 InitializerHash = TPrecachePipelineCacheDerived::PipelineStateInitializerHash(Initializer);
		bool bMarkUsed = true;
		FGraphEventRef CompletionEvent;		
		FPSOPrecacheRequestID PSOPrecacheRequestID;
		EPSOPrecacheInMemoryMode InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
		return GetPrecachingStateInternal(InitializerHash, bMarkUsed, bOutMarkedUsed, CompletionEvent, PSOPrecacheRequestID, InMemoryMode);
	}

	EPSOPrecacheResult GetPrecachingStateAndInMemoryMode(const TPrecachedPSOInitializer& Initializer, EPSOPrecacheInMemoryMode& OutInMemoryMode)
	{
		uint64 InitializerHash = TPrecachePipelineCacheDerived::PipelineStateInitializerHash(Initializer);
		bool bMarkUsed = false;
		bool bOutMarkedUsed = false;
		FGraphEventRef CompletionEvent;		
		FPSOPrecacheRequestID PSOPrecacheRequestID;
		return GetPrecachingStateInternal(InitializerHash, bMarkUsed, bOutMarkedUsed, CompletionEvent, PSOPrecacheRequestID, OutInMemoryMode);
	}

	bool IsPrecaching()
	{
		FRWScopeLock ReadLock(PrecachePSOsRWLock, SLT_ReadOnly);
		return ActiveCompileCount != 0;
	}

	void BoostPriority(EPSOPrecachePriority PSOPrecachePriority, const FPSOPrecacheRequestID& RequestID)
	{
		check(RequestID.GetType() == PSOType);

		// Won't modify anything in this cache so readlock should be enough?
		FRWScopeLock ReadLock(PrecachePSOsRWLock, SLT_ReadOnly);
		uint64 InitializerHash = PrecachedPSOInitializers[RequestID.RequestID];
		FPrecacheTask* FindResult = PrecachedPSOInitializerData.Find(InitializerHash);
		check(FindResult);

		SetPrecachePriorityInternal(*FindResult, PSOPrecachePriority);
	}

	uint32 NumActivePrecacheRequests()
	{
		// Take process priority into account as well, because otherwise lower priority requests will never be compiled when we try
		// to wait for them
		int32 MinPriorityForActivePrecacheRequests = FMath::Max(GPSOPrecacheProcessPriorityThreshold, GPSOPrecacheMinPriorityForActivePrecacheRequests);
		if (MinPriorityForActivePrecacheRequests <= 0)
		{
			return FPlatformAtomics::AtomicRead(&ActiveCompileCount);
		}

		uint32 ActivePrecacheRequests = 0;
		for (int32 Priority = MinPriorityForActivePrecacheRequests; Priority < (int32)EPSOPrecachePriority::Num; ++Priority)
		{
			ActivePrecacheRequests += FPlatformAtomics::AtomicRead(&PerPriorityCompileCount[Priority]);
		}
		return ActivePrecacheRequests;
	}

	void PrecacheFinished(const TPrecachedPSOInitializer& Initializer, bool bValid, uint64 PSOCompilationTime)
	{
		uint64 InitializerHash = TPrecachePipelineCacheDerived::PipelineStateInitializerHash(Initializer);

		EPSOPrecacheStateMask PreviousStateMask = EPSOPrecacheStateMask::None;	
		EPSOPrecachePriority StatePriority = EPSOPrecachePriority::Medium;

		{
			FRWScopeLock WriteLock(PrecachePSOsRWLock, SLT_Write);

			// Mark compiled (either succeeded or failed)
			FPrecacheTask* FindResult = PrecachedPSOInitializerData.Find(InitializerHash);
			check(FindResult);

#if !UE_BUILD_SHIPPING
			// Store the compilation time for stat tracking (only store the first compilation time - otherwise coming from driver cache)
			if (FindResult->PSOCompilationTime == 0)
			{
				FindResult->PSOCompilationTime = PSOCompilationTime;
			}
#endif

			FindResult->PipelineState->CSVStat.PrecacheFinished();

			// We still add the 'compiling' bit here because if the task is fast enough, we can get here before the end of TryAddNewState
			const EPSOPrecacheStateMask CompleteStateMask = bValid ? (EPSOPrecacheStateMask::Cached | EPSOPrecacheStateMask::Compiling) : (EPSOPrecacheStateMask::Failed | EPSOPrecacheStateMask::Compiling);
			PreviousStateMask = FindResult->AddPSOPrecacheState(CompleteStateMask);

			StatePriority = FindResult->GetPrecachePriority();

			// Release immediately if not kept in memory
			if (FindResult->InMemoryMode != EPSOPrecacheInMemoryMode::Cached && !ShouldKeepPrecachedPSOsInMemory())
			{
				if (!ShouldKeepPrecachedPSOsInMemory())
				{
					check(FindResult->RefCount == 1);
					FPlatformAtomics::InterlockedDecrement(&FindResult->RefCount);
				}
				PrecachedPSOsToCleanup.Add(InitializerHash);
			}

			UpdateActiveCompileCount(false /*Increment*/);
			UpdatePerPriorityCompileCount(StatePriority, false /*Increment*/);
		}

		// Log this PSO to the pipeline file cache as 'Active' (precached, not yet seen at draw time).
		// Whether it is actually stored is controlled by r.ShaderPipelineCache.IncludeAllPrecachedPSOs.
		// The result will be upgraded to the real runtime outcome if the PSO is later needed for rendering.
		if (bValid)
		{
			TPrecachePipelineCacheDerived::CacheToFileCache(Initializer);
		}

		// Mark as in memory now - if it's not cached then it was added to list for cleanup and will be subtracted there
		if (PSOType == FPSOPrecacheRequestID::EType::Graphics)
		{
			INC_DWORD_STAT(STAT_PrecachedGraphicsPrecachedPSOsInMemory);
		}
		else
		{
			INC_DWORD_STAT(STAT_PrecachedComputePrecachedPSOsInMemory);
		}
	}

	static bool IsCompilationDone(EPSOPrecacheStateMask StateMask)
	{
		// When marked uncached then compilation is done as well but released again so valid in the driver cache but not runtime cached
		return EnumHasAnyFlags(StateMask, EPSOPrecacheStateMask::Cached | EPSOPrecacheStateMask::Uncached | EPSOPrecacheStateMask::Failed);
	}

	void ProcessDelayedCleanup()
	{
		SET_DWORD_STAT_FName(TPrecachePipelineCacheDerived::GetActiveCompileStatName(), ActiveCompileCount);
		for (int32 PrecachePriority = 0; PrecachePriority < (int32)EPSOPrecachePriority::Num; ++PrecachePriority)
		{
			SET_DWORD_STAT_FName(TPrecachePipelineCacheDerived::GetPerPriorityCompileStatName((EPSOPrecachePriority)PrecachePriority), PerPriorityCompileCount[PrecachePriority]);
		}

		FRWScopeLock WriteLock(PrecachePSOsRWLock, SLT_Write);
		for (int32 Index = 0; Index < PrecachedPSOsToCleanup.Num(); ++Index)
		{
			uint64 InitializerHash = PrecachedPSOsToCleanup[Index];

			FPrecacheTask* FindResult = PrecachedPSOInitializerData.Find(InitializerHash);
			check(FindResult && FindResult->RefCount == 0);
			if (!FindResult || !FindResult->PipelineState)
			{
				// Was already cleaned up (can happen if it was marked as used).
				PrecachedPSOsToCleanup.RemoveAtSwap(Index);
				Index--;
			}
			else if (FindResult->PipelineState->IsComplete())
			{
				// This is needed to cleanup the members - bit strange because it's complete already
				verify(!FindResult->PipelineState->WaitCompletion());
				delete FindResult->PipelineState;
				FindResult->PipelineState = nullptr;

				// Mark as uncached
				FindResult->AddPSOPrecacheState(EPSOPrecacheStateMask::Uncached);
				FindResult->RemovePSOPrecacheState(EPSOPrecacheStateMask::Cached);

				// Update in memory stats
				if (PSOType == FPSOPrecacheRequestID::EType::Graphics)
				{
					DEC_DWORD_STAT(STAT_PrecachedGraphicsPrecachedPSOsInMemory);
				}
				else
				{
					DEC_DWORD_STAT(STAT_PrecachedComputePrecachedPSOsInMemory);
				}

				PrecachedPSOsToCleanup.RemoveAtSwap(Index);
				Index--;
			}
			else
			{
				// Compilation still busy - try and cancel?
			}
		}
	}

	// Marks a PSO as used for rendering so that it will be enqueued for cleanup if it's currently kept in memory.
	// Does not do anything if PSOs are not kept in memory, or if a bound on the number of PSOs kept in memory is set.
	void MarkPSOAsUsed(const TPrecachedPSOInitializer& Initializer)
	{
		if (!ShouldTrackUsedPrecachedPSOs())
		{
			return;
		}

		uint64 InitializerHash = TPrecachePipelineCacheDerived::PipelineStateInitializerHash(Initializer);

		FRWScopeLock WriteLock(PrecachePSOsRWLock, SLT_Write);
		PrecachedPSOsToCleanup.Add(InitializerHash);

		// Should also remove the index from InMemoryPSOIndices but it's a deque so can't be done fast
	}

protected:
	bool IsCachedOrCompiling(const TPrecachedPSOInitializer& Initializer, uint64 InitializerHash, EPSOPrecacheInMemoryMode InMemoryMode, FPSOPrecacheRequestResult& Result)
	{
		FPrecacheTask* FindResult = PrecachedPSOInitializerData.Find(InitializerHash);

		// Found and not uncached?
		if (FindResult)
		{
			if (!ShouldKeepPrecachedPSOsInMemory())
			{
				// If it's uncached and want to cache then return false because needs to be processed with write lock
				if (FindResult->InMemoryMode != InMemoryMode && FindResult->InMemoryMode == EPSOPrecacheInMemoryMode::Uncached)
				{
					return false;
				}
				
				// Special ref count tracking when kept in memory
				if (InMemoryMode == EPSOPrecacheInMemoryMode::Cached)
				{
					// If it's uncached then return false because it should be compiled again
					if (EnumHasAnyFlags(FindResult->ReadPSOPrecacheState(), EPSOPrecacheStateMask::Uncached))
					{
						return false;
					}

					// If refcount 0 then it means it's in the cleanup list and needs to be removed there with write lock (only have read lock here)
					// only location where ref count is incremented without write lock is below
					if (FindResult->RefCount == 0)
					{
						return false;
					}

					// Find in cache and still referenced by something else
					FPlatformAtomics::InterlockedIncrement(&FindResult->RefCount);
					check(!PrecachedPSOsToCleanup.Contains(InitializerHash));
				}
			}
			
			// Add async compile event if it's still compiling
			if (!IsCompilationDone(FindResult->ReadPSOPrecacheState()))
			{
				Result.AsyncCompileEvent = FindResult->PipelineState->GetCompletionEvent();
			}

			Result.RequestID = FindResult->RequestID;
			check(Result.RequestID.IsValid());

			return true;
		}

		return false;
	}

	EPSOPrecacheResult GetPrecachingStateInternal(uint64 InitializerHash, bool bTryMarkUsed, bool& bOutMarkedUsed, FGraphEventRef& OutCompletionEvent, FPSOPrecacheRequestID& OutPSOPrecacheRequestID, EPSOPrecacheInMemoryMode& OutInMemoryMode)
	{
		EPSOPrecacheStateMask CompilationState = EPSOPrecacheStateMask::None;
		FGraphEventRef AsyncCompileEvent;
		OutInMemoryMode = EPSOPrecacheInMemoryMode::Uncached;

		{
			FRWScopeLock ReadLock(PrecachePSOsRWLock, SLT_ReadOnly);
			FPrecacheTask* FindResult = PrecachedPSOInitializerData.Find(InitializerHash);
			if (FindResult == nullptr)
			{
				return EPSOPrecacheResult::Missed;
			}

			AsyncCompileEvent = FindResult->PipelineState ? FindResult->PipelineState->GetCompletionEvent() : nullptr;
			OutPSOPrecacheRequestID = FindResult->RequestID;

			if (bTryMarkUsed)
			{
				CompilationState = FindResult->AddPSOPrecacheState(EPSOPrecacheStateMask::UsedForRendering);
			}
			else
			{
				CompilationState = FindResult->ReadPSOPrecacheState();
			}

			OutInMemoryMode = FindResult->InMemoryMode;
		}

		// We successfully marked it as used if nobody else did concurrently.
		bOutMarkedUsed = bTryMarkUsed && !EnumHasAnyFlags(CompilationState, EPSOPrecacheStateMask::UsedForRendering);

		if (!IsCompilationDone(CompilationState))
		{
			OutCompletionEvent = AsyncCompileEvent;
			return EPSOPrecacheResult::Active;
		}

		// check we only set 1 completion bit
		const EPSOPrecacheStateMask CompletionMask = EPSOPrecacheStateMask::Cached | EPSOPrecacheStateMask::Failed | EPSOPrecacheStateMask::Uncached;
		check(EnumHasAnyFlags(CompilationState, CompletionMask) && !EnumHasAllFlags(CompilationState, CompletionMask));

		return (EnumHasAnyFlags(CompilationState, EPSOPrecacheStateMask::Failed)) ? EPSOPrecacheResult::NotSupported : EPSOPrecacheResult::Complete;
	}

	void UpdateActiveCompileCount(bool bIncrement)
	{
		if (bIncrement)
		{
			FPlatformAtomics::InterlockedIncrement(&ActiveCompileCount);
		}
		else
		{
			FPlatformAtomics::InterlockedDecrement(&ActiveCompileCount);
		}
	}

	void UpdatePerPriorityCompileCount(EPSOPrecachePriority PrecachePriority, bool bIncrement)
	{
		if (bIncrement)
		{
			FPlatformAtomics::InterlockedIncrement(&PerPriorityCompileCount[(int32)PrecachePriority]);
		}
		else
		{
			FPlatformAtomics::InterlockedDecrement(&PerPriorityCompileCount[(int32)PrecachePriority]);
			ensure(PerPriorityCompileCount[(int32)PrecachePriority] >= 0);
		}
	}

	// Type of PSOs which the cache manages
	FPSOPrecacheRequestID::EType PSOType;

	// Cache of all precached PSOs - need correct compare to make sure we precache all of them
	FRWLock PrecachePSOsRWLock;

	// Array containing all the precached PSO initializers thus far - the index in this array is used to uniquely identify the PSO requests
	TArray<uint64> PrecachedPSOInitializers;

	// Hash map used for fast retrieval of already precached PSOs
	struct FPrecacheTask
	{
		TPipelineState* PipelineState = nullptr;
		FPSOPrecacheRequestID RequestID;
		volatile int32 RefCount = 0;
		EPSOPrecacheInMemoryMode InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
#if !UE_BUILD_SHIPPING
		uint64 PSOCompilationTime = 0;
#endif

		EPSOPrecacheStateMask AddPSOPrecacheState(EPSOPrecacheStateMask DesiredState)
		{
			static_assert(sizeof(EPSOPrecacheStateMask) == sizeof(int8), "Fix the cast below");
			return (EPSOPrecacheStateMask)FPlatformAtomics::InterlockedOr((volatile int8*)&StateMask, (int8)DesiredState);
		}

		EPSOPrecacheStateMask RemovePSOPrecacheState(EPSOPrecacheStateMask StateToRemove)
		{
			static_assert(sizeof(EPSOPrecacheStateMask) == sizeof(int8), "Fix the cast below");
			return (EPSOPrecacheStateMask)FPlatformAtomics::InterlockedAnd((volatile int8*)&StateMask, (int8)~StateToRemove);
		}

		EPSOPrecacheStateMask ResetStateMask()
		{
			static_assert(sizeof(EPSOPrecacheStateMask) == sizeof(int8), "Fix the cast below");
			return (EPSOPrecacheStateMask)FPlatformAtomics::InterlockedExchange((volatile int8*)&StateMask, (int8)EPSOPrecacheStateMask::None);
		}

		void ResetPrecachePriority()
		{
			FPlatformAtomics::InterlockedExchange((volatile int8*)&PrecachePriorityBitSet, 0);
		}

		inline EPSOPrecacheStateMask ReadPSOPrecacheState()
		{
			static_assert(sizeof(EPSOPrecacheStateMask) == sizeof(int8), "Fix the cast below");
			return (EPSOPrecacheStateMask)FPlatformAtomics::AtomicRead((volatile int8*)&StateMask);
		}

		inline EPSOPrecachePriority GetPrecachePriority()
		{
			int8 BitSet = FPlatformAtomics::AtomicRead((volatile int8*)&PrecachePriorityBitSet);
			check(BitSet != 0);
			return (EPSOPrecachePriority)FMath::FloorLog2(BitSet);
		}

		inline EPSOPrecachePriority AddPrecachePriority(EPSOPrecachePriority NewPriority)
		{
			int8 PreviousBitSet = FPlatformAtomics::InterlockedOr((volatile int8*)&PrecachePriorityBitSet, (1<<(int8)NewPriority));
			if (PreviousBitSet == 0)
			{
				return EPSOPrecachePriority::NotSet;
			}
			else
			{
				return (EPSOPrecachePriority)FMath::FloorLog2(PreviousBitSet);
			}
		}

	private:
		volatile EPSOPrecacheStateMask StateMask = EPSOPrecacheStateMask::None;
		volatile int8 PrecachePriorityBitSet = 0;
	};

	TMap<uint64, FPrecacheTask> PrecachedPSOInitializerData;

	// Total number of open active compiles
	volatile int32 ActiveCompileCount = 0;

	// Number of open compiles per priority type
	volatile int32 PerPriorityCompileCount[(int32)EPSOPrecachePriority::Num] = { 0, 0, 0, 0 };

	// Finished Precached PSOs which can be garbage collected
	TArray<uint64> PrecachedPSOsToCleanup;

	// Circular buffer of indices of PSOs that are kept in memory if MaxInMemoryPSOs is non-zero.
	TDeque<uint32> InMemoryPSOIndices;
	uint32 MaxInMemoryPSOs = 0;

	void SetPrecachePriorityInternal(FPrecacheTask& FindResult, EPSOPrecachePriority NewPSOPrecachePriority)
	{
		// Make sure compilation is not done yet (written via write lock)
		EPSOPrecacheStateMask CurrentState = FindResult.ReadPSOPrecacheState();
		if (!IsCompilationDone(CurrentState))
		{			
			// Ignore requests which try to lower the priority again or set to the same one
			// (can happen with material side boost but PSO already boosted to highest because required for rendering by other component)		
			EPSOPrecachePriority PreviousPriority = FindResult.AddPrecachePriority(NewPSOPrecachePriority);		
			if ((int32)PreviousPriority >= (int32)NewPSOPrecachePriority)
			{
				return;
			}

			// Update the stats here even though the reschedule might not be processed yet because it can happen
			// that the actual reschedule never happens if it's done by the time the TryAddNewState actually
			// tried the reschedule and stats need to be correct to process process finished call
			UpdatePerPriorityCompileCount(PreviousPriority, false);
			UpdatePerPriorityCompileCount(NewPSOPrecachePriority, true);

			// It's possible to get a priority change request while the task has not been started yet. In this case, TryAddNewState will take care of it
			// if TryAddNewState is done, then we can proceed to process the request it, if the task is not done yet
			if (EnumHasAnyFlags(CurrentState, EPSOPrecacheStateMask::Compiling))
			{
				RescheduleTaskToNewPriority(CurrentState, NewPSOPrecachePriority, PreviousPriority, FindResult.PipelineState);
			}
		}
	}
};

class FPrecacheComputePipelineCache : public TPrecachePipelineCacheBase<FPrecacheComputePipelineCache, FComputePipelineStateInitializer, FComputePipelineState>
{
public:
	static FComputePipelineState* CreateNewPSO(const FComputePipelineStateInitializer& ComputeShaderInitializer)
	{
		bool bRuntimePSO = false;
		bool bFromInMemoryPSOCache = false;
		return new FComputePipelineState(ComputeShaderInitializer.ComputeShader, bRuntimePSO, bFromInMemoryPSOCache);
	}

	FPrecacheComputePipelineCache(uint32 InMemoryPSOsMaxNum) : TPrecachePipelineCacheBase(FPSOPrecacheRequestID::EType::Compute, InMemoryPSOsMaxNum) {}
	FPSOPrecacheRequestResult PrecacheComputePipelineState(const FComputePipelineStateInitializer& ComputeInitializer, const FPSOPrecacheRequestParams& RequestParams);
	static void OnNewPipelineStateCreated(const FComputePipelineStateInitializer& ComputeInitializer, const FPSOPrecacheRequestParams& RequestParams, FComputePipelineState* NewComputePipelineState, bool bDoAsyncCompile);
 
	static const FName GetActiveCompileStatName()
	{
		return GET_STATFNAME(STAT_ActiveComputePSOPrecacheRequests);
	}
	static const FName GetPerPriorityCompileStatName(EPSOPrecachePriority PrecachePriority)
	{
		switch (PrecachePriority)
		{
			case EPSOPrecachePriority::Highest: return GET_STATFNAME(STAT_HighestPriorityComputePSOPrecacheRequests);
			case EPSOPrecachePriority::High:  	return GET_STATFNAME(STAT_HighPriorityComputePSOPrecacheRequests);
			case EPSOPrecachePriority::Medium:  return GET_STATFNAME(STAT_MediumPriorityComputePSOPrecacheRequests);
			default: checkNoEntry(); [[fallthrough]];
			case EPSOPrecachePriority::Low:  	return GET_STATFNAME(STAT_LowPriorityComputePSOPrecacheRequests);
		}

		checkNoEntry();
	}

	static FORCEINLINE uint64 PipelineStateInitializerHash(const FComputePipelineStateInitializer& Key)
	{
		return HashCombineFast(GetTypeHash(Key.ComputeShader->GetHash()), GetTypeHash(Key.SpecializationConstants));
	}

	static void CacheToFileCache(const FComputePipelineStateInitializer& Initializer)
	{
		FPipelineFileCacheManager::CacheComputePSO(GetTypeHash(Initializer), Initializer, EPSOPrecacheResult::Active, nullptr);
	}
};

class FPrecacheGraphicsPipelineCache : public TPrecachePipelineCacheBase<FPrecacheGraphicsPipelineCache, FGraphicsPipelineStateInitializer, FGraphicsPipelineState>
{
public:

	static FGraphicsPipelineState* CreateNewPSO(const FGraphicsPipelineStateInitializer& Initializer)
	{
		bool bRuntimePSO = false;
		bool bFromInMemoryPSOCache = false;
		return new FGraphicsPipelineState(bRuntimePSO, bFromInMemoryPSOCache);
	}

	static FORCEINLINE uint64 PipelineStateInitializerHash(const FGraphicsPipelineStateInitializer& Key)
	{
		return RHIComputePrecachePSOHash(Key);
	}

	static void CacheToFileCache(const FGraphicsPipelineStateInitializer& Initializer)
	{
		FPipelineFileCacheManager::CacheGraphicsPSO(GetTypeHash(Initializer), Initializer, EPSOPrecacheResult::Active, nullptr);
	}

	static const FName GetActiveCompileStatName()
	{
		return GET_STATFNAME(STAT_ActiveGraphicsPSOPrecacheRequests);
	}
	static const FName GetPerPriorityCompileStatName(EPSOPrecachePriority PrecachePriority)
	{
		switch (PrecachePriority)
		{
			case EPSOPrecachePriority::Highest: return GET_STATFNAME(STAT_HighestPriorityGraphicsPSOPrecacheRequests);
			case EPSOPrecachePriority::High:  	return GET_STATFNAME(STAT_HighPriorityGraphicsPSOPrecacheRequests);
			case EPSOPrecachePriority::Medium:  return GET_STATFNAME(STAT_MediumPriorityGraphicsPSOPrecacheRequests);
			default: checkNoEntry(); [[fallthrough]];
			case EPSOPrecachePriority::Low:  	return GET_STATFNAME(STAT_LowPriorityGraphicsPSOPrecacheRequests);
		}

		checkNoEntry();
	}
	FPrecacheGraphicsPipelineCache(uint32 InMemoryPSOsMaxNum) : TPrecachePipelineCacheBase(FPSOPrecacheRequestID::EType::Graphics, InMemoryPSOsMaxNum) {}
	FPSOPrecacheRequestResult PrecacheGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer, const FPSOPrecacheRequestParams& RequestParams);

	static void OnNewPipelineStateCreated(const FGraphicsPipelineStateInitializer& Initializer, const FPSOPrecacheRequestParams& RequestParams, FGraphicsPipelineState* NewGraphicsPipelineState, bool bDoAsyncCompile);

};

// Typed caches for compute and graphics
typedef TSharedPipelineStateCache<FComputePipelineStateInitializer, FComputePipelineState*> FComputePipelineCache;
typedef TSharedPipelineStateCache<FWorkGraphPipelineStateInitializer, FWorkGraphPipelineState*> FWorkGraphPipelineCache;
typedef TSharedPipelineStateCache<FGraphicsPipelineStateInitializer, FGraphicsPipelineState*> FGraphicsPipelineCache;

// These are the actual caches for both pipelines
FComputePipelineCache GComputePipelineCache;
FWorkGraphPipelineCache GWorkGraphPipelineCache;
FGraphicsPipelineCache GGraphicsPipelineCache;
TUniquePtr<FPrecacheGraphicsPipelineCache> GPrecacheGraphicsPipelineCache;
TUniquePtr<FPrecacheComputePipelineCache> GPrecacheComputePipelineCache;


FAutoConsoleTaskPriority CPrio_FCompilePipelineStateTask(
	TEXT("TaskGraph.TaskPriorities.CompilePipelineStateTask"),
	TEXT("Task and thread priority for FCompilePipelineStateTask."),
	ENamedThreads::HighThreadPriority,		// if we have high priority task threads, then use them...
	ENamedThreads::NormalTaskPriority,		// .. at normal task priority
	ENamedThreads::HighTaskPriority		// if we don't have hi pri threads, then use normal priority threads at high task priority instead
);
#if RHI_RAYTRACING

// Simple thread-safe pipeline state cache that's designed for low-frequency pipeline creation operations.
// The expected use case is a very infrequent (i.e. startup / load / streaming time) creation of ray tracing PSOs.
// This cache uses a single internal lock and therefore is not designed for highly concurrent operations.
class FRayTracingPipelineCache
{
public:
	FRayTracingPipelineCache()
	{}

	~FRayTracingPipelineCache()
	{}

	bool FindBase(const FRayTracingPipelineStateInitializer& Initializer, FRayTracingPipelineState*& OutPipeline) const
	{
		FScopeLock ScopeLock(&CriticalSection);

		// Find the most recently used pipeline with compatible configuration

		FRayTracingPipelineState* BestPipeline = nullptr;

		for (const auto& It : FullPipelines)
		{
			const FRayTracingPipelineStateSignature& CandidateInitializer = It.Key;
			FRayTracingPipelineState* CandidatePipeline = It.Value;

			if (!CandidatePipeline->RHIPipeline.IsValid()
				|| CandidateInitializer.MaxPayloadSizeInBytes != Initializer.MaxPayloadSizeInBytes
				|| CandidateInitializer.GetRayGenHash() != Initializer.GetRayGenHash()
				|| CandidateInitializer.GetRayMissHash() != Initializer.GetRayMissHash()
				|| CandidateInitializer.GetCallableHash() != Initializer.GetCallableHash())
			{
				continue;
			}

			if (BestPipeline == nullptr || *BestPipeline < *CandidatePipeline)
			{
				BestPipeline = CandidatePipeline;
			}
		}

		if (BestPipeline)
		{
			OutPipeline = BestPipeline;
			return true;
		}
		else
		{
			return false;
		}
	}

	bool FindBySignature(const FRayTracingPipelineStateSignature& Signature, FRayTracingPipelineState*& OutCachedState) const
	{
		FScopeLock ScopeLock(&CriticalSection);

		FRayTracingPipelineState* const* FoundState = FullPipelines.Find(Signature);
		if (FoundState)
		{
			OutCachedState = *FoundState;
			return true;
		}
		else
		{
			return false;
		}
	}

	bool Find(const FRayTracingPipelineStateInitializer& Initializer, FRayTracingPipelineState*& OutCachedState) const
	{
		FScopeLock ScopeLock(&CriticalSection);

		const FPipelineMap& Cache = Initializer.bPartial ? PartialPipelines : FullPipelines;

		FRayTracingPipelineState* const* FoundState = Cache.Find(Initializer);
		if (FoundState)
		{
			OutCachedState = *FoundState;
			return true;
		}
		else
		{
			return false;
		}
	}

	// Creates and returns a new pipeline state object, adding it to internal cache.
	// The cache itself owns the object and is responsible for destroying it.
	FRayTracingPipelineState* Add(const FRayTracingPipelineStateInitializer& Initializer)
	{
		FRayTracingPipelineState* Result = new FRayTracingPipelineState(Initializer);
		Result->MaxLocalBindingSize = Initializer.GetMaxLocalBindingDataSize();

		FScopeLock ScopeLock(&CriticalSection);

		FPipelineMap& Cache = Initializer.bPartial ? PartialPipelines : FullPipelines;

		Cache.Add(Initializer, Result);
		Result->AddHit();

		return Result;
	}

	void GetResources(TArray<TRefCountPtr<FRHIResource>>& OutResources)
	{
		FScopeLock ScopeLock(&CriticalSection);

		for (auto&& [Desc, State] : FullPipelines)
		{
			if (State)
			{
				OutResources.Add(TRefCountPtr<FRHIResource>(State->RHIPipeline));
			}
		}
		
		for (auto&& [Desc, State] : PartialPipelines)
		{
			if (State)
			{
				OutResources.Add(TRefCountPtr<FRHIResource>(State->RHIPipeline));
			}
		}
	}

	void Shutdown()
	{
		FScopeLock ScopeLock(&CriticalSection);
		for (auto& It : FullPipelines)
		{
			if (It.Value)
			{
				It.Value->WaitCompletion();
				delete It.Value;
				It.Value = nullptr;
			}
		}
		for (auto& It : PartialPipelines)
		{
			if (It.Value)
			{
				It.Value->WaitCompletion();
				delete It.Value;
				It.Value = nullptr;
			}
		}
		FullPipelines.Reset();
		PartialPipelines.Reset();
	}

	void Trim(int32 TargetNumEntries)
	{
		FScopeLock ScopeLock(&CriticalSection);

		// Only full pipeline cache is automatically trimmed.
		FPipelineMap& Cache = FullPipelines;

		if (Cache.Num() < TargetNumEntries)
		{
			return;
		}

		struct FEntry
		{
			FRayTracingPipelineStateSignature Key;
			uint64 LastFrameHit;
			uint64 HitsAcrossFrames;
			FRayTracingPipelineState* Pipeline;
		};
		TArray<FEntry, FConcurrentLinearArrayAllocator> Entries;
		Entries.Reserve(Cache.Num());

		const uint64 CurrentFrame = GFrameCounter;
		const uint32 NumLatencyFrames = 10;

		// Find all pipelines that were not used in the last 10 frames

		for (const auto& It : Cache)
		{
			if (It.Value->LastFrameHit + NumLatencyFrames <= CurrentFrame
				&& It.Value->IsCompilationComplete())
			{
				FEntry Entry;
				Entry.Key = It.Key;
				Entry.HitsAcrossFrames = It.Value->HitsAcrossFrames;
				Entry.LastFrameHit = It.Value->LastFrameHit;
				Entry.Pipeline = It.Value;
				Entries.Add(Entry);
			}
		}

		Entries.Sort([](const FEntry& A, const FEntry& B)
		{
			if (A.LastFrameHit == B.LastFrameHit)
			{
				return B.HitsAcrossFrames < A.HitsAcrossFrames;
			}
			else
			{
				return B.LastFrameHit < A.LastFrameHit;
			}
		});

		// Remove least useful pipelines

		while (Cache.Num() > TargetNumEntries && Entries.Num())
		{
			FEntry& LastEntry = Entries.Last();
			check(LastEntry.Pipeline->RHIPipeline);
			check(LastEntry.Pipeline->IsCompilationComplete());
			delete LastEntry.Pipeline;
			Cache.Remove(LastEntry.Key);
			Entries.Pop(EAllowShrinking::No);
		}

		LastTrimFrame = CurrentFrame;
	}

	uint64 GetLastTrimFrame() const { return LastTrimFrame; }

private:

	mutable FCriticalSection CriticalSection;
	using FPipelineMap = TMap<FRayTracingPipelineStateSignature, FRayTracingPipelineState*>;
	FPipelineMap FullPipelines;
	FPipelineMap PartialPipelines;
	uint64 LastTrimFrame = 0;
};

FRayTracingPipelineCache GRayTracingPipelineCache;
#endif

/**
 *  Compile task
 */
static std::atomic<int32> GPipelinePrecompileTasksInFlight = { 0 };

int32 PipelineStateCache::GetNumActivePipelinePrecompileTasks()
{
	return GPipelinePrecompileTasksInFlight.load();
}

template <typename TCompilePipelineStateTaskDerived, typename TPipelineInitializer>
class TCompilePipelineStateTaskBase
{
protected:
	FPipelineStateAsync* Pipeline;
	TPipelineInitializer Initializer;
	EPSOPrecacheResult PSOPrecacheResult;
	EPSOPrecachePriority PSOPrecachePriority = EPSOPrecachePriority::NotSet;
	bool bPrecachedPSOMarkedUsed;
	bool bInImmediateCmdList;
	FPSOCompilationDebugData PSOCompilationDebugData;

public:
	TCompilePipelineStateTaskBase(
		FPipelineStateAsync* InPipeline,
		const TPipelineInitializer& InInitializer,
		EPSOPrecacheResult InPSOPrecacheResult,
		bool InbPrecachedPSOMarkedUsed,
		bool InbInImmediateCmdList,
		const FPSOCompilationDebugData& InPSOCompilationDebugData)
		: Pipeline(InPipeline)
		, Initializer(InInitializer)
		, PSOPrecacheResult(InPSOPrecacheResult)
		, bPrecachedPSOMarkedUsed(InbPrecachedPSOMarkedUsed)
		, bInImmediateCmdList(InbInImmediateCmdList)
		, PSOCompilationDebugData(InPSOCompilationDebugData)
	{
		ensure(Pipeline->GetCompletionEvent() != nullptr);
		if (Initializer.bFromPSOFileCache)
		{
			GPipelinePrecompileTasksInFlight++;
		}
	}

	~TCompilePipelineStateTaskBase()
	{
		if (Initializer.bFromPSOFileCache)
		{
			GPipelinePrecompileTasksInFlight--;
		}
	}

	static constexpr ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void SetPSOPrecachePriority(EPSOPrecachePriority TaskPriority)
	{
		check(PSOPrecachePriority == EPSOPrecachePriority::NotSet);
		PSOPrecachePriority = TaskPriority;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		CompilePSO();
	}

	void CompilePSO()
	{
		LLM_SCOPE(ELLMTag::PSO);
		FTaskTagScope Scope(GetDesiredThread() == ENamedThreads::RHIThread ? ETaskTag::EParallelRhiThread : ETaskTag::EParallelRenderingThread);

#if WITH_RHI_BREADCRUMBS
		if (PSOCompilationDebugData.BreadcrumbNode)
		{
			FRHIBreadcrumbNode::WalkInRange(
				PSOCompilationDebugData.BreadcrumbNode,
				PSOCompilationDebugData.BreadcrumbRoot
			);
		}
#endif // WITH_RHI_BREADCRUMBS
				
		bool bRestoreThreadPriority = false;
		if (Initializer.bPSOPrecache)
		{
			// Might be not set if running without async compile tasks
			PSOPrecachePriority = PSOPrecachePriority != EPSOPrecachePriority::NotSet ? PSOPrecachePriority : EPSOPrecachePriority::Medium;
			Pipeline->CSVStat.SetInFlightPriority(PSOPrecachePriority);

			if (GPSOPrecacheBoostThreadPriorityForHighestPriorityRequests > 0 && PSOPrecachePriority == EPSOPrecachePriority::Highest)
			{
				FPlatformProcess::SetThreadPriority(TPri_AboveNormal);
				bRestoreThreadPriority = true;
			}
		}

		const TCHAR* PSOPrecacheResultScopeString = nullptr;
		switch (PSOPrecacheResult)
		{
			case EPSOPrecacheResult::Unknown:      PSOPrecacheResultScopeString = TEXT("PSOPrecache: Unknown"); break;
			case EPSOPrecacheResult::Active:       PSOPrecacheResultScopeString = TEXT("PSOPrecache: Precaching"); break;
			case EPSOPrecacheResult::Complete:     PSOPrecacheResultScopeString = TEXT("PSOPrecache: Precached"); break;
			case EPSOPrecacheResult::Missed:       PSOPrecacheResultScopeString = TEXT("PSOPrecache: Missed"); break;
			case EPSOPrecacheResult::TooLate:      PSOPrecacheResultScopeString = TEXT("PSOPrecache: Too Late"); break;
			case EPSOPrecacheResult::NotSupported: PSOPrecacheResultScopeString = TEXT("PSOPrecache: Not Supported"); break;
			case EPSOPrecacheResult::Untracked:    PSOPrecacheResultScopeString = TEXT("PSOPrecache: Untracked"); break;
		}

		const TCHAR* PSOPrecachePriorityScopeString = nullptr;
		if (Initializer.bPSOPrecache)
		{
			switch (PSOPrecachePriority)
			{
				case EPSOPrecachePriority::Low:      PSOPrecachePriorityScopeString = TEXT("Priority: Low"); break;
				case EPSOPrecachePriority::Medium:   PSOPrecachePriorityScopeString = TEXT("Priority: Medium"); break;
				case EPSOPrecachePriority::High:     PSOPrecachePriorityScopeString = TEXT("Priority: High"); break;
				case EPSOPrecachePriority::Highest:  PSOPrecachePriorityScopeString = TEXT("Priority: Highest"); break;
				default:  checkNoEntry();			 PSOPrecachePriorityScopeString = TEXT("Priority: Unknown"); break;
			}
		}

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(PSOPrecacheResultScopeString);
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(PSOPrecachePriorityScopeString, PSOPrecachePriorityScopeString != nullptr);
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(PSOCompilationDebugData.PassName, PSOCompilationDebugData.PassName != nullptr);
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(*PSOCompilationDebugData.ResourceName, !PSOCompilationDebugData.ResourceName.IsEmpty());

			static_cast<TCompilePipelineStateTaskDerived*>(this)->CompilePSOInternal();
		}

#if WITH_RHI_BREADCRUMBS
		if (PSOCompilationDebugData.BreadcrumbNode)
		{
			FRHIBreadcrumbNode::WalkOutRange(
				PSOCompilationDebugData.BreadcrumbNode,
				PSOCompilationDebugData.BreadcrumbRoot
			);
		}
#endif // WITH_RHI_BREADCRUMBS
		
		// We kicked a task: the event really should be there
		if (ensure(Pipeline->GetCompletionEvent()))
		{
			Pipeline->GetCompletionEvent()->DispatchSubsequents();
			// At this point, it's not safe to use Pipeline anymore, as it might get picked up by ProcessDelayedCleanup and deleted
			Pipeline = nullptr;
		}

		if (bRestoreThreadPriority)
		{
			FPlatformProcess::SetThreadPriority((EThreadPriority)GPSOPrecompileThreadPoolThreadPriority);
		}
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FCompilePipelineStateTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		ENamedThreads::Type DesiredThread = GRunPSOCreateTasksOnRHIT && IsRunningRHIInSeparateThread() && bInImmediateCmdList ? ENamedThreads::RHIThread : CPrio_FCompilePipelineStateTask.Get();

		// On Mac the compilation is handled using external processes, so engine threads have very little work to do
		// and it's better to leave more CPU time to these external processes and other engine threads.
		// Also use background threads for PSO precaching when the PSO thread pool is not used
		// Compute pipelines usually take much longer to compile, compile them on background thread as well.
		return (PLATFORM_MAC || PSOPrecacheResult == EPSOPrecacheResult::Active || (Pipeline && Pipeline->IsCompute() && Initializer.bFromPSOFileCache)) ? ENamedThreads::AnyBackgroundThreadNormalTask : DesiredThread;
	}
};

class FCompileGraphicsPipelineStateTask : public TCompilePipelineStateTaskBase<FCompileGraphicsPipelineStateTask, FGraphicsPipelineStateInitializer>
{
public:
	FCompileGraphicsPipelineStateTask(
		FPipelineStateAsync* InPipeline,
		const FGraphicsPipelineStateInitializer& InInitializer,
		EPSOPrecacheResult InPSOPrecacheResult,
		bool InbPrecachedPSOMarkedUsed,
		bool InbInImmediateCmdList,
		const FPSOCompilationDebugData& InPSOCompilationDebugData)
		: TCompilePipelineStateTaskBase(InPipeline, InInitializer, InPSOPrecacheResult, InbPrecachedPSOMarkedUsed, InbInImmediateCmdList, InPSOCompilationDebugData)
	{
#if PLATFORM_WINDOWS
		auto MarkInUseByPSOCompilation = [](FRHIShader* Shader)
			{
				if (Shader)
				{
					Shader->SetInUseByPSOCompilation(true);
				}
			};
		MarkInUseByPSOCompilation(Initializer.BoundShaderState.GetMeshShader());
		MarkInUseByPSOCompilation(Initializer.BoundShaderState.GetAmplificationShader());
		MarkInUseByPSOCompilation(Initializer.BoundShaderState.VertexShaderRHI);
		MarkInUseByPSOCompilation(Initializer.BoundShaderState.PixelShaderRHI);
		MarkInUseByPSOCompilation(Initializer.BoundShaderState.GetGeometryShader());
		MarkInUseByPSOCompilation(Initializer.BoundShaderState.GetMeshShader());
#endif // PLATFORM_WINDOWS

		if (Initializer.BoundShaderState.GetMeshShader())
		{
			Initializer.BoundShaderState.GetMeshShader()->AddRef();
		}
		if (Initializer.BoundShaderState.GetAmplificationShader())
		{
			Initializer.BoundShaderState.GetAmplificationShader()->AddRef();
		}
		if (Initializer.BoundShaderState.VertexDeclarationRHI)
		{
			Initializer.BoundShaderState.VertexDeclarationRHI->AddRef();
		}
		if (Initializer.BoundShaderState.VertexShaderRHI)
		{
			Initializer.BoundShaderState.VertexShaderRHI->AddRef();
		}
		if (Initializer.BoundShaderState.PixelShaderRHI)
		{
			Initializer.BoundShaderState.PixelShaderRHI->AddRef();
		}
		if (Initializer.BoundShaderState.GetGeometryShader())
		{
			Initializer.BoundShaderState.GetGeometryShader()->AddRef();
		}
		if (Initializer.BlendState)
		{
			Initializer.BlendState->AddRef();
		}
		if (Initializer.RasterizerState)
		{
			Initializer.RasterizerState->AddRef();
		}
		if (Initializer.DepthStencilState)
		{
			Initializer.DepthStencilState->AddRef();
		}
	}

	void CompilePSOInternal()
	{
		bool bSkipCreation = false;
		if (GRHISupportsMeshShadersTier0)
		{
			if (!Initializer.BoundShaderState.VertexShaderRHI && !Initializer.BoundShaderState.GetMeshShader())
			{
				UE_LOGF(LogRHI, Error, "Tried to create a Gfx Pipeline State without Vertex or Mesh Shader");
				bSkipCreation = true;
			}
		}
		else
		{
			if (Initializer.BoundShaderState.GetMeshShader())
			{
				UE_LOGF(LogRHI, Error, "Tried to create a Gfx Pipeline State with Mesh Shader on hardware without mesh shader support.");
				bSkipCreation = true;
			}

			if (!Initializer.BoundShaderState.VertexShaderRHI)
			{
				UE_LOGF(LogRHI, Error, "Tried to create a Gfx Pipeline State without Vertex Shader");
				bSkipCreation = true;
			}
		}

		const bool bAbortPSOCompileDueToShutdown = IsEngineExitRequested() && Initializer.bPSOPrecache;
		if (bAbortPSOCompileDueToShutdown)
		{
			UE_LOGF(LogRHI, Verbose, "Skipping a precache compile due to engine shutdown.");
			bSkipCreation = true;
		}

		if (Initializer.bPSOPrecache)
		{
			Initializer.PrecacheCompileType = (uint32)GetPSOCompileTypeFromPrecachePriority(PSOPrecachePriority);			
		}

		FGraphicsPipelineState* GfxPipeline = static_cast<FGraphicsPipelineState*>(Pipeline);

		uint64 StartTime = FPlatformTime::Cycles64();
		GfxPipeline->RHIPipeline = bSkipCreation ? nullptr : RHICreateGraphicsPipelineState(Initializer);
		uint64 PSOCompilationTime = CheckAndUpdateHitchCountStat(FPSOPrecacheRequestID::EType::Graphics, !IsPrecachedPSO(Initializer), PSOCompilationDebugData, StartTime, PSOPrecacheResult);

		if (GfxPipeline->RHIPipeline)
		{
			GfxPipeline->SortKey = GfxPipeline->RHIPipeline->GetSortKey();
		}
		else if (!bAbortPSOCompileDueToShutdown)
		{
			HandlePipelineCreationFailure(Initializer);
		}

		// Mark as finished when it's a precaching job
		if (Initializer.bPSOPrecache)
		{
			GPrecacheGraphicsPipelineCache->PrecacheFinished(Initializer, GfxPipeline->RHIPipeline != nullptr, PSOCompilationTime);
		}
		else if (bPrecachedPSOMarkedUsed)
		{
			if (Initializer.StatePrecachePSOHash == 0)
			{
				Initializer.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(Initializer);
			}
			GPrecacheGraphicsPipelineCache->MarkPSOAsUsed(Initializer);
		}

#if PLATFORM_WINDOWS
		auto MarkUnusedByPSOCompilation = [](FRHIShader* Shader)
			{
				if (Shader)
				{
					Shader->SetInUseByPSOCompilation(false);
				}
			};
		MarkUnusedByPSOCompilation(Initializer.BoundShaderState.GetMeshShader());
		MarkUnusedByPSOCompilation(Initializer.BoundShaderState.GetAmplificationShader());
		MarkUnusedByPSOCompilation(Initializer.BoundShaderState.VertexShaderRHI);
		MarkUnusedByPSOCompilation(Initializer.BoundShaderState.PixelShaderRHI);
		MarkUnusedByPSOCompilation(Initializer.BoundShaderState.GetGeometryShader());
		MarkUnusedByPSOCompilation(Initializer.BoundShaderState.GetMeshShader());
#endif // PLATFORM_WINDOWS

		if (Initializer.BoundShaderState.GetMeshShader())
		{
			Initializer.BoundShaderState.GetMeshShader()->Release();
		}
		if (Initializer.BoundShaderState.GetAmplificationShader())
		{
			Initializer.BoundShaderState.GetAmplificationShader()->Release();
		}
		if (Initializer.BoundShaderState.VertexDeclarationRHI)
		{
			Initializer.BoundShaderState.VertexDeclarationRHI->Release();
		}
		if (Initializer.BoundShaderState.VertexShaderRHI)
		{
			Initializer.BoundShaderState.VertexShaderRHI->Release();
		}
		if (Initializer.BoundShaderState.PixelShaderRHI)
		{
			Initializer.BoundShaderState.PixelShaderRHI->Release();
		}
		if (Initializer.BoundShaderState.GetGeometryShader())
		{
			Initializer.BoundShaderState.GetGeometryShader()->Release();
		}
		if (Initializer.BlendState)
		{
			Initializer.BlendState->Release();
		}
		if (Initializer.RasterizerState)
		{
			Initializer.RasterizerState->Release();
		}
		if (Initializer.DepthStencilState)
		{
			Initializer.DepthStencilState->Release();
		}
	}
};

class FCompileComputePipelineStateTask : public TCompilePipelineStateTaskBase<FCompileComputePipelineStateTask, FComputePipelineStateInitializer>
{
public:
	FCompileComputePipelineStateTask(
		FPipelineStateAsync* InPipeline,
		const FComputePipelineStateInitializer& InInitializer,
		EPSOPrecacheResult InPSOPrecacheResult,
		bool InbPrecachedPSOMarkedUsed,
		bool InbInImmediateCmdList,
		const FPSOCompilationDebugData& InPSOCompilationDebugData)
		: TCompilePipelineStateTaskBase(InPipeline, InInitializer, InPSOPrecacheResult, InbPrecachedPSOMarkedUsed, InbInImmediateCmdList, InPSOCompilationDebugData)
	{
#if PLATFORM_WINDOWS
		InInitializer.ComputeShader->SetInUseByPSOCompilation(true);
#endif // PLATFORM_WINDOWS
	}

	void CompilePSOInternal()
	{
		FComputePipelineState* ComputePipeline = static_cast<FComputePipelineState*>(Pipeline);

		uint64 StartTime = FPlatformTime::Cycles64();

		// TODO: Send the priority to the RHI.
		ComputePipeline->RHIPipeline = RHICreateComputePipelineState(Initializer);
		uint64 PSOCompilationTime = CheckAndUpdateHitchCountStat(FPSOPrecacheRequestID::EType::Compute, !IsPrecachedPSO(Initializer), PSOCompilationDebugData, StartTime, PSOPrecacheResult);

		if (!ComputePipeline->RHIPipeline)
		{
			HandlePipelineCreationFailure(Initializer);
		}

		if (Initializer.bPSOPrecache)
		{
			bool bCSValid = ComputePipeline->RHIPipeline != nullptr && ComputePipeline->RHIPipeline->IsValid();
			GPrecacheComputePipelineCache->PrecacheFinished(Initializer, bCSValid, PSOCompilationTime);
		}
		else if (bPrecachedPSOMarkedUsed)
		{
			GPrecacheComputePipelineCache->MarkPSOAsUsed(Initializer);
		}

#if PLATFORM_WINDOWS
		Initializer.ComputeShader->SetInUseByPSOCompilation(false);
#endif // PLATFORM_WINDOWS
	}
};

void PipelineStateCache::ReportFrameHitchToCSV()
{
	ReportFrameHitchThisFrame = true;
}

/**
* Called at the end of each frame during the RHI . Evicts all items left in the backfill cached based on time
*/
void PipelineStateCache::FlushResources()
{
	check(IsInRenderingThread());

	static double LastEvictionTime = FPlatformTime::Seconds();
	double CurrentTime = FPlatformTime::Seconds();

#if PSO_DO_CACHE_EVICT_EACH_FRAME
	LastEvictionTime = 0;
#endif

	// because it takes two cycles for an object to move from main->backfill->gone we check
	// at half the desired eviction time
	int32 EvictionPeriod = CVarPSOEvictionTime.GetValueOnAnyThread();
	bool bDiscardAndSwap = !(EvictionPeriod == 0 || CurrentTime - LastEvictionTime < EvictionPeriod);
	if (bDiscardAndSwap)
	{
		LastEvictionTime = CurrentTime;
	}

	GComputePipelineCache.FlushResources(bDiscardAndSwap);
	GWorkGraphPipelineCache.FlushResources(bDiscardAndSwap);
	GGraphicsPipelineCache.FlushResources(bDiscardAndSwap);

	check(GPrecacheGraphicsPipelineCache && GPrecacheComputePipelineCache);
	GPrecacheGraphicsPipelineCache->SetMaxInMemoryPSOs(GPSOPrecacheKeepInMemoryGraphicsMaxNum);
	GPrecacheComputePipelineCache->SetMaxInMemoryPSOs(GPSOPrecacheKeepInMemoryComputeMaxNum);
	GPrecacheGraphicsPipelineCache->ProcessDelayedCleanup();
	GPrecacheComputePipelineCache->ProcessDelayedCleanup();

	FPipelineFileCacheManager::BroadcastNewPSOsDelegate();

	{
		int32 NumMissesThisFrame = GraphicsPipelineCacheMisses.Load(EMemoryOrder::Relaxed);
		int32 NumMissesLastFrame = GraphicsPipelineCacheMissesHistory.Num() >= 2 ? GraphicsPipelineCacheMissesHistory[1] : 0;
		CSV_CUSTOM_STAT(PSO, PSOMisses, NumMissesThisFrame, ECsvCustomStatOp::Set);

		// Put a negative number in the CSV to report that there was no hitch this frame for the PSO hitch stat.
		if (!ReportFrameHitchThisFrame)
		{
			NumMissesThisFrame = -1;
			NumMissesLastFrame = -1;
		}
		CSV_CUSTOM_STAT(PSO, PSOMissesOnHitch, NumMissesThisFrame, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(PSO, PSOPrevFrameMissesOnHitch, NumMissesLastFrame, ECsvCustomStatOp::Set);
	}

	{
		int32 NumMissesThisFrame = ComputePipelineCacheMisses.Load(EMemoryOrder::Relaxed);
		int32 NumMissesLastFrame = ComputePipelineCacheMissesHistory.Num() >= 2 ? ComputePipelineCacheMissesHistory[1] : 0;
		CSV_CUSTOM_STAT(PSO, PSOComputeMisses, NumMissesThisFrame, ECsvCustomStatOp::Set);

		// Put a negative number in the CSV to report that there was no hitch this frame for the PSO hitch stat.
		if (!ReportFrameHitchThisFrame)
		{
			NumMissesThisFrame = -1;
			NumMissesLastFrame = -1;
		}
		CSV_CUSTOM_STAT(PSO, PSOComputeMissesOnHitch, NumMissesThisFrame, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(PSO, PSOComputePrevFrameMissesOnHitch, NumMissesLastFrame, ECsvCustomStatOp::Set);
	}
	ReportFrameHitchThisFrame = false;

	GraphicsPipelineCacheMissesHistory.Insert(GraphicsPipelineCacheMisses, 0);
	GraphicsPipelineCacheMissesHistory.SetNum(PSO_MISS_FRAME_HISTORY_SIZE);
	ComputePipelineCacheMissesHistory.Insert(ComputePipelineCacheMisses, 0);
	ComputePipelineCacheMissesHistory.SetNum(PSO_MISS_FRAME_HISTORY_SIZE);
	GraphicsPipelineCacheMisses = 0;
	ComputePipelineCacheMisses = 0;

#if PSO_TRACK_CACHE_STATS
	DumpPipelineCacheStats();

	int32 ReleasedComputeEntries = GComputePipelineCache.NumReleasedEntries();
	int32 ReleasedGraphicsEntries = GGraphicsPipelineCache.NumReleasedEntries();
	int32 ReleasedWorkGraphEntries = GWorkGraphPipelineCache.NumReleasedEntries();

	if (ReleasedComputeEntries > 0 || ReleasedGraphicsEntries > 0 || ReleasedWorkGraphEntries > 0)
	{
		UE_LOGF(LogRHI, Log, "Cleared state cache in %.02f ms. %d ComputeEntries, %d GraphicsEntries, %d WorkGraphEntries"
			, (FPlatformTime::Seconds() - CurrentTime) / 1000
			, ReleasedComputeEntries, ReleasedGraphicsEntries, ReleasedWorkGraphEntries);
	}
#endif // PSO_TRACK_CACHE_STATS
}

static bool IsAsyncCompilationAllowed(FRHIComputeCommandList& RHICmdList, bool bIsPrecompileRequest)
{
	const EPSOCompileAsyncMode PSOCompileAsyncMode = (EPSOCompileAsyncMode)GCVarAsyncPipelineCompile.GetValueOnAnyThread();

	const bool bCVarAllowsAsyncCreate = PSOCompileAsyncMode == EPSOCompileAsyncMode::All
		|| (PSOCompileAsyncMode == EPSOCompileAsyncMode::Precompile && bIsPrecompileRequest)
		|| (PSOCompileAsyncMode == EPSOCompileAsyncMode::NonPrecompiled && !bIsPrecompileRequest);

	return GRHISupportsAsyncPipelinePrecompile &&
		FDataDrivenShaderPlatformInfo::GetSupportsAsyncPipelineCompilation(GMaxRHIShaderPlatform) &&
		bCVarAllowsAsyncCreate && !RHICmdList.Bypass() && (IsRunningRHIInSeparateThread() && !IsInRHIThread()) && !RHICmdList.IsRecursive();
}

uint64 PipelineStateCache::RetrieveGraphicsPipelineStateSortKey(const FGraphicsPipelineState* GraphicsPipelineState)
{
	return GraphicsPipelineState != nullptr ? GraphicsPipelineState->SortKey : 0;
}

template <typename TCompilePipelineTask, typename TPipelineInitializer>
static void InternalAsyncCompilePipelineState(const TPipelineInitializer& Initializer, EPSOPrecacheResult PSOPrecacheResult, EPSOPrecachePriority PSOPrecachePriority, bool bPrecachedPSOMarkedUsed, FPipelineStateAsync* CachedState, const FPSOCompilationDebugData& PSOCompilationDebugData, bool bInImmediateCmdList)
{
	if (!Initializer.bPSOPrecache || !FPSOPrecacheThreadPool::UsePool())
	{
		TGraphTask<TCompilePipelineTask>::CreateTask().ConstructAndDispatchWhenReady(CachedState, Initializer, PSOPrecacheResult, bPrecachedPSOMarkedUsed, bInImmediateCmdList, PSOCompilationDebugData);
	}
	else
	{
		// Here, PSO precompiles use a separate thread pool.
		// Note that we do not add precompile tasks as cmdlist prerequisites.
		TUniquePtr<TCompilePipelineTask> ThreadPoolTask = MakeUnique<TCompilePipelineTask>(CachedState, Initializer, PSOPrecacheResult, bPrecachedPSOMarkedUsed, bInImmediateCmdList, PSOCompilationDebugData);
		CachedState->SetPrecompileTask(MakeUnique<FPSOPrecacheAsyncTask>(
			[ThreadPoolTask = MoveTemp(ThreadPoolTask)](const FPSOPrecacheAsyncTask* ThisTask)
			{
				// Convert the task priority to PSO precompile priority.
				// Update here as the task's priority may have changed since creation.
				EPSOPrecachePriority TaskPrecachePriority = GetPSOPrecachePriorityFromQueuePri(ThisTask->GetPriority());
				ThreadPoolTask->SetPSOPrecachePriority(TaskPrecachePriority);
				ThreadPoolTask->CompilePSO();
			}
		));

		CachedState->CSVStat.SetState(Initializer.bPSOPrecache, CachedState->IsCompute(), PSOPrecachePriority);

		EQueuedWorkPriority QueuedWorkPriority = GetQueuedWorkPriorityFromPSOPrecachePriority(PSOPrecachePriority);		
		CachedState->GetPrecompileTask()->StartBackgroundTask(&GPSOPrecacheThreadPool.Get(), QueuedWorkPriority);
	}
}

static void InternalCreateComputePipelineState(const FComputePipelineStateInitializer& Initializer, bool bDoAsyncCompile, EPSOPrecacheResult PSOPrecacheResult, EPSOPrecachePriority PrecachePriority, bool bPrecachedPSOMarkedUsed, FComputePipelineState* CachedState, const FPSOCompilationDebugData& PSOCompilationDebugData, bool bInImmediateCmdList)
{
	check(!Initializer.ComputeShader->HasSpecializationConstants() || !Initializer.SpecializationConstants.IsEmpty());
	FGraphEventRef GraphEvent = CachedState->GetCompletionEvent();

	// create a compilation task, or just do it now...
	if (bDoAsyncCompile)
	{
		check(GraphEvent != nullptr);
		InternalAsyncCompilePipelineState<FCompileComputePipelineStateTask, FComputePipelineStateInitializer>(Initializer, PSOPrecacheResult, PrecachePriority, bPrecachedPSOMarkedUsed, CachedState, PSOCompilationDebugData, bInImmediateCmdList);
	}
	else
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(PSOCompilationDebugData.PassName, PSOCompilationDebugData.PassName != nullptr);
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(*PSOCompilationDebugData.ResourceName, !PSOCompilationDebugData.ResourceName.IsEmpty());

		check(GraphEvent == nullptr);
		uint64 StartTime = FPlatformTime::Cycles64();
		CachedState->RHIPipeline = RHICreateComputePipelineState(Initializer);
		uint64 PSOCompilationTime = CheckAndUpdateHitchCountStat(FPSOPrecacheRequestID::EType::Compute, !IsPrecachedPSO(Initializer), PSOCompilationDebugData, StartTime, PSOPrecacheResult);

		if (Initializer.bPSOPrecache)
		{
			GPrecacheComputePipelineCache->PrecacheFinished(Initializer, CachedState->RHIPipeline != nullptr, PSOCompilationTime);
		}

		if (!CachedState->RHIPipeline)
		{
			HandlePipelineCreationFailure(Initializer);
		}
	}
}

FComputePipelineState* PipelineStateCache::GetAndOrCreateComputePipelineState(FRHIComputeCommandList& RHICmdList, const FComputePipelineStateInitializer& ComputeInitializer)
{
	LLM_SCOPE(ELLMTag::PSO);

	FComputePipelineState* OutCachedState = nullptr;

	bool bWasFound = GComputePipelineCache.Find(ComputeInitializer, OutCachedState);
	bool DoAsyncCompile = IsAsyncCompilationAllowed(RHICmdList, ComputeInitializer.bFromPSOFileCache);

	if (bWasFound == false)
	{		
		EPSOPrecacheResult PSOPrecacheResult = EPSOPrecacheResult::Unknown;
		bool bFromInMemoryPSOCache = false;
		bool bPrecachedPSOMarkedUsed = false;
		if (IsPSOPrecachingEnabled())
		{
			if (ShouldTrackUsedPrecachedPSOs())
			{
				PSOPrecacheResult = GPrecacheComputePipelineCache->GetPrecachingStateAndMarkUsed(ComputeInitializer, bPrecachedPSOMarkedUsed);
			}
			else
			{
				EPSOPrecacheInMemoryMode InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
				PSOPrecacheResult = GPrecacheComputePipelineCache->GetPrecachingStateAndInMemoryMode(ComputeInitializer, InMemoryMode);
				bFromInMemoryPSOCache = InMemoryMode == EPSOPrecacheInMemoryMode::Cached;
			}

		}
		
		FPipelineStateStats* Stats = nullptr;
		FPipelineFileCacheManager::CacheComputePSO(GetTypeHash(ComputeInitializer), ComputeInitializer, PSOPrecacheResult, &Stats);

		// create new compute state
		bool bRuntimePSO = true;
		OutCachedState = new FComputePipelineState(ComputeInitializer.ComputeShader, bRuntimePSO, bFromInMemoryPSOCache);
		OutCachedState->Stats = Stats;
		if (DoAsyncCompile)
		{
			OutCachedState->SetCompletionEvent(FGraphEvent::CreateGraphEvent());
		}

		if (!ComputeInitializer.bFromPSOFileCache)
		{
			++ComputePipelineCacheMisses;
		}

		// If the PSO is still precaching then mark as too late.
		if (PSOPrecacheResult == EPSOPrecacheResult::Active)
		{
			PSOPrecacheResult = EPSOPrecacheResult::TooLate;
		}

		FPSOCompilationDebugData PSOCompilationDebugData;
#if WITH_RHI_BREADCRUMBS
		if (DoAsyncCompile && !ComputeInitializer.bFromPSOFileCache)
		{
			PSOCompilationDebugData.BreadcrumbRoot = FRHIBreadcrumbNode::GetNonNullRoot(RHICmdList.GetCurrentBreadcrumbRef());
			PSOCompilationDebugData.BreadcrumbNode = PSOCompilationDebugData.BreadcrumbRoot
				? RHICmdList.GetCurrentBreadcrumbRef()
				: nullptr;

			// Adding Shared ref to BreadcrumbNodeAllocator to keep it alive. During the warming of the PSO cache at launch, the RHICmdLis doesn't wait for the PSO to finish compiling.
			if (PSOCompilationDebugData.BreadcrumbNode)
			{
				PSOCompilationDebugData.BreadcrumbNodeAllocator = PSOCompilationDebugData.BreadcrumbNode->Allocator->AsShared();
			}
		}
#endif // WITH_RHI_BREADCRUMBS
		
		FGraphEventRef GraphEvent = OutCachedState->GetCompletionEvent();
		InternalCreateComputePipelineState(ComputeInitializer, DoAsyncCompile, PSOPrecacheResult, EPSOPrecachePriority::Medium, bPrecachedPSOMarkedUsed, OutCachedState, PSOCompilationDebugData, RHICmdList.IsImmediate());

		// Don't add precached PSOs as a dispatch prerequisite. We don't need to wait for them to complete before the RHICmdList can be dispatched.
		if (GraphEvent.IsValid() && !ComputeInitializer.bFromPSOFileCache)
		{
			check(DoAsyncCompile);
			RHICmdList.AddDispatchPrerequisite(GraphEvent);
		}

		GComputePipelineCache.Add(ComputeInitializer, OutCachedState);
	}
	else
	{
		if (!ComputeInitializer.bFromPSOFileCache && !OutCachedState->IsComplete())
		{
			RHICmdList.AddDispatchPrerequisite(OutCachedState->GetCompletionEvent());
		}

	#if PSO_TRACK_CACHE_STATS
		OutCachedState->AddHit();
	#endif
	}

	// return the state pointer
	return OutCachedState;
}

inline void ValidateWorkGraphPipelineStateInitializer(const FWorkGraphPipelineStateInitializer& Initializer)
{
	check(Initializer.GetShaderTable().Num() > 0);
}

uint64 FWorkGraphPipelineStateInitializer::ComputeGraphicsPSOTableHash(const TArrayView<FGraphicsPipelineStateInitializer const*>& InGraphicsPSOTable, uint64 InitialHash)
{
	uint64 CombinedHash = InitialHash;
	for (FGraphicsPipelineStateInitializer const* GraphicsPSO : InGraphicsPSOTable)
	{
		uint32 GraphicsPSOHash = 0;
		if (GraphicsPSO != nullptr)
		{
			GraphicsPSOHash = GetTypeHash(GraphicsPSO->BoundShaderState);
			GraphicsPSOHash = HashCombineFast(GraphicsPSOHash, GetTypeHash(GraphicsPSO->RasterizerState));
			GraphicsPSOHash = HashCombineFast(GraphicsPSOHash, GetTypeHash(GraphicsPSO->DepthStencilState));
			GraphicsPSOHash = HashCombineFast(GraphicsPSOHash, GetTypeHash(GraphicsPSO->PrimitiveType));
			GraphicsPSOHash = HashCombineFast(GraphicsPSOHash, GetTypeHash(GraphicsPSO->RenderTargetsEnabled));
			GraphicsPSOHash = HashCombineFast(GraphicsPSOHash, GetTypeHash(GraphicsPSO->RenderTargetFormats));
			GraphicsPSOHash = HashCombineFast(GraphicsPSOHash, GetTypeHash(GraphicsPSO->DepthStencilTargetFormat));
		}

		// 64 bit hash combination as per boost::hash_combine_impl
		CombinedHash ^= (uint64)GraphicsPSOHash + 0x9e3779b97f4a7c15ull + (CombinedHash << 12) + (CombinedHash >> 4);
	}
	return CombinedHash;
}

FWorkGraphPipelineState* PipelineStateCache::GetAndOrCreateWorkGraphPipelineState(FRHIComputeCommandList& RHICmdList, const FWorkGraphPipelineStateInitializer& Initializer)
{
	LLM_SCOPE(ELLMTag::PSO);
	ValidateWorkGraphPipelineStateInitializer(Initializer);

	FWorkGraphPipelineState* OutCachedState = nullptr;
	bool bWasFound = GWorkGraphPipelineCache.Find(Initializer, OutCachedState);

	if (!bWasFound)
	{
		OutCachedState = new FWorkGraphPipelineState(Initializer.GetShaderTable()[0]);
		OutCachedState->RHIPipeline = RHICreateWorkGraphPipelineState(Initializer);
		GWorkGraphPipelineCache.Add(Initializer, OutCachedState);
	}
	else
	{
#if PSO_TRACK_CACHE_STATS
		OutCachedState->AddHit();
#endif
	}

	return OutCachedState;
}

#if RHI_RAYTRACING

class FCompileRayTracingPipelineStateTask
{
public:

	UE_NONCOPYABLE(FCompileRayTracingPipelineStateTask)

	FPipelineStateSync* Pipeline;

	FRayTracingPipelineStateInitializer Initializer;
	const bool bBackgroundTask;

	FCompileRayTracingPipelineStateTask(FPipelineStateSync* InPipeline, const FRayTracingPipelineStateInitializer& InInitializer, bool bInBackgroundTask)
		: Pipeline(InPipeline)
		, Initializer(InInitializer)
		, bBackgroundTask(bInBackgroundTask)
	{
		Initializer.bBackgroundCompilation = bBackgroundTask;

		// Copy all referenced shaders and AddRef them while the task is alive

		RayGenTable   = CopyShaderTable(InInitializer.GetRayGenTable());
		MissTable     = CopyShaderTable(InInitializer.GetMissTable());
		HitGroupTable = CopyShaderTable(InInitializer.GetHitGroupTable());
		CallableTable = CopyShaderTable(InInitializer.GetCallableTable());

		// Point initializer to shader tables owned by this task

		Initializer.SetRayGenShaderTable(RayGenTable, InInitializer.GetRayGenHash());
		Initializer.SetMissShaderTable(MissTable, InInitializer.GetRayMissHash());
		Initializer.SetHitGroupTable(HitGroupTable, InInitializer.GetHitGroupHash());
		Initializer.SetCallableTable(CallableTable, InInitializer.GetCallableHash());

		// Also copy over the shader binding layout and update the reference to make sure the binding layout is kept alive
		if (InInitializer.ShaderBindingLayout)
		{
			ShaderBindingLayout = *InInitializer.ShaderBindingLayout;
			Initializer.ShaderBindingLayout = &ShaderBindingLayout;
		}		
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		FRayTracingPipelineState* RayTracingPipeline = static_cast<FRayTracingPipelineState*>(Pipeline);
		check(!RayTracingPipeline->RHIPipeline.IsValid());
		RayTracingPipeline->RHIPipeline = RHICreateRayTracingPipelineState(Initializer);

		// References to shaders no longer need to be held by this task

		ReleaseShaders(CallableTable);
		ReleaseShaders(HitGroupTable);
		ReleaseShaders(MissTable);
		ReleaseShaders(RayGenTable);

		Initializer = FRayTracingPipelineStateInitializer();
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FCompileRayTracingPipelineStateTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		// NOTE: RT PSO compilation internally spawns high-priority shader compilation tasks and waits on them.
		// FCompileRayTracingPipelineStateTask itself must run at lower priority to prevent deadlocks when
		// there are multiple RTPSO tasks that all wait on compilation via WaitUntilTasksComplete().
		return bBackgroundTask ? ENamedThreads::AnyBackgroundThreadNormalTask : ENamedThreads::AnyNormalThreadNormalTask;
	}

private:

	void AddRefShaders(TArray<FRHIRayTracingShader*>& ShaderTable)
	{
		for (FRHIRayTracingShader* Ptr : ShaderTable)
		{
			Ptr->AddRef();
		}
	}

	void ReleaseShaders(TArray<FRHIRayTracingShader*>& ShaderTable)
	{
		for (FRHIRayTracingShader* Ptr : ShaderTable)
		{
			Ptr->Release();
		}
	}

	TArray<FRHIRayTracingShader*> CopyShaderTable(const TArrayView<FRHIRayTracingShader*>& Source)
	{
		TArray<FRHIRayTracingShader*> Result(Source.GetData(), Source.Num());
		AddRefShaders(Result);
		return Result;
	}

	TArray<FRHIRayTracingShader*> RayGenTable;
	TArray<FRHIRayTracingShader*> MissTable;
	TArray<FRHIRayTracingShader*> HitGroupTable;
	TArray<FRHIRayTracingShader*> CallableTable;

	FRHIShaderBindingLayout ShaderBindingLayout;
};

static bool ValidateRayTracingPipelinePayloadMask(const FRayTracingPipelineStateInitializer& InInitializer)
{
	if (InInitializer.GetRayGenTable().IsEmpty())
	{
		// if we don't have any raygen shaders, the RTPSO is not complete and we can't really do any validation
		return true;
	}
	uint32 BaseRayTracingPayloadType = 0;
	for (FRHIRayTracingShader* Shader : InInitializer.GetRayGenTable())
	{
		checkf(Shader != nullptr, TEXT("RayGen shader table should not contain any NULL entries."));
		BaseRayTracingPayloadType |= Shader->RayTracingPayloadType; // union of all possible bits the raygen shaders want
		checkf(Shader->RayTracingPayloadSize <= InInitializer.MaxPayloadSizeInBytes,
			TEXT("Raytracing shader has a %u byte payload, but RTPSO has max set to %u"),
			Shader->RayTracingPayloadSize,
			InInitializer.MaxPayloadSizeInBytes);
	}
	for (FRHIRayTracingShader* Shader : InInitializer.GetMissTable())
	{
		checkf(Shader != nullptr, TEXT("Miss shader table should not contain any NULL entries"));
		checkf((Shader->RayTracingPayloadType & BaseRayTracingPayloadType) == Shader->RayTracingPayloadType,
			TEXT("Mismatched Ray Tracing Payload type among miss shaders! Found payload type %d but expecting %d"),
			Shader->RayTracingPayloadType,
			BaseRayTracingPayloadType);
		checkf(Shader->RayTracingPayloadSize <= InInitializer.MaxPayloadSizeInBytes,
			TEXT("Raytracing shader has a %u byte payload, but RTPSO has max set to %u"),
			Shader->RayTracingPayloadSize,
			InInitializer.MaxPayloadSizeInBytes);
	}
	for (FRHIRayTracingShader* Shader : InInitializer.GetHitGroupTable())
	{
		checkf(Shader != nullptr, TEXT("Hit group shader table should not contain any NULL entries"));
		checkf((Shader->RayTracingPayloadType & BaseRayTracingPayloadType) == Shader->RayTracingPayloadType,
			TEXT("Mismatched Ray Tracing Payload type among hitgroup shaders! Found payload type %d but expecting %d"),
			Shader->RayTracingPayloadType,
			BaseRayTracingPayloadType);
		checkf(Shader->RayTracingPayloadSize <= InInitializer.MaxPayloadSizeInBytes,
			TEXT("Raytracing shader has a %u byte payload, but RTPSO has max set to %u"),
			Shader->RayTracingPayloadSize,
			InInitializer.MaxPayloadSizeInBytes);
	}
	for (FRHIRayTracingShader* Shader : InInitializer.GetCallableTable())
	{
		checkf(Shader != nullptr, TEXT("Callable shader table should not contain any NULL entries"));
		checkf((Shader->RayTracingPayloadType & BaseRayTracingPayloadType) == Shader->RayTracingPayloadType,
			TEXT("Mismatched Ray Tracing Payload type among callable shaders! Found payload type %d but expecting %d"),
			Shader->RayTracingPayloadType,
			BaseRayTracingPayloadType);
		checkf(Shader->RayTracingPayloadSize <= InInitializer.MaxPayloadSizeInBytes,
			TEXT("Raytracing shader has a %u byte payload, but RTPSO has max set to %u"),
			Shader->RayTracingPayloadSize,
			InInitializer.MaxPayloadSizeInBytes);
	}
	// pass the check that called us, any failure above is sufficient
	return true;
}

#endif // RHI_RAYTRACING

FRayTracingPipelineState* PipelineStateCache::GetAndOrCreateRayTracingPipelineState(
	FRHICommandList& RHICmdList,
	const FRayTracingPipelineStateInitializer& InInitializer,
	ERayTracingPipelineCacheFlags Flags)
{
#if RHI_RAYTRACING
	LLM_SCOPE(ELLMTag::PSO);

	check(IsInRenderingThread() || IsInParallelRenderingThread());
	check(ValidateRayTracingPipelinePayloadMask(InInitializer));

	const bool bDoAsyncCompile = IsAsyncCompilationAllowed(RHICmdList, false);
	const bool bNonBlocking = !!(Flags & ERayTracingPipelineCacheFlags::NonBlocking);

	FRayTracingPipelineState* Result = nullptr;

	bool bWasFound = GRayTracingPipelineCache.Find(InInitializer, Result);

	if (bWasFound)
	{
		if (!Result->IsCompilationComplete())
		{
			if (!bDoAsyncCompile)
			{
				// Pipeline is in cache, but compilation is not finished and async compilation is disallowed, so block here RHI pipeline is created.
				Result->WaitCompletion();
			}
			else if (bNonBlocking)
			{
				// Pipeline is in cache, but compilation has not finished yet, so it can't be used for rendering.
				// Caller must use a fallback pipeline now and try again next frame.
				Result = nullptr;
			}
			else
			{
				// Pipeline is in cache, but compilation is not finished and caller requested blocking mode.
				// RHI command list can't begin translation until this event is complete.
				RHICmdList.AddDispatchPrerequisite(Result->CompletionEvent);
			}
		}
		else
		{
			checkf(Result->RHIPipeline.IsValid(), TEXT("If pipeline is in cache and it doesn't have a completion event, then RHI pipeline is expected to be ready"));
		}
	}
	else
	{
		FPipelineFileCacheManager::CacheRayTracingPSO(InInitializer, Flags);

		// Copy the initializer as we may want to patch it below
		FRayTracingPipelineStateInitializer Initializer = InInitializer;

		// If explicit base pipeline is not provided then find a compatible one from the cache
		if (GRHISupportsRayTracingPSOAdditions && InInitializer.BasePipeline == nullptr)
		{
			FRayTracingPipelineState* BasePipeline = nullptr;
			bool bBasePipelineFound = GRayTracingPipelineCache.FindBase(Initializer, BasePipeline);
			if (bBasePipelineFound)
			{
				Initializer.BasePipeline = BasePipeline->RHIPipeline;
			}
		}

		// Remove old pipelines once per frame
		const int32 TargetCacheSize = CVarRTPSOCacheSize.GetValueOnAnyThread();
		if (TargetCacheSize > 0 && GRayTracingPipelineCache.GetLastTrimFrame() != GFrameCounter)
		{
			GRayTracingPipelineCache.Trim(TargetCacheSize);
		}

		Result = GRayTracingPipelineCache.Add(Initializer);

		if (bDoAsyncCompile)
		{
			Result->CompletionEvent = TGraphTask<FCompileRayTracingPipelineStateTask>::CreateTask().ConstructAndDispatchWhenReady(
				Result,
				Initializer,
				bNonBlocking);

			// Partial or non-blocking pipelines can't be used for rendering, therefore this command list does not need to depend on them.

			if (bNonBlocking)
			{
				Result = nullptr;
			}
			else if (!Initializer.bPartial)
			{
				RHICmdList.AddDispatchPrerequisite(Result->CompletionEvent);
			}
		}
		else
		{
			Result->RHIPipeline = RHICreateRayTracingPipelineState(Initializer);
		}
	}

	if (Result)
	{
		Result->AddHit();
	}

	return Result;

#else // RHI_RAYTRACING
	return nullptr;
#endif // RHI_RAYTRACING
}

FRayTracingPipelineState* PipelineStateCache::GetRayTracingPipelineState(const FRayTracingPipelineStateSignature& Signature)
{
#if RHI_RAYTRACING
	FRayTracingPipelineState* Result = nullptr;
	bool bWasFound = GRayTracingPipelineCache.FindBySignature(Signature, Result);
	if (bWasFound)
	{
		Result->AddHit();
	}
	return Result;
#else // RHI_RAYTRACING
	return nullptr;
#endif // RHI_RAYTRACING
}

FRHIComputePipelineState* ExecuteSetComputePipelineState(FComputePipelineState* PipelineState)
{
	ensure(PipelineState->RHIPipeline);
	PipelineState->AddUse(); // Update Stats
	PipelineState->Verify_DecUse(); // Lifetime Tracking
	return PipelineState->RHIPipeline;
}


inline void ValidateGraphicsPipelineStateInitializer(const FGraphicsPipelineStateInitializer& Initializer)
{
	if (GRHISupportsMeshShadersTier0)
	{
		if (!Initializer.BoundShaderState.VertexShaderRHI && !Initializer.BoundShaderState.GetMeshShader())
		{
			UE_LOG(LogRHI, Warning, TEXT("GraphicsPipelineState must include a vertex or mesh shader"));
		}
	}
	else
	{
		if (!Initializer.BoundShaderState.VertexShaderRHI)
		{
			UE_LOG(LogRHI, Warning, TEXT("GraphicsPipelineState must include a vertex shader"));
		}
	}

	check(Initializer.DepthStencilState && Initializer.BlendState && Initializer.RasterizerState);
}

static void InternalCreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer, EPSOPrecacheResult PSOPrecacheResult, EPSOPrecachePriority PSOPrecachePriority, bool bPrecachedPSOMarkedUsed, bool bDoAsyncCompile, FGraphicsPipelineState* CachedState, const FPSOCompilationDebugData& PSOCompilationDebugData, bool bInImmediateCmdList)
{
	FGraphEventRef GraphEvent = CachedState->GetCompletionEvent();

	// create a compilation task, or just do it now...
	if (bDoAsyncCompile)
	{
		check(GraphEvent != nullptr);
		InternalAsyncCompilePipelineState<FCompileGraphicsPipelineStateTask, FGraphicsPipelineStateInitializer>(Initializer, PSOPrecacheResult, PSOPrecachePriority, bPrecachedPSOMarkedUsed, CachedState, PSOCompilationDebugData, bInImmediateCmdList);
	}
	else
	{		
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(PSOCompilationDebugData.PassName, PSOCompilationDebugData.PassName != nullptr);
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(*PSOCompilationDebugData.ResourceName, !PSOCompilationDebugData.ResourceName.IsEmpty());

		check(GraphEvent == nullptr);
		uint64 StartTime = FPlatformTime::Cycles64();
		CachedState->RHIPipeline = RHICreateGraphicsPipelineState(Initializer);
		uint64 PSOCompilationTime = CheckAndUpdateHitchCountStat(FPSOPrecacheRequestID::EType::Graphics, !IsPrecachedPSO(Initializer), PSOCompilationDebugData, StartTime, PSOPrecacheResult);

		if (Initializer.bPSOPrecache)
		{
			GPrecacheGraphicsPipelineCache->PrecacheFinished(Initializer, CachedState->RHIPipeline != nullptr, PSOCompilationTime);
		}

		if (CachedState->RHIPipeline)
		{
			CachedState->SortKey = CachedState->RHIPipeline->GetSortKey();
		}
		else
		{
			HandlePipelineCreationFailure(Initializer);
		}
	}
}

FGraphicsPipelineState* PipelineStateCache::GetAndOrCreateGraphicsPipelineState(FRHICommandList& RHICmdList, const FGraphicsPipelineStateInitializer& Initializer, EApplyRendertargetOption ApplyFlags)
{
	LLM_SCOPE(ELLMTag::PSO);
	ValidateGraphicsPipelineStateInitializer(Initializer);

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST 
	if (ApplyFlags == EApplyRendertargetOption::CheckApply)
	{
		// Catch cases where the state does not match
		FGraphicsPipelineStateInitializer NewInitializer = Initializer;
		RHICmdList.ApplyCachedRenderTargets(NewInitializer);

		int32 AnyFailed = 0;
		AnyFailed |= (NewInitializer.RenderTargetsEnabled != Initializer.RenderTargetsEnabled) << 0;

		if (AnyFailed == 0)
		{
			for (int32 i = 0; i < (int32)NewInitializer.RenderTargetsEnabled; i++)
			{
				AnyFailed |= (NewInitializer.RenderTargetFormats[i] != Initializer.RenderTargetFormats[i]) << 1;
				// as long as RT formats match, the flags shouldn't matter. We only store format-influencing flags in the recorded PSOs, so the check would likely fail.
				//AnyFailed |= (NewInitializer.RenderTargetFlags[i] != Initializer.RenderTargetFlags[i]) << 2;
				if (AnyFailed)
				{
					AnyFailed |= i << 24;
					break;
				}
			}
		}

		AnyFailed |= (NewInitializer.DepthStencilTargetFormat != Initializer.DepthStencilTargetFormat) << 3;
		AnyFailed |= (NewInitializer.DepthStencilTargetFlag != Initializer.DepthStencilTargetFlag) << 4;
		AnyFailed |= (NewInitializer.DepthTargetLoadAction != Initializer.DepthTargetLoadAction) << 5;
		AnyFailed |= (NewInitializer.DepthTargetStoreAction != Initializer.DepthTargetStoreAction) << 6;
		AnyFailed |= (NewInitializer.StencilTargetLoadAction != Initializer.StencilTargetLoadAction) << 7;
		AnyFailed |= (NewInitializer.StencilTargetStoreAction != Initializer.StencilTargetStoreAction) << 8;

		checkf(!AnyFailed, TEXT("GetAndOrCreateGraphicsPipelineState RenderTarget check failed with: %i !"), AnyFailed);
	}
#endif

	// Precache PSOs should never go through here.
	ensure(!Initializer.bPSOPrecache);

	FGraphicsPipelineState* OutCachedState = nullptr;

	bool bWasFound = GGraphicsPipelineCache.Find(Initializer, OutCachedState);
	if (bWasFound == false)
	{
		EPSOPrecacheResult PSOPrecacheResult = EPSOPrecacheResult::Unknown;
		bool bFromInMemoryPSOCache = false;
		bool bPrecachedPSOMarkedUsed = false;
		if (IsPSOPrecachingEnabled())
		{
			FGraphicsPipelineStateInitializer PSOPrecacheInitializer = Initializer;
			if (PSOPrecacheInitializer.StatePrecachePSOHash == 0)
			{
				PSOPrecacheInitializer.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(PSOPrecacheInitializer);
			}

			if (ShouldTrackUsedPrecachedPSOs())
			{
				PSOPrecacheResult = GPrecacheGraphicsPipelineCache->GetPrecachingStateAndMarkUsed(PSOPrecacheInitializer, bPrecachedPSOMarkedUsed);
			}
			else
			{
				EPSOPrecacheInMemoryMode InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
				PSOPrecacheResult = GPrecacheGraphicsPipelineCache->GetPrecachingStateAndInMemoryMode(PSOPrecacheInitializer, InMemoryMode);
				bFromInMemoryPSOCache = InMemoryMode == EPSOPrecacheInMemoryMode::Cached;
			}
		}

		bool DoAsyncCompile = IsAsyncCompilationAllowed(RHICmdList, Initializer.bFromPSOFileCache);

		FPipelineStateStats* Stats = nullptr;
		FPipelineFileCacheManager::CacheGraphicsPSO(GetTypeHash(Initializer), Initializer, PSOPrecacheResult, &Stats);

		// create new graphics state
		bool bRuntimePSO = true;
		OutCachedState = new FGraphicsPipelineState(bRuntimePSO, bFromInMemoryPSOCache);
		OutCachedState->Stats = Stats;
		if (DoAsyncCompile)
		{
			OutCachedState->SetCompletionEvent(FGraphEvent::CreateGraphEvent());
		}

		if (!Initializer.bFromPSOFileCache)
		{
			GraphicsPipelineCacheMisses++;
		}

		// If the PSO is still precaching then mark as too late
		if (PSOPrecacheResult == EPSOPrecacheResult::Active)
		{
			PSOPrecacheResult = EPSOPrecacheResult::TooLate;
		}

		FPSOCompilationDebugData PSOCompilationDebugData;
#if WITH_RHI_BREADCRUMBS
		if (DoAsyncCompile && !Initializer.bFromPSOFileCache)
		{
			PSOCompilationDebugData.BreadcrumbRoot = FRHIBreadcrumbNode::GetNonNullRoot(RHICmdList.GetCurrentBreadcrumbRef());
			PSOCompilationDebugData.BreadcrumbNode = PSOCompilationDebugData.BreadcrumbRoot
				? RHICmdList.GetCurrentBreadcrumbRef()
				: nullptr;

			// Adding Shared ref to BreadcrumbNodeAllocator to keep it alive. During the warming of the PSO cache at launch, the RHICmdLis doesn't wait for the PSO to finish compiling.
			if (PSOCompilationDebugData.BreadcrumbNode)
			{
				PSOCompilationDebugData.BreadcrumbNodeAllocator = PSOCompilationDebugData.BreadcrumbNode->Allocator->AsShared();
			}
		}
#endif // WITH_RHI_BREADCRUMBS

		FGraphEventRef GraphEvent = OutCachedState->GetCompletionEvent();
		InternalCreateGraphicsPipelineState(Initializer, PSOPrecacheResult, EPSOPrecachePriority::Medium, bPrecachedPSOMarkedUsed, DoAsyncCompile, OutCachedState, PSOCompilationDebugData, RHICmdList.IsImmediate());

		// Add dispatch pre requisite for non precaching jobs only
		//if (GraphEvent.IsValid() && (!bPSOPrecache || !FPSOPrecacheThreadPool::UsePool()))
		if (GraphEvent.IsValid() && !Initializer.bFromPSOFileCache)
		{
			check(DoAsyncCompile);
			RHICmdList.AddDispatchPrerequisite(GraphEvent);
		}

		GGraphicsPipelineCache.Add(Initializer, OutCachedState);
	}
	else
	{
		if (!Initializer.bFromPSOFileCache && !OutCachedState->IsComplete())
		{
			if (OutCachedState->GetPrecompileTask())
			{
				// if this is an in-progress threadpool precompile task then it could be seconds away in the queue.
				// Reissue this task so that it jumps the precompile queue.
				OutCachedState->GetPrecompileTask()->Reschedule(&GPSOPrecacheThreadPool.Get(), EQueuedWorkPriority::Highest);
#if PSO_TRACK_CACHE_STATS
		UE_LOGF(LogRHI, Log, "An incomplete precompile task was required for rendering!");
#endif
			}
			RHICmdList.AddDispatchPrerequisite(OutCachedState->GetCompletionEvent());
		}

#if PSO_TRACK_CACHE_STATS
		OutCachedState->AddHit();
#endif
	}

	// return the state pointer
	return OutCachedState;
}

FComputePipelineState* PipelineStateCache::FindComputePipelineState(const FComputePipelineStateInitializer& Initializer, bool bVerifyUse)
{
	LLM_SCOPE(ELLMTag::PSO);
	check(Initializer.ComputeShader != nullptr);

	FComputePipelineState* PipelineState = nullptr;
	GComputePipelineCache.Find(Initializer, PipelineState);

	if (PipelineState && PipelineState->IsComplete())
	{
		if (bVerifyUse)
		{
			PipelineState->Verify_IncUse();
		}

		return PipelineState;
	}
	else
	{
		return nullptr;
	}
}

FWorkGraphPipelineState* PipelineStateCache::FindWorkGraphPipelineState(const FWorkGraphPipelineStateInitializer& Initializer, bool bVerifyUse)
{
	LLM_SCOPE(ELLMTag::PSO);
	ValidateWorkGraphPipelineStateInitializer(Initializer);

	FWorkGraphPipelineState* PipelineState = nullptr;
	GWorkGraphPipelineCache.Find(Initializer, PipelineState);

	if (PipelineState && PipelineState->IsComplete())
	{
		if (bVerifyUse)
		{
			PipelineState->Verify_IncUse();
		}

		return PipelineState;
	}
	else
	{
		return nullptr;
	}
}

FGraphicsPipelineState* PipelineStateCache::FindGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer, bool bVerifyUse)
{
	LLM_SCOPE(ELLMTag::PSO);
	ValidateGraphicsPipelineStateInitializer(Initializer);

	FGraphicsPipelineState* PipelineState = nullptr;
	GGraphicsPipelineCache.Find(Initializer, PipelineState);

	if (PipelineState && PipelineState->IsComplete())
	{
		if (bVerifyUse)
		{
			PipelineState->Verify_IncUse();
		}

		return PipelineState;
	}
	else
	{
		return nullptr;
	}
}

bool PipelineStateCache::IsPSOPrecachingEnabled()
{	
#if WITH_EDITOR
	// Disables in the editor for now by default untill more testing is done - still WIP
	return false;
#else
	return GPSOPrecaching != 0 && GRHISupportsPSOPrecaching;
#endif // WITH_EDITOR
}

FPSOPrecacheRequestResult FPrecacheComputePipelineCache::PrecacheComputePipelineState(const FComputePipelineStateInitializer& ComputeInitializer, const FPSOPrecacheRequestParams& RequestParams)
{
	FPSOPrecacheRequestResult Result;
	if (!PipelineStateCache::IsPSOPrecachingEnabled() && !RequestParams.bForcePrecache)
	{
		return Result;
	}
	if (ComputeInitializer.ComputeShader == nullptr)
	{
		return Result;
	}

	static bool bDoAsyncCompile = FApp::ShouldUseThreadingForPerformance();

	// Mark as precache.
	FComputePipelineStateInitializer PrecacheComputeInitializer(ComputeInitializer);
	PrecacheComputeInitializer.bPSOPrecache = true;
	
	return TryAddNewState(PrecacheComputeInitializer, RequestParams, bDoAsyncCompile);
}

void FPrecacheComputePipelineCache::OnNewPipelineStateCreated(const FComputePipelineStateInitializer& ComputeInitializer, const FPSOPrecacheRequestParams& RequestParams, FComputePipelineState* CachedState, bool bDoAsyncCompile)
{
	check(ComputeInitializer.ComputeShader);
	check(ComputeInitializer.bPSOPrecache);

	FPSOCompilationDebugData PSOCompilationDebugData;
	PSOCompilationDebugData.PassName = RequestParams.CollectorName;
	PSOCompilationDebugData.ResourceName = RequestParams.ResourceName;

	InternalCreateComputePipelineState(ComputeInitializer, bDoAsyncCompile, EPSOPrecacheResult::Active, RequestParams.PrecachePriority, false, CachedState, PSOCompilationDebugData, false);
}

FPSOPrecacheRequestResult PipelineStateCache::PrecacheComputePipelineState(const FComputePipelineStateInitializer& ComputeInitializer, const FPSOPrecacheRequestParams& RequestParams)
{
	return GPrecacheComputePipelineCache->PrecacheComputePipelineState(ComputeInitializer, RequestParams);
}

FPSOPrecacheRequestResult FPrecacheGraphicsPipelineCache::PrecacheGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer, const FPSOPrecacheRequestParams& RequestParams)
{
	FPSOPrecacheRequestResult Result;
	if (!PipelineStateCache::IsPSOPrecachingEnabled())
	{
		return Result;
	}

	LLM_SCOPE(ELLMTag::PSO);

	// Use async compilation if available
	static bool bDoAsyncCompile = FApp::ShouldUseThreadingForPerformance();

	return TryAddNewState(Initializer, RequestParams, bDoAsyncCompile);
}
	
void FPrecacheGraphicsPipelineCache::OnNewPipelineStateCreated(const FGraphicsPipelineStateInitializer & Initializer, const FPSOPrecacheRequestParams& RequestParams, FGraphicsPipelineState * NewGraphicsPipelineState, bool bDoAsyncCompile)
{
	ValidateGraphicsPipelineStateInitializer(Initializer);
	check((NewGraphicsPipelineState->GetCompletionEvent() != nullptr) == bDoAsyncCompile);

	// Mark as precache so it will try and use the background thread pool if available
	FGraphicsPipelineStateInitializer InitializerCopy(Initializer);
	InitializerCopy.bPSOPrecache = true;
	
	FPSOCompilationDebugData PSOCompilationDebugData;
	PSOCompilationDebugData.PassName = RequestParams.CollectorName;
	PSOCompilationDebugData.ResourceName = RequestParams.ResourceName;

	// Start the precache task	
	InternalCreateGraphicsPipelineState(InitializerCopy, EPSOPrecacheResult::Active, RequestParams.PrecachePriority, false, bDoAsyncCompile, NewGraphicsPipelineState, PSOCompilationDebugData, false);
}

FPSOPrecacheRequestResult PipelineStateCache::PrecacheGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer, const FPSOPrecacheRequestParams& RequestParams)
{
	return GPrecacheGraphicsPipelineCache->PrecacheGraphicsPipelineState(Initializer, RequestParams);
}

EPSOPrecacheResult PipelineStateCache::CheckPipelineStateInCache(const FGraphicsPipelineStateInitializer& PipelineStateInitializer, FGraphEventRef& OutCompletionEvent, FPSOPrecacheRequestID& OutPSOPrecacheRequestID)
{
	if (!IsPSOPrecachingEnabled())
	{
		return EPSOPrecacheResult::Unknown;
	}

	return GPrecacheGraphicsPipelineCache->GetPrecachingState(PipelineStateInitializer, OutCompletionEvent, OutPSOPrecacheRequestID);
}

EPSOPrecacheResult PipelineStateCache::CheckPipelineStateInCache(FRHIComputeShader* ComputeShader, FGraphEventRef& OutCompletionEvent, FPSOPrecacheRequestID& OutPSOPrecacheRequestID)
{
	if (!IsPSOPrecachingEnabled() || ComputeShader == nullptr)
	{
		return EPSOPrecacheResult::Unknown;
	}

	return GPrecacheComputePipelineCache->GetPrecachingState(FComputePipelineStateInitializer(ComputeShader), OutCompletionEvent, OutPSOPrecacheRequestID);
}

uint64 PipelineStateCache::GetPrecachePSOCompilationTime(bool bGraphics, uint64 PSOPrecacheHash)
{
	if (!IsPSOPrecachingEnabled())
	{
		return 0;
	}

	return bGraphics ? GPrecacheGraphicsPipelineCache->GetPSOCompilationTime(PSOPrecacheHash) : GPrecacheComputePipelineCache->GetPSOCompilationTime(PSOPrecacheHash);
}

bool PipelineStateCache::ReleasePrecachedPSO(const FPSOPrecacheRequestID& PSOPrecacheRequestID)
{
	if (!IsPSOPrecachingEnabled())
	{
		return false;
	}

	bool bReleased = false;
	if (PSOPrecacheRequestID.GetType() == FPSOPrecacheRequestID::EType::Graphics)
	{
		bReleased = GPrecacheGraphicsPipelineCache->ReleasePrecachedPSO(PSOPrecacheRequestID);
	}
	else
	{
		bReleased = GPrecacheComputePipelineCache->ReleasePrecachedPSO(PSOPrecacheRequestID);
	}
	return bReleased;
}

bool PipelineStateCache::IsPrecaching(const FPSOPrecacheRequestID& PSOPrecacheRequestID)
{
	if (!IsPSOPrecachingEnabled())
	{
		return false;
	}

	EPSOPrecacheResult PrecacheResult = EPSOPrecacheResult::Unknown;
	if (PSOPrecacheRequestID.GetType() == FPSOPrecacheRequestID::EType::Graphics)
	{
		PrecacheResult = GPrecacheGraphicsPipelineCache->GetPrecachingState(PSOPrecacheRequestID);
	}
	else
	{
		PrecacheResult = GPrecacheComputePipelineCache->GetPrecachingState(PSOPrecacheRequestID);
	}
	return PrecacheResult == EPSOPrecacheResult::Active;
}

bool PipelineStateCache::IsPrecaching(const FGraphicsPipelineStateInitializer& PipelineStateInitializer)
{
	if (!IsPSOPrecachingEnabled())
	{
		return false;
	}

	FGraphEventRef CompletionEvent;	
	FPSOPrecacheRequestID PSOPrecacheRequestID;
	return GPrecacheGraphicsPipelineCache->GetPrecachingState(PipelineStateInitializer, CompletionEvent, PSOPrecacheRequestID) == EPSOPrecacheResult::Active;
}

bool PipelineStateCache::IsPrecaching(FRHIComputeShader* ComputeShader)
{
	if (!IsPSOPrecachingEnabled())
	{
		return false;
	}

	FGraphEventRef CompletionEvent;	
	FPSOPrecacheRequestID PSOPrecacheRequestID;
	return GPrecacheComputePipelineCache->GetPrecachingState(FComputePipelineStateInitializer(ComputeShader), CompletionEvent, PSOPrecacheRequestID) == EPSOPrecacheResult::Active;
}

bool PipelineStateCache::IsPrecaching()
{
	if (!IsPSOPrecachingEnabled())
	{
		return false;
	}

	return GPrecacheGraphicsPipelineCache->IsPrecaching() || GPrecacheComputePipelineCache->IsPrecaching();
}

void PipelineStateCache::BoostPrecachePriority(EPSOPrecachePriority PSOPrecachePriority, const FPSOPrecacheRequestID& PSOPrecacheRequestID)
{
	if (IsPSOPrecachingEnabled())
	{
		PSOPrecachePriority = GForceHighToHighestPri && PSOPrecachePriority == EPSOPrecachePriority::High ? EPSOPrecachePriority::Highest : PSOPrecachePriority;

		if (PSOPrecacheRequestID.GetType() == FPSOPrecacheRequestID::EType::Graphics)
		{
			GPrecacheGraphicsPipelineCache->BoostPriority(PSOPrecachePriority, PSOPrecacheRequestID);
		}
		else
		{
			GPrecacheComputePipelineCache->BoostPriority(PSOPrecachePriority, PSOPrecacheRequestID);
		}
	}
}


void PipelineStateCache::PrecachePSOsBoostToHighestPriority(bool bForceHighest)
{
	bForceHighest = bForceHighest && GPSOPrecachePermitPriorityEscalation;
	UE_CLOGF(GForceHighToHighestPri != bForceHighest, LogRHI, Log, "PipelineStateCache: PSO precaching %ls highest priority boost", bForceHighest ? TEXT("enabling") : TEXT("disabling"));
	GForceHighToHighestPri = bForceHighest;
#if PSO_TRACK_CACHE_STATS
	DumpPipelineCacheStats();
#endif
}

uint32 PipelineStateCache::NumActivePrecacheRequests()
{
	if (!IsPSOPrecachingEnabled())
	{
		return 0;
	}

	return GPrecacheGraphicsPipelineCache->NumActivePrecacheRequests() + GPrecacheComputePipelineCache->NumActivePrecacheRequests();
}

FPSORuntimeCreationStats PipelineStateCache::GetPSORuntimeCreationStats()
{
	FPSORuntimeCreationStats Stats;
	Stats.TotalPSOCreations = RuntimePSOCreationCount.load(std::memory_order_relaxed);
	Stats.ComputePSOHitches = ComputePSOCreationHitchCount.load(std::memory_order_relaxed);
	Stats.GraphicsPSOHitches = GraphicsPSOCreationHitchCount.load(std::memory_order_relaxed);
	Stats.PreviouslyPrecachedPSOHitches = PrecachedPSOCreationHitchCount.load(std::memory_order_relaxed);
	Stats.SuspectedUnhealthyDriverCachePSOHitches = UnhealthyDriverCachePSOCreationHitchCount.load(std::memory_order_relaxed);
	Stats.bDriverCacheSuspectedUnhealthy = bDriverCacheSuspectedUnhealthy.load(std::memory_order_relaxed);
	return Stats;
}

void PipelineStateCache::ResetPSOHitchTrackingStats()
{
	SET_DWORD_STAT(STAT_RuntimeGraphicsPSOHitchCount, 0);
	GraphicsPSOCreationHitchCount.store(0, std::memory_order_relaxed);

	SET_DWORD_STAT(STAT_RuntimeComputePSOHitchCount, 0);
	ComputePSOCreationHitchCount.store(0, std::memory_order_relaxed);

	PrecachedPSOCreationHitchCount.store(0, std::memory_order_relaxed);
	UnhealthyDriverCachePSOCreationHitchCount.store(0, std::memory_order_relaxed);
	bDriverCacheSuspectedUnhealthy.store(false, std::memory_order_relaxed);
}

FRHIGraphicsPipelineState* ExecuteSetGraphicsPipelineState(FGraphicsPipelineState* GraphicsPipelineState)
{
	FRHIGraphicsPipelineState* RHIPipeline = GraphicsPipelineState->RHIPipeline;
	GraphicsPipelineState->AddUse(); // Update Stats
	GraphicsPipelineState->Verify_DecUse(); // Lifetime Tracking
	return RHIPipeline;
}

void DumpPipelineCacheStats()
{
#if PSO_TRACK_CACHE_STATS
	double TotalTime = 0.0;
	double MinTime = FLT_MAX;
	double MaxTime = FLT_MIN;

	int MinFrames = INT_MAX;
	int MaxFrames = INT_MIN;
	int TotalFrames = 0;

	int NumUsedLastMin = 0;
	int NumHits = 0;
	int NumHitsAcrossFrames = 0;
	int NumItemsMultipleFrameHits = 0;

	int NumCachedItems = GGraphicsPipelineCache.CurrentMap->Num();

	if (NumCachedItems == 0)
	{
		return;
	}

	for (auto GraphicsPipeLine : *GGraphicsPipelineCache.CurrentMap)
	{
		FGraphicsPipelineState* State = GraphicsPipeLine.Value;

		// calc timestats
		double SinceUse = FPlatformTime::Seconds() - State->FirstUsedTime;

		TotalTime += SinceUse;

		if (SinceUse <= 30.0)
		{
			NumUsedLastMin++;
		}

		MinTime = FMath::Min(MinTime, SinceUse);
		MaxTime = FMath::Max(MaxTime, SinceUse);

		// calc frame stats
		int FramesUsed = State->LastFrameUsed - State->FirstFrameUsed;
		TotalFrames += FramesUsed;
		MinFrames = FMath::Min(MinFrames, FramesUsed);
		MaxFrames = FMath::Max(MaxFrames, FramesUsed);

		NumHits += State->Hits;

		if (State->HitsAcrossFrames > 0)
		{
			NumHitsAcrossFrames += State->HitsAcrossFrames;
			NumItemsMultipleFrameHits++;
		}
	}

	UE_LOGF(LogRHI, Log, "Have %d GraphicsPipeline entries", NumCachedItems);
	for (int i = 0; i < (int)EPSOPrecachePriority::Num; i++)
	{
		if(TotalPrecompileCompleteTime[i].load() > 0)
		{
			UE_LOGF(LogRHI, Log, "Threadpool precompile: pri %d: %d GraphicsPipeline in flight, %d Jobs started, %d completed", i, GPipelinePrecompileTasksInFlight.load(), TotalNumPrecompileJobs[i].load(), TotalNumPrecompileJobsCompleted[i].load());
			UE_LOGF(LogRHI, Log, "Threadpool precompile: pri %d: %f s avg precompile time", i, FPlatformTime::GetSecondsPerCycle64() * (TotalPrecompileCompleteTime[i].load() / FMath::Max((int64_t)1, TotalNumPrecompileJobsCompleted[i].load())));
			UE_LOGF(LogRHI, Log, "Threadpool precompile: pri %d: %f s max precompile time", i, FPlatformTime::GetSecondsPerCycle64() * (MaxPrecompileJobTime[i].load()));

			UE_LOGF(LogRHI, Log, "Threadpool precompile: pri %d: %f s avg precompile compile time", i, FPlatformTime::GetSecondsPerCycle64() * (TotalPrecompileCompileTime[i].load() / FMath::Max((int64_t)1, TotalNumPrecompileJobsCompleted[i].load())));
			UE_LOGF(LogRHI, Log, "Threadpool precompile: pri %d: %f s max precompile compile time", i, FPlatformTime::GetSecondsPerCycle64() * (MaxPrecompileTimeToCompile[i].load()));

			UE_LOGF(LogRHI, Log, "Threadpool precompile: pri %d: %f s avg precompile latency time", i, FPlatformTime::GetSecondsPerCycle64() * (TotalPrecompileTimeToBegin[i].load() / FMath::Max((int64_t)1, TotalNumPrecompileJobsCompleted[i].load())));
			UE_LOGF(LogRHI, Log, "Threadpool precompile: pri %d: %f s max precompile latency time", i, FPlatformTime::GetSecondsPerCycle64() * (MaxPrecompileTimeToBegin[i].load()));
		}
	}

	UE_LOGF(LogRHI, Log, "Secs Used: Min=%.02f, Max=%.02f, Avg=%.02f. %d used in last 30 secs", MinTime, MaxTime, TotalTime / NumCachedItems, NumUsedLastMin);
	UE_LOGF(LogRHI, Log, "Frames Used: Min=%d, Max=%d, Avg=%d", MinFrames, MaxFrames, TotalFrames / NumCachedItems);
	UE_LOGF(LogRHI, Log, "Hits: Avg=%d, Items with hits across frames=%d, Avg Hits across Frames=%d", NumHits / NumCachedItems, NumItemsMultipleFrameHits, NumHitsAcrossFrames / NumCachedItems);

	size_t TrackingMem = sizeof(FGraphicsPipelineStateInitializer) * GGraphicsPipelineCache.CurrentMap->Num();
	UE_LOGF(LogRHI, Log, "Tracking Mem: %d kb", TrackingMem / 1024);
#else
	UE_LOGF(LogRHI, Error, "DEfine PSO_TRACK_CACHE_STATS for state and stats!");
#endif // PSO_VALIDATE_CACHE
}

/** Global cache of vertex declarations. Note we don't store TRefCountPtrs, instead we AddRef() manually. */
static TMap<uint32, FRHIVertexDeclaration*> GVertexDeclarationCache;
static FCriticalSection GVertexDeclarationLock;

void PipelineStateCache::WaitForAllTasks()
{
	GComputePipelineCache.WaitTasksComplete();
	GWorkGraphPipelineCache.WaitTasksComplete();
	GGraphicsPipelineCache.WaitTasksComplete();

	if (GPrecacheGraphicsPipelineCache)
	{
		GPrecacheGraphicsPipelineCache->WaitTasksComplete();
	}
	if (GPrecacheComputePipelineCache)
	{
		GPrecacheComputePipelineCache->WaitTasksComplete();
	}
}

void PipelineStateCache::Init()
{
	GPrecacheGraphicsPipelineCache = MakeUnique<FPrecacheGraphicsPipelineCache>(GPSOPrecacheKeepInMemoryGraphicsMaxNum);
	GPrecacheComputePipelineCache = MakeUnique<FPrecacheComputePipelineCache>(GPSOPrecacheKeepInMemoryComputeMaxNum);
}

void PipelineStateCache::Shutdown()
{
	WaitForAllTasks();

#if RHI_RAYTRACING
	GRayTracingPipelineCache.Shutdown();
#endif

	GComputePipelineCache.Shutdown();
	GWorkGraphPipelineCache.Shutdown();
	GGraphicsPipelineCache.Shutdown();

	FPipelineFileCacheManager::Shutdown();

	for (auto Pair : GVertexDeclarationCache)
	{
		Pair.Value->Release();
	}
	GVertexDeclarationCache.Empty();

	GPSOPrecacheThreadPool.ShutdownThreadPool();

	GPrecacheGraphicsPipelineCache.Reset();
	GPrecacheComputePipelineCache.Reset();
}

FRHIVertexDeclaration*	PipelineStateCache::GetOrCreateVertexDeclaration(const FVertexDeclarationElementList& Elements)
{
	uint32 Key = FCrc::MemCrc_DEPRECATED(Elements.GetData(), Elements.Num() * sizeof(FVertexElement));

	FScopeLock ScopeLock(&GVertexDeclarationLock);
	FRHIVertexDeclaration** Found = GVertexDeclarationCache.Find(Key);
	if (Found)
	{
		return *Found;
	}

	FVertexDeclarationRHIRef NewDeclaration = RHICreateVertexDeclaration(Elements);

	// Add an extra reference so we don't have TRefCountPtr in the maps
	NewDeclaration->AddRef();
	GVertexDeclarationCache.Add(Key, NewDeclaration);
	return NewDeclaration;
}

void PipelineStateCache::GetPipelineStates(TArray<TRefCountPtr<FRHIResource>>& Out, bool bConsolidateCaches, UE::FTimeout ConsolidationTimeout)
{
	GComputePipelineCache.GetResources(Out, bConsolidateCaches, ConsolidationTimeout);
	GGraphicsPipelineCache.GetResources(Out, bConsolidateCaches, ConsolidationTimeout);
	GWorkGraphPipelineCache.GetResources(Out, bConsolidateCaches, ConsolidationTimeout);
#if RHI_RAYTRACING
	GRayTracingPipelineCache.GetResources(Out);
#endif // RHI_RAYTRACING
}
