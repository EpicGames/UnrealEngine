// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHI.cpp: Render Hardware Interface implementation.
=============================================================================*/

#include "RHI.h"
#include "Async/ParallelFor.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "RHITransientResourceAllocator.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "RHIFwd.h"
#include "String/LexFromString.h"
#include "RHIStrings.h"
#include "String/ParseTokens.h"
#include "Misc/BufferedOutputDevice.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Serialization/MemoryImage.h"
#include "Stats/StatsTrace.h"
#include "RHITextureReference.h"
#include "RHIStats.h"
#include "RHICommandList.h"
#include "RHIUniformBufferLayoutInitializer.h"
#include <type_traits>

#if RHI_ENABLE_RESOURCE_INFO
#include "HAL/FileManager.h"
#endif

IMPLEMENT_MODULE(FDefaultModuleImpl, RHI);

/** RHI Logging. */
DEFINE_LOG_CATEGORY(LogRHI);

IMPLEMENT_TYPE_LAYOUT(FRHIUniformBufferLayoutInitializer);
IMPLEMENT_TYPE_LAYOUT(FRHIUniformBufferResourceInitializer);

#if !defined(RHIRESOURCE_NUM_FRAMES_TO_EXPIRE)
	#define RHIRESOURCE_NUM_FRAMES_TO_EXPIRE 3
#endif

static TAutoConsoleVariable<int32> CVarDisableEngineAndAppRegistration(
	TEXT("r.DisableEngineAndAppRegistration"),
	0,
	TEXT("If true, disables engine and app registration, to disable GPU driver optimizations during debugging and development\n")
	TEXT("Changes will only take effect in new game/editor instances - can't be changed at runtime.\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarGraphicsAdapter(
	TEXT("r.GraphicsAdapter"),
	-1,
	TEXT("User request to pick a specific graphics adapter (e.g. when using a integrated graphics card with a discrete one)\n")
	TEXT("For Windows D3D, unless a specific adapter is chosen we reject Microsoft adapters because we don't want the software emulation.\n")
	TEXT("This takes precedence over -prefer{AMD|NVidia|Intel} when the value is >= 0.\n")
	TEXT(" -2: Take the first one that fulfills the criteria\n")
	TEXT(" -1: Favour non integrated because there are usually faster (default)\n")
	TEXT("  0: Adapter #0\n")
	TEXT("  1: Adapter #1, ..."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static FAutoConsoleCommandWithWorldAndArgs CVarRHISetGPUCaptureOptions(
	TEXT("r.RHISetGPUCaptureOptions"),
	TEXT("Utility function to change multiple CVARs useful when profiling or debugging GPU rendering. Setting to 1 or 0 will guarantee all options are in the appropriate state.\n")
	TEXT("r.showmaterialdrawevents, toggledrawevents."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* World)
	{
		if (Args.Num() > 0)
		{
			const bool bEnabled = Args[0].ToBool();
			FDynamicRHI::EnableIdealGPUCaptureOptions(bEnabled);
		}
		else
		{
			UE_LOGF(LogRHI, Display, "Usage: r.RHISetGPUCaptureOptions 0 or r.RHISetGPUCaptureOptions 1");
		}
	})
);

#if STATS
#include "ProfilingDebugging/CsvProfilerConfig.h"
#include "Stats/StatsData.h"
static void DumpRHIMemory(FOutputDevice& OutputDevice)
{
	TArray<FStatMessage> Stats;
	GetPermanentStats(Stats);

	FName NAME_STATGROUP_RHI(FStatGroup_STATGROUP_RHI::GetGroupName());
	OutputDevice.Logf(TEXT("RHI resource memory (not tracked by our allocator)"));
	int64 TotalMemory = 0;
	for (int32 Index = 0; Index < Stats.Num(); Index++)
	{
		FStatMessage const& Meta = Stats[Index];
		FName LastGroup = Meta.NameAndInfo.GetGroupName();
		if (LastGroup == NAME_STATGROUP_RHI && Meta.NameAndInfo.GetFlag(EStatMetaFlags::IsMemory))
		{
			OutputDevice.Logf(TEXT("%s"), *FStatsUtils::DebugPrint(Meta));
			TotalMemory += Meta.GetValue_int64();
		}
	}
	OutputDevice.Logf(TEXT("%.3fMB total"), TotalMemory / 1024.f / 1024.f);
}

static FAutoConsoleCommandWithOutputDevice GDumpRHIMemoryCmd(
	TEXT("rhi.DumpMemory"),
	TEXT("Dumps RHI memory stats to the log"),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic(DumpRHIMemory)
	);
#endif

//DO NOT USE THE STATIC FLINEARCOLORS TO INITIALIZE THIS STUFF.  
//Static init order is undefined and you will likely end up with bad values on some platforms.
const FClearValueBinding FClearValueBinding::None(EClearBinding::ENoneBound);
const FClearValueBinding FClearValueBinding::Black(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f));
const FClearValueBinding FClearValueBinding::BlackMaxAlpha(FLinearColor(0.0f, 0.0f, 0.0f, FLT_MAX));
const FClearValueBinding FClearValueBinding::White(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
const FClearValueBinding FClearValueBinding::Transparent(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
const FClearValueBinding FClearValueBinding::DepthOne(1.0f, 0);
const FClearValueBinding FClearValueBinding::DepthZero(0.0f, 0);
const FClearValueBinding FClearValueBinding::DepthNear(1.0f, 0);
const FClearValueBinding FClearValueBinding::DepthFar(0.0f, 0);
const FClearValueBinding FClearValueBinding::Green(FLinearColor(0.0f, 1.0f, 0.0f, 1.0f));
// Note: this is used as the default normal for DBuffer decals.  It must decode to a value of 0 in DecodeDBufferData.
const FClearValueBinding FClearValueBinding::DefaultNormal8Bit(FLinearColor(128.0f / 255.0f, 128.0f / 255.0f, 128.0f / 255.0f, 1.0f));

#if HAS_GPU_STATS

	FRHIDrawStatsCategory::FRHIDrawStatsCategory(FName InName)
		: Name(InName)
		, Index(GetManager().NumCategory++)
	{
		check(Index < MAX_DRAWCALL_CATEGORY);
		if (Index < MAX_DRAWCALL_CATEGORY)
		{
			GetManager().Array[Index] = this;
		}
	}

	FRHIDrawStatsCategory::FManager::FManager()
		: NumCategory(0)
	{
		FMemory::Memzero(Array);
		FMemory::Memzero(DisplayCounts);
	}

	FRHIDrawStatsCategory::FManager& FRHIDrawStatsCategory::GetManager()
	{
		// Categories are global scope objects, so the initialization order is undefined.
		// Lazy init the manager on first use.
		static FManager Manager;
		return Manager;
	}

	void FRHIDrawStatsCategory::FManager::AccumulateFrameStats(FDrawCounts const& NewCounts)
	{
		// Update the current counts using the maximum over the update period
		for (int32 GPUIndex = 0; GPUIndex < CurrentCounts.Num(); ++GPUIndex)
		{
			FDrawCountsPerGPU      & Dst = CurrentCounts[GPUIndex];
			FDrawCountsPerGPU const& Src = NewCounts[GPUIndex];

			for (int32 CategoryIndex = 0; CategoryIndex < Src.Num(); ++CategoryIndex)
			{
				Dst[CategoryIndex] = FMath::Max(Dst[CategoryIndex], Src[CategoryIndex]);
			}
		}

		// Only copy the display counters every half second keep things more stable.
		constexpr float TimeoutSeconds = 0.5;
		static double LastTime = 0.0;
		double CurrentTime = FPlatformTime::Seconds();

		if (CurrentTime - LastTime > TimeoutSeconds)
		{
			LastTime = CurrentTime;

			{
				FScopeLock Lock(&DisplayCountsCS);
				DisplayCounts = CurrentCounts;
			}
			
			// Reset the counts to zero
			CurrentCounts = {};
		}
	}

#endif

TRefCountPtr<FRHITexture> FRHITextureReference::DefaultTexture;

// This is necessary to get expected results for code that zeros, assigns and then CRC's the whole struct.
//
// See: https://en.cppreference.com/w/cpp/types/has_unique_object_representations
// "This trait was introduced to make it possible to determine whether a type can be correctly hashed by hashing its object representation as a byte array."
static_assert(std::has_unique_object_representations_v<FVertexElement>, "FVertexElement should not have compiler-injected padding");

FString FVertexElement::ToString() const
{
	return FString::Printf(TEXT("<%u %u %u %u %u %u>")
		, uint32(StreamIndex)
		, uint32(Offset)
		, uint32(Type)
		, uint32(AttributeIndex)
		, uint32(Stride)
		, uint32(bUseInstanceIndex)
	);
}

void FVertexElement::FromString(const FString& InSrc)
{
	FromString(FStringView(InSrc));
}

void FVertexElement::FromString(const FStringView& InSrc)
{
	constexpr int32 PartCount = 6;

	TArray<FStringView, TInlineAllocator<PartCount>> Parts;
	UE::String::ParseTokensMultiple(InSrc.TrimStartAndEnd(), {TEXT('\r'), TEXT('\n'), TEXT('\t'), TEXT('<'), TEXT('>'), TEXT(' ')},
		[&Parts](FStringView Part) { if (!Part.IsEmpty()) { Parts.Add(Part); } });

	check(Parts.Num() == PartCount && sizeof(Type) == 1); //not a very robust parser
	const FStringView* PartIt = Parts.GetData();
	LexFromString(StreamIndex, *PartIt++);
	LexFromString(Offset, *PartIt++);
	LexFromString((uint8&)Type, *PartIt++);
	LexFromString(AttributeIndex, *PartIt++);
	LexFromString(Stride, *PartIt++);
	LexFromString(bUseInstanceIndex, *PartIt++);
	check(Parts.GetData() + PartCount == PartIt);
}

uint32 GetTypeHash(const FSamplerStateInitializerRHI& Initializer)
{
	uint32 Hash = GetTypeHash(Initializer.Filter);
	Hash = HashCombine(Hash, GetTypeHash(Initializer.AddressU));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.AddressV));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.AddressW));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.MipBias));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.MinMipLevel));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.MaxMipLevel));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.MaxAnisotropy));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.BorderColor));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.SamplerComparisonFunction));
	return Hash;
}

bool operator== (const FSamplerStateInitializerRHI& A, const FSamplerStateInitializerRHI& B)
{
	bool bSame = 
		A.Filter == B.Filter &&
		A.AddressU == B.AddressU &&
		A.AddressV == B.AddressV &&
		A.AddressW == B.AddressW &&
		A.MipBias == B.MipBias &&
		A.MinMipLevel == B.MinMipLevel &&
		A.MaxMipLevel == B.MaxMipLevel &&
		A.MaxAnisotropy == B.MaxAnisotropy &&
		A.BorderColor == B.BorderColor &&
		A.SamplerComparisonFunction == B.SamplerComparisonFunction;
	return bSame;
}

uint32 GetTypeHash(const FRasterizerStateInitializerRHI& Initializer)
{
	uint32 Hash = GetTypeHash(Initializer.FillMode);
	Hash = HashCombine(Hash, GetTypeHash(Initializer.CullMode));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.DepthBias));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.SlopeScaleDepthBias));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.DepthClipMode));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.bAllowMSAA));
	return Hash;
}
	
bool operator== (const FRasterizerStateInitializerRHI& A, const FRasterizerStateInitializerRHI& B)
{
	bool bSame = 
		A.FillMode == B.FillMode && 
		A.CullMode == B.CullMode && 
		A.DepthBias == B.DepthBias && 
		A.SlopeScaleDepthBias == B.SlopeScaleDepthBias &&
		A.DepthClipMode == B.DepthClipMode &&
		A.bAllowMSAA == B.bAllowMSAA;
	return bSame;
}

uint32 GetTypeHash(const FDepthStencilStateInitializerRHI& Initializer)
{
	uint32 Hash = GetTypeHash(Initializer.bEnableDepthWrite);
	Hash = HashCombine(Hash, GetTypeHash(Initializer.DepthTest));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.bEnableFrontFaceStencil));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.FrontFaceStencilTest));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.FrontFaceStencilFailStencilOp));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.FrontFaceDepthFailStencilOp));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.FrontFacePassStencilOp));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.bEnableBackFaceStencil));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.BackFaceStencilTest));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.BackFaceStencilFailStencilOp));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.BackFaceDepthFailStencilOp));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.BackFacePassStencilOp));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.StencilReadMask));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.StencilWriteMask));
	return Hash;
}

bool operator== (const FDepthStencilStateInitializerRHI& A, const FDepthStencilStateInitializerRHI& B)
{
	bool bSame = 
		A.bEnableDepthWrite == B.bEnableDepthWrite && 
		A.DepthTest == B.DepthTest && 
		A.bEnableFrontFaceStencil == B.bEnableFrontFaceStencil && 
		A.FrontFaceStencilTest == B.FrontFaceStencilTest && 
		A.FrontFaceStencilFailStencilOp == B.FrontFaceStencilFailStencilOp && 
		A.FrontFaceDepthFailStencilOp == B.FrontFaceDepthFailStencilOp && 
		A.FrontFacePassStencilOp == B.FrontFacePassStencilOp && 
		A.bEnableBackFaceStencil == B.bEnableBackFaceStencil && 
		A.BackFaceStencilTest == B.BackFaceStencilTest && 
		A.BackFaceStencilFailStencilOp == B.BackFaceStencilFailStencilOp && 
		A.BackFaceDepthFailStencilOp == B.BackFaceDepthFailStencilOp && 
		A.BackFacePassStencilOp == B.BackFacePassStencilOp && 
		A.StencilReadMask == B.StencilReadMask && 
		A.StencilWriteMask == B.StencilWriteMask;
	return bSame;
}

FString FDepthStencilStateInitializerRHI::ToString() const
{
	return
		FString::Printf(TEXT("<%u %u ")
			, uint32(!!bEnableDepthWrite)
			, uint32(DepthTest)
		)
		+ FString::Printf(TEXT("%u %u %u %u %u ")
			, uint32(!!bEnableFrontFaceStencil)
			, uint32(FrontFaceStencilTest)
			, uint32(FrontFaceStencilFailStencilOp)
			, uint32(FrontFaceDepthFailStencilOp)
			, uint32(FrontFacePassStencilOp)
		)
		+ FString::Printf(TEXT("%u %u %u %u %u ")
			, uint32(!!bEnableBackFaceStencil)
			, uint32(BackFaceStencilTest)
			, uint32(BackFaceStencilFailStencilOp)
			, uint32(BackFaceDepthFailStencilOp)
			, uint32(BackFacePassStencilOp)
		)
		+ FString::Printf(TEXT("%u %u>")
			, uint32(StencilReadMask)
			, uint32(StencilWriteMask)
		);
}

void FDepthStencilStateInitializerRHI::FromString(const FString& InSrc)
{
	FromString(FStringView(InSrc));
}

void FDepthStencilStateInitializerRHI::FromString(const FStringView& InSrc)
{
	constexpr int32 PartCount = 14;

	TArray<FStringView, TInlineAllocator<PartCount>> Parts;
	UE::String::ParseTokensMultiple(InSrc.TrimStartAndEnd(), {TEXT('\r'), TEXT('\n'), TEXT('\t'), TEXT('<'), TEXT('>'), TEXT(' ')},
		[&Parts](FStringView Part) { if (!Part.IsEmpty()) { Parts.Add(Part); } });

	check(Parts.Num() == PartCount && sizeof(bool) == 1 && sizeof(FrontFaceStencilFailStencilOp) == 1 && sizeof(BackFaceStencilTest) == 1 && sizeof(BackFaceDepthFailStencilOp) == 1); //not a very robust parser

	const FStringView* PartIt = Parts.GetData();

	LexFromString((uint8&)bEnableDepthWrite, *PartIt++);
	LexFromString((uint8&)DepthTest, *PartIt++);

	LexFromString((uint8&)bEnableFrontFaceStencil, *PartIt++);
	LexFromString((uint8&)FrontFaceStencilTest, *PartIt++);
	LexFromString((uint8&)FrontFaceStencilFailStencilOp, *PartIt++);
	LexFromString((uint8&)FrontFaceDepthFailStencilOp, *PartIt++);
	LexFromString((uint8&)FrontFacePassStencilOp, *PartIt++);

	LexFromString((uint8&)bEnableBackFaceStencil, *PartIt++);
	LexFromString((uint8&)BackFaceStencilTest, *PartIt++);
	LexFromString((uint8&)BackFaceStencilFailStencilOp, *PartIt++);
	LexFromString((uint8&)BackFaceDepthFailStencilOp, *PartIt++);
	LexFromString((uint8&)BackFacePassStencilOp, *PartIt++);

	LexFromString(StencilReadMask, *PartIt++);
	LexFromString(StencilWriteMask, *PartIt++);

	check(Parts.GetData() + PartCount == PartIt);
}

FString FBlendStateInitializerRHI::ToString() const
{
	FString Result = "<";
	for (int32 Index = 0; Index < MaxSimultaneousRenderTargets; Index++)
	{
		Result += RenderTargets[Index].ToString();
	}
	Result += FString::Printf(TEXT("%d %d>"), uint32(!!bUseIndependentRenderTargetBlendStates), uint32(!!bUseAlphaToCoverage));
	return Result;
}

void FBlendStateInitializerRHI::FromString(const FString& InSrc)
{
	FromString(FStringView(InSrc));
}

void FBlendStateInitializerRHI::FromString(const FStringView& InSrc)
{
	// files written before bUseAlphaToCoverage change (added in CL 13846572) have one less part
	constexpr int32 BackwardCompatiblePartCount = MaxSimultaneousRenderTargets * FRenderTarget::NUM_STRING_FIELDS + 1;
	constexpr int32 PartCount = BackwardCompatiblePartCount + 1;

	TArray<FStringView, TInlineAllocator<PartCount>> Parts;
	UE::String::ParseTokensMultiple(InSrc.TrimStartAndEnd(), {TEXT('\r'), TEXT('\n'), TEXT('\t'), TEXT('<'), TEXT('>'), TEXT(' ')},
		[&Parts](FStringView Part) { if (!Part.IsEmpty()) { Parts.Add(Part); } });

	checkf((Parts.Num() == PartCount || Parts.Num() == BackwardCompatiblePartCount) && sizeof(bool) == 1, 
		TEXT("Expecting %d (or %d, for an older format) parts in the blendstate string, got %d"), PartCount, BackwardCompatiblePartCount, Parts.Num()); //not a very robust parser
	bool bHasAlphaToCoverageField = Parts.Num() == PartCount;

	const FStringView* PartIt = Parts.GetData();
	for (int32 Index = 0; Index < MaxSimultaneousRenderTargets; Index++)
	{
		RenderTargets[Index].FromString(MakeArrayView(PartIt, FRenderTarget::NUM_STRING_FIELDS));
		PartIt += FRenderTarget::NUM_STRING_FIELDS;
	}
	LexFromString((int8&)bUseIndependentRenderTargetBlendStates, *PartIt++);
	if (bHasAlphaToCoverageField)
	{
		LexFromString((int8&)bUseAlphaToCoverage, *PartIt++);
		check(Parts.GetData() + PartCount == PartIt);
	}
	else
	{
		bUseAlphaToCoverage = false;
		check(Parts.GetData() + BackwardCompatiblePartCount == PartIt);
	}
}

uint32 GetTypeHash(const FBlendStateInitializerRHI& Initializer)
{
	uint32 Hash = GetTypeHash(Initializer.bUseIndependentRenderTargetBlendStates);
	Hash = HashCombine(Hash, Initializer.bUseAlphaToCoverage);
	for (int32 i = 0; i < MaxSimultaneousRenderTargets; ++i)
	{
		Hash = HashCombine(Hash, GetTypeHash(Initializer.RenderTargets[i]));
	}
	
	return Hash;
}

bool operator== (const FBlendStateInitializerRHI& A, const FBlendStateInitializerRHI& B)
{
	bool bSame = A.bUseIndependentRenderTargetBlendStates == B.bUseIndependentRenderTargetBlendStates;
	bSame = bSame && A.bUseAlphaToCoverage == B.bUseAlphaToCoverage;
	for (int32 i = 0; i < MaxSimultaneousRenderTargets && bSame; ++i)
	{
		bSame = bSame && A.RenderTargets[i] == B.RenderTargets[i];
	}
	return bSame;
}


FString FBlendStateInitializerRHI::FRenderTarget::ToString() const
{
	return FString::Printf(TEXT("%u %u %u %u %u %u %u ")
		, uint32(ColorBlendOp)
		, uint32(ColorSrcBlend)
		, uint32(ColorDestBlend)
		, uint32(AlphaBlendOp)
		, uint32(AlphaSrcBlend)
		, uint32(AlphaDestBlend)
		, uint32(ColorWriteMask)
	);
}

void FBlendStateInitializerRHI::FRenderTarget::FromString(const TArray<FString>& Parts, int32 Index)
{
	check(Index + NUM_STRING_FIELDS <= Parts.Num());
	LexFromString((uint8&)ColorBlendOp, *Parts[Index++]);
	LexFromString((uint8&)ColorSrcBlend, *Parts[Index++]);
	LexFromString((uint8&)ColorDestBlend, *Parts[Index++]);
	LexFromString((uint8&)AlphaBlendOp, *Parts[Index++]);
	LexFromString((uint8&)AlphaSrcBlend, *Parts[Index++]);
	LexFromString((uint8&)AlphaDestBlend, *Parts[Index++]);
	LexFromString((uint8&)ColorWriteMask, *Parts[Index++]);
}

void FBlendStateInitializerRHI::FRenderTarget::FromString(TArrayView<const FStringView> Parts)
{
	check(Parts.Num() == NUM_STRING_FIELDS);
	const FStringView* PartIt = Parts.GetData();
	LexFromString((uint8&)ColorBlendOp, *PartIt++);
	LexFromString((uint8&)ColorSrcBlend, *PartIt++);
	LexFromString((uint8&)ColorDestBlend, *PartIt++);
	LexFromString((uint8&)AlphaBlendOp, *PartIt++);
	LexFromString((uint8&)AlphaSrcBlend, *PartIt++);
	LexFromString((uint8&)AlphaDestBlend, *PartIt++);
	LexFromString((uint8&)ColorWriteMask, *PartIt++);
}

uint32 GetTypeHash(const FBlendStateInitializerRHI::FRenderTarget& Initializer)
{
	uint32 Hash = GetTypeHash(Initializer.ColorBlendOp);
	Hash = HashCombine(Hash, GetTypeHash(Initializer.ColorDestBlend));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.ColorSrcBlend));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.AlphaBlendOp));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.AlphaDestBlend));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.AlphaSrcBlend));
	Hash = HashCombine(Hash, GetTypeHash(Initializer.ColorWriteMask));
	return Hash;
}

bool operator==(const FBlendStateInitializerRHI::FRenderTarget& A, const FBlendStateInitializerRHI::FRenderTarget& B)
{
	bool bSame = 
		A.ColorBlendOp == B.ColorBlendOp && 
		A.ColorDestBlend == B.ColorDestBlend && 
		A.ColorSrcBlend == B.ColorSrcBlend && 
		A.AlphaBlendOp == B.AlphaBlendOp && 
		A.AlphaDestBlend == B.AlphaDestBlend && 
		A.AlphaSrcBlend == B.AlphaSrcBlend && 
		A.ColorWriteMask == B.ColorWriteMask;
	return bSame;
}

#if RHI_ENABLE_RESOURCE_INFO

static FCriticalSection GRHIResourceTrackingCriticalSection;
static TSet<FRHIResource*> GRHITrackedResources;
static bool GRHITrackingResources = false;

bool FRHIResource::GetResourceInfo(FRHIResourceInfo& OutResourceInfo) const
{
	OutResourceInfo = FRHIResourceInfo{};
	return false;
}

void FRHIResource::BeginTrackingResource(FRHIResource* InResource)
{
	if (GRHITrackingResources)
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/ResourceTracking"));
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();

		FScopeLock Lock(&GRHIResourceTrackingCriticalSection);

		InResource->bBeingTracked = true;

		GRHITrackedResources.Add(InResource);
	}
}

void FRHIResource::EndTrackingResource(FRHIResource* InResource)
{
	if (InResource->bBeingTracked)
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/ResourceTracking"));
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		FScopeLock Lock(&GRHIResourceTrackingCriticalSection);
		GRHITrackedResources.Remove(InResource);
		InResource->bBeingTracked = false;
	}
}

void FRHIResource::StartTrackingAllResources()
{
	GRHITrackingResources = true;
}

void FRHIResource::StopTrackingAllResources()
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/ResourceTracking"));
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();
	FScopeLock Lock(&GRHIResourceTrackingCriticalSection);
	for (FRHIResource* Resource : GRHITrackedResources)
	{
		if (Resource)
		{
			Resource->bBeingTracked = false;
		}
	}
	GRHITrackedResources.Empty();
	GRHITrackingResources = false;
}

enum class EBooleanFilter
{
	No,
	Yes,
	All
};

static EBooleanFilter ParseBooleanFilter(const FString& InText)
{
	if (InText.Equals(TEXT("No"), ESearchCase::IgnoreCase))
	{
		return EBooleanFilter::No;
	}
	if (InText.Equals(TEXT("Yes"), ESearchCase::IgnoreCase))
	{
		return EBooleanFilter::Yes;
	}
	if (InText.Equals(TEXT("All"), ESearchCase::IgnoreCase))
	{
		return EBooleanFilter::All;
	}
	return EBooleanFilter::No;
}

static FAutoConsoleCommandWithWorldArgsAndOutputDevice GDumpRHIResourceCountsCmd(
	TEXT("rhi.DumpResourceCounts"),
	TEXT("Dumps RHI resource counts to the log"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, UWorld*, FOutputDevice& OutputDevice)
{
	int32 ResourceCounts[RRT_Num]{};
	int32 TotalResources{};

	FRHIResourceInfo ResourceInfo;

	{
		FScopeLock Lock(&GRHIResourceTrackingCriticalSection);

		TotalResources = GRHITrackedResources.Num();

		for (const FRHIResource* Resource : GRHITrackedResources)
		{
			if (Resource)
			{
				ERHIResourceType ResourceType = Resource->GetType();
				if (ResourceType > 0 && ResourceType < RRT_Num)
				{
					ResourceCounts[ResourceType]++;
				}
			}
		}
	}

	FBufferedOutputDevice BufferedOutput;
	FName CategoryName("RHIResources");

	BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("RHIResource Counts"));

	for (int32 Index = 0; Index < RRT_Num; Index++)
	{
		const int32 CurrentCount = ResourceCounts[Index];
		if (CurrentCount > 0)
		{
			BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("%s: %d"),
				StringFromRHIResourceType((ERHIResourceType)Index),
				CurrentCount);
		}
	}
	BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Total: %d"), TotalResources);

	BufferedOutput.RedirectTo(OutputDevice);
}));


namespace RHIInternal
{
	struct FResourceEntry
	{
		const FRHIResource* Resource;
		FRHIResourceInfo ResourceInfo;
	};

	struct FResourceFlags
	{
		bool bResident = false;
		bool bMarkedForDelete = false;
		bool bTransient = false;
		bool bStreaming = false;
		bool bRT = false;
		bool bDS = false;
		bool bUAV = false;
		bool bRTAS = false;
		bool bReserved = false;
		bool bHasFlags = false;

		FString GetString()
		{
			FString FlagsString;
			bool bHasFlag = false;
			if (bResident)
			{
				FlagsString += "Resident";
				bHasFlag = true;
			}
			if (bMarkedForDelete)
			{
				FlagsString += bHasFlag ? " | MarkedForDelete" : "MarkedForDelete";
				bHasFlag = true;
			}
			if (bTransient)
			{
				FlagsString += bHasFlag ? " | Transient" : "Transient";
				bHasFlag = true;
			}
			if (bStreaming)
			{
				FlagsString += bHasFlag ? " | Streaming" : "Streaming";
				bHasFlag = true;
			}
			if (bRT)
			{
				FlagsString += bHasFlag ? " | RT" : "RT";
				bHasFlag = true;
			}
			else if (bDS)
			{
				FlagsString += bHasFlag ? " | DS" : "DS";
				bHasFlag = true;
			}
			if (bUAV)
			{
				FlagsString += bHasFlag ? " | UAV" : "UAV";
				bHasFlag = true;
			}
			if (bRTAS)
			{
				FlagsString += bHasFlag ? " | RTAS" : "RTAS";
				bHasFlag = true;
			}
			if (bReserved)
			{
				FlagsString += bHasFlag ? " | Reserved" : "Reserved";
				bHasFlag = true;
			}
			return FlagsString;
		}
	};

	void GetTrackedResourcesInternal(const FString& NameFilter, ERHIResourceType TypeFilter, EBooleanFilter TransientFilter, TArray<FResourceEntry>& OutResources, int32& OutNumberOfResourcesToShow,
		int32& OutTotalResourcesWithInfo, int32& OutTotalTrackedResources, int64& OutTotalTrackedResourceSize, int64& OutTotalTrackedTransientResourceSize)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RHIInternal_GetTrackedResourcesInternal);

		TCHAR ResourceNameBuffer[FName::StringBufferSize];

		auto ShouldIncludeResource = [&](const FRHIResource* Resource, const FRHIResourceInfo& ResourceInfo) -> bool
		{
			if (!NameFilter.IsEmpty())
			{
				if (ResourceInfo.Name.ToString(ResourceNameBuffer) == 0 || UE::String::FindFirst(ResourceNameBuffer, *NameFilter, ESearchCase::IgnoreCase) == INDEX_NONE)
				{
					return false;
				}
			}
			if (TypeFilter != RRT_None)
			{
				if (TypeFilter == RRT_Texture)
				{
					return false;
				}
				else if (TypeFilter != ResourceInfo.Type)
				{
					return false;
				}
			}
			if (TransientFilter != EBooleanFilter::All)
			{
				const bool bAllowedFlag = TransientFilter == EBooleanFilter::Yes ? true : false;
				if (ResourceInfo.IsTransient != bAllowedFlag)
				{
					return false;
				}
			}
			return true;
		};

		OutResources.Reset();

		{
			FRHIResourceInfo ResourceInfo;
			OutTotalTrackedResources = GRHITrackedResources.Num();

			for (const FRHIResource* Resource : GRHITrackedResources)
			{
				if (Resource && Resource->GetResourceInfo(ResourceInfo))
				{
					ResourceInfo.bValid = Resource->IsValid();

					if (ShouldIncludeResource(Resource, ResourceInfo))
					{
						OutResources.Emplace(FResourceEntry{ Resource, ResourceInfo });
					}

					OutTotalResourcesWithInfo++;
					if (ResourceInfo.IsTransient)
					{
						OutTotalTrackedTransientResourceSize += ResourceInfo.VRamAllocation.AllocationSize;
					}
					else
					{
						OutTotalTrackedResourceSize += ResourceInfo.VRamAllocation.AllocationSize;
					}
				}
			}
		}

		if (OutNumberOfResourcesToShow < 0 || OutNumberOfResourcesToShow > OutResources.Num())
		{
			OutNumberOfResourcesToShow = OutResources.Num();
		}

		OutResources.Sort([](const FResourceEntry& EntryA, const FResourceEntry& EntryB)
		{
			return EntryA.ResourceInfo.VRamAllocation.AllocationSize > EntryB.ResourceInfo.VRamAllocation.AllocationSize;
		});
	}

	FResourceFlags GetResourceFlagsInternal(const FResourceEntry& Resource)
	{
		FResourceFlags Flags;
		Flags.bResident = Resource.ResourceInfo.bResident;
		Flags.bMarkedForDelete = !Resource.ResourceInfo.bValid;
		Flags.bTransient = Resource.ResourceInfo.IsTransient;

		if (Resource.ResourceInfo.Type == RRT_Texture)
		{
			FRHITexture* Texture = (FRHITexture*)Resource.Resource;
			Flags.bRT = EnumHasAnyFlags(Texture->GetFlags(), TexCreate_RenderTargetable);
			Flags.bDS = EnumHasAnyFlags(Texture->GetFlags(), TexCreate_DepthStencilTargetable);
			Flags.bUAV = EnumHasAnyFlags(Texture->GetFlags(), TexCreate_UAV);
			Flags.bStreaming = EnumHasAnyFlags(Texture->GetFlags(), TexCreate_Streamable);
			Flags.bReserved = EnumHasAnyFlags(Texture->GetFlags(), TexCreate_ReservedResource);
		}
		else if (Resource.ResourceInfo.Type == RRT_Buffer)
		{
			FRHIBuffer* Buffer = (FRHIBuffer*)Resource.Resource;
			Flags.bUAV = EnumHasAnyFlags((EBufferUsageFlags)Buffer->GetUsage(), BUF_UnorderedAccess);
			Flags.bRTAS = EnumHasAnyFlags((EBufferUsageFlags)Buffer->GetUsage(), BUF_AccelerationStructure);
			Flags.bReserved = EnumHasAnyFlags((EBufferUsageFlags)Buffer->GetUsage(), BUF_ReservedResource);
		}

		Flags.bHasFlags = Flags.bResident || Flags.bMarkedForDelete || Flags.bTransient || Flags.bStreaming || Flags.bRT || Flags.bDS || Flags.bUAV || Flags.bRTAS || Flags.bReserved;
		return Flags;
	}
}

void RHIGetTrackedResourceStats(TArray<TSharedPtr<FRHIResourceStats>>& OutResourceStats)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RHIGetTrackedResourceStats);
	FScopeLock Lock(&GRHIResourceTrackingCriticalSection);

	TArray<RHIInternal::FResourceEntry> Resources;
	int32 TotalResourcesWithInfo = 0;
	int32 TotalTrackedResources = 0;
	int64 TotalTrackedResourceSize = 0;
	int64 TotalTrackedTransientResourceSize = 0;
	int32 NumberOfResourcesToShow = -1;

	RHIInternal::GetTrackedResourcesInternal(TEXT(""), ERHIResourceType::RRT_None, EBooleanFilter::All, Resources, NumberOfResourcesToShow, TotalResourcesWithInfo, TotalTrackedResources, TotalTrackedResourceSize, TotalTrackedTransientResourceSize);

	OutResourceStats.SetNum(Resources.Num());
	ParallelFor(Resources.Num(), [&](int32 Index)
	{
		const FRHIResource* Resource = Resources[Index].Resource;
		const FRHIResourceInfo& ResourceInfo = Resources[Index].ResourceInfo;
		const TCHAR* ResourceType = StringFromRHIResourceType(ResourceInfo.Type);
		const int64 SizeInBytes = ResourceInfo.VRamAllocation.AllocationSize;
		RHIInternal::FResourceFlags Flags = GetResourceFlagsInternal(Resources[Index]);
		OutResourceStats[Index] = MakeShared<FRHIResourceStats>(ResourceInfo.Name, Resource->GetOwnerName(), ResourceType, Flags.GetString(), SizeInBytes,
									Flags.bResident, Flags.bMarkedForDelete, Flags.bTransient, Flags.bStreaming, Flags.bRT, Flags.bDS, Flags.bUAV, Flags.bRTAS, Flags.bHasFlags);
	});
}

void RHIDumpResourceMemory(const FString& NameFilter, ERHIResourceType TypeFilter, EBooleanFilter TransientFilter, int32 NumberOfResourcesToShow, bool bUseCSVOutput, bool bSummaryOutput, bool bOutputToCSVFile, FBufferedOutputDevice& BufferedOutput)
{	
	FArchive* CSVFile{ nullptr };
	if (bOutputToCSVFile)
	{
		const FString Filename = FString::Printf(TEXT("%srhiDumpResourceMemory-%s.csv"), *FPaths::ProfilingDir(), *FDateTime::Now().ToString());
		CSVFile = IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_AllowRead);
	}

	FScopeLock Lock(&GRHIResourceTrackingCriticalSection);

	TArray<RHIInternal::FResourceEntry> Resources;
	int32 TotalResourcesWithInfo = 0;
	int32 TotalTrackedResources = 0;
	int64 TotalTrackedResourceSize = 0;
	int64 TotalTrackedTransientResourceSize = 0;

	RHIInternal::GetTrackedResourcesInternal(NameFilter, TypeFilter, TransientFilter, Resources, NumberOfResourcesToShow, TotalResourcesWithInfo, TotalTrackedResources, TotalTrackedResourceSize, TotalTrackedTransientResourceSize);
	const int32 NumberOfResourcesBeforeNumberFilter = Resources.Num();

	FName CategoryName("RHIResources");

	if (bOutputToCSVFile)
	{
		const TCHAR* Header = TEXT("Name,Type,Size,Resident,MarkedForDelete,Transient,Streaming,RenderTarget,UAV,\"Raytracing Acceleration Structure\",Owner\n");
		CSVFile->Serialize(TCHAR_TO_ANSI(Header), FPlatformString::Strlen(Header));
	}
	else if (bUseCSVOutput)
	{
		BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Name,Type,Size,Resident,MarkedForDelete,Transient,Streaming,RenderTarget,UAV,\"Raytracing Acceleration Structure\",Owner"));
	}
	else
	{
		if (bSummaryOutput == false)
		{
			BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Tracked RHIResources (%d total with info, %d total tracked)"), TotalResourcesWithInfo, TotalTrackedResources);
		}

		if (NumberOfResourcesToShow != NumberOfResourcesBeforeNumberFilter)
		{
			BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Showing %d of %d matched resources"), NumberOfResourcesToShow, NumberOfResourcesBeforeNumberFilter);
		}
	}

	TCHAR ResourceNameBuffer[FName::StringBufferSize];
	TCHAR ResourceOwnerBuffer[FName::StringBufferSize];
	int64 TotalShownResourceSize = 0;

	for (int32 Index = 0; Index < Resources.Num(); Index++)
	{
		if (Index < NumberOfResourcesToShow)
		{
			const FRHIResourceInfo& ResourceInfo = Resources[Index].ResourceInfo;

			ResourceInfo.Name.ToString(ResourceNameBuffer);
			const TCHAR* ResourceType = StringFromRHIResourceType(ResourceInfo.Type);
			const int64 SizeInBytes = ResourceInfo.VRamAllocation.AllocationSize;			
			Resources[Index].Resource->GetOwnerName().ToString(ResourceOwnerBuffer);

			RHIInternal::FResourceFlags Flags = GetResourceFlagsInternal(Resources[Index]);

			if (bSummaryOutput == false)
			{
				if (bOutputToCSVFile || bUseCSVOutput)
				{		
					const FString Row = FString::Printf(TEXT("%s,%s,%.9f,%s,%s,%s,%s,%s,%s,%s,%s\n"),
						ResourceNameBuffer,
						ResourceType,
						SizeInBytes / double(1 << 20),
						Flags.bResident ? TEXT("Yes") : TEXT("No"),
						Flags.bMarkedForDelete ? TEXT("Yes") : TEXT("No"),
						Flags.bTransient ? TEXT("Yes") : TEXT("No"),
						Flags.bStreaming ? TEXT("Yes") : TEXT("No"),
						(Flags.bRT || Flags.bDS) ? TEXT("Yes") : TEXT("No"),
						Flags.bUAV ? TEXT("Yes") : TEXT("No"),
						Flags.bRTAS ? TEXT("Yes") : TEXT("No"),
						ResourceOwnerBuffer);

					if (bOutputToCSVFile)
					{
						CSVFile->Serialize(TCHAR_TO_ANSI(*Row), Row.Len());
					}
					else
					{
						BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("%s"), *Row);
					}
				}
				else
				{
					FString ResoureFlags = Flags.GetString();
					BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Name: %s - Type: %s - Size: %.9f MB - Flags: %s - Owner: %s"),
						ResourceNameBuffer,
						ResourceType,
						SizeInBytes / double(1 << 20),
						ResoureFlags.IsEmpty() ? TEXT("None") : *ResoureFlags,
						ResourceOwnerBuffer);
				}
			}

			TotalShownResourceSize += SizeInBytes;
		}
	}

	if (bOutputToCSVFile)
	{
		delete CSVFile;
		CSVFile = nullptr;
	}
	else if (!bUseCSVOutput)
	{
		const double TotalNonTransientSizeF = TotalTrackedResourceSize / double(1 << 20);
		const double TotalTransientSizeF = TotalTrackedTransientResourceSize / double(1 << 20);
		const double TotalSizeF = (TotalTrackedResourceSize + TotalTrackedTransientResourceSize) / double(1 << 20);
		const double ShownSizeF = TotalShownResourceSize / double(1 << 20);

		if (NumberOfResourcesToShow != TotalResourcesWithInfo)
		{
			double TotalSizeToUse = 0.0;
			const TCHAR* ExtraText = TEXT("");

			if (TransientFilter == EBooleanFilter::No)
			{
				TotalSizeToUse = TotalNonTransientSizeF;
				ExtraText = TEXT(" non-transient");
			}
			else if (TransientFilter == EBooleanFilter::Yes)
			{
				TotalSizeToUse = TotalTransientSizeF;
				ExtraText = TEXT(" transient");
			}
			else
			{
				TotalSizeToUse = TotalSizeF;
				ExtraText = TEXT("");
			}

			if (TotalSizeToUse == 0.0)
			{
				TotalSizeToUse = TotalSizeF;
			}

			BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Shown %d entries%s. Size: %.2f/%.2f MB (%.2f%% of total%s)"),
				NumberOfResourcesToShow, !NameFilter.IsEmpty() ? *FString::Printf(TEXT(" with name %s"), *NameFilter) : TEXT(""), ShownSizeF, TotalSizeToUse, 100.0 * ShownSizeF / TotalSizeToUse, ExtraText);
		}

		if (bSummaryOutput == false)
		{
			BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Total tracked resource size: %.9f MB"), TotalSizeF);
			if (TotalTrackedTransientResourceSize > 0)
			{
				BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("    Non-Transient: %.9f MB"), TotalNonTransientSizeF);
				BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("    Transient: %.9f MB"), TotalTransientSizeF);
			}
		}
	}
}

static FAutoConsoleCommandWithWorldArgsAndOutputDevice GDumpRHIResourceMemoryCmd(
	TEXT("rhi.DumpResourceMemory"),
	TEXT("Dumps RHI resource memory stats to the log\n")
	TEXT("Usage: rhi.DumpResourceMemory [<Number To Show>] [all] [summary] [Name=<Filter Text>] [Type=<RHI Resource Type>] [Transient=<no, yes, or all> [csv]"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, UWorld*, FOutputDevice& OutputDevice)
{
	FString NameFilter;
	ERHIResourceType TypeFilter = RRT_None;
	EBooleanFilter TransientFilter = EBooleanFilter::No;
	int32 NumberOfResourcesToShow = 50;
	bool bUseCSVOutput = false;
	bool bSummaryOutput = false;
	bool bOutputToCSVFile = false;

	for (const FString& Argument : Args)
	{
		if (Argument.Equals(TEXT("all"), ESearchCase::IgnoreCase))
		{
			NumberOfResourcesToShow = -1;
		}
		else if (Argument.Equals(TEXT("-csv"), ESearchCase::IgnoreCase))
		{
			bUseCSVOutput = true;
		}
		else if (Argument.Equals(TEXT("-csvfile"), ESearchCase::IgnoreCase))
		{
			bOutputToCSVFile = true;
		}
		else if (Argument.StartsWith(TEXT("Name="), ESearchCase::IgnoreCase))
		{
			NameFilter = Argument.RightChop(5);
		}
		else if (Argument.StartsWith(TEXT("Type="), ESearchCase::IgnoreCase))
		{
			TypeFilter = RHIResourceTypeFromString(Argument.RightChop(5));
		}
		else if (Argument.StartsWith(TEXT("Transient="), ESearchCase::IgnoreCase))
		{
			TransientFilter = ParseBooleanFilter(Argument.RightChop(10));
		}
		else if (FCString::IsNumeric(*Argument))
		{
			LexFromString(NumberOfResourcesToShow, *Argument);
		}
		else if (Argument.Equals(TEXT("summary"), ESearchCase::IgnoreCase))
		{
			// Respects name, type and transient filters but only reports total sizes.
			// Does not report a list of individual resources.
			bSummaryOutput = true;
			NumberOfResourcesToShow = -1;
		}
		else
		{
			NameFilter = Argument;
		}
	}

	FBufferedOutputDevice BufferedOutput;
	RHIDumpResourceMemory(NameFilter, TypeFilter, TransientFilter, NumberOfResourcesToShow, bUseCSVOutput, bSummaryOutput, bOutputToCSVFile, BufferedOutput);
	BufferedOutput.RedirectTo(OutputDevice);
}));

void RHIDumpResourceMemoryToCSV()
{
	FString NameFilter;
	ERHIResourceType TypeFilter = RRT_None;
	EBooleanFilter TransientFilter = EBooleanFilter::No;
	int32 NumberOfResourcesToShow = -1;
	bool bUseCSVOutput = false;
	bool bSummaryOutput = false;
	bool bOutputToCSVFile = true;

	FBufferedOutputDevice BufferedOutput;
	RHIDumpResourceMemory(NameFilter, TypeFilter, TransientFilter, NumberOfResourcesToShow, bUseCSVOutput, bSummaryOutput, bOutputToCSVFile, BufferedOutput);
}

#endif // RHI_ENABLE_RESOURCE_INFO

/**
 * RHI configuration settings.
 */
static TAutoConsoleVariable<int32> GCVarRHIRenderPass(
	TEXT("r.RHIRenderPasses"),
	0,
	TEXT(""),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarGPUCrashOnOutOfMemory(
	TEXT("r.GPUCrashOnOutOfMemory"),
	0,
	TEXT("Enable crash reporting on GPU OOM"),
	ECVF_ReadOnly
);

TOptional<ERHIBindlessConfiguration> RHIGetForcedBindlessConfiguration()
{
#if WITH_EDITOR
	static const bool bCachedOff = FParse::Param(FCommandLine::Get(), TEXT("BindlessOff"));
	if (bCachedOff)
	{
		return ERHIBindlessConfiguration::Disabled;
	}

	static const bool bCachedRayTracing = FParse::Param(FCommandLine::Get(), TEXT("BindlessRT"));
	if (bCachedRayTracing)
	{
		return ERHIBindlessConfiguration::RayTracing;
	}

	static const bool bCachedMinimal = FParse::Param(FCommandLine::Get(), TEXT("BindlessMinimal"));
	if (bCachedMinimal)
	{
		return ERHIBindlessConfiguration::Minimal;
	}

	static const bool bCachedAll = FParse::Param(FCommandLine::Get(), TEXT("BindlessAll"));
	if (bCachedAll)
	{
		return ERHIBindlessConfiguration::All;
	}

	// As of 5.7, -Bindless only means "Minimal". This may change in the future to be All.
	static const bool bCachedBindless = FParse::Param(FCommandLine::Get(), TEXT("Bindless"));
	if (bCachedBindless)
	{
		return ERHIBindlessConfiguration::Minimal;
	}
#endif

	return TOptional<ERHIBindlessConfiguration>();
}

// By default, read only states and UAV states are allowed to participate in state merging.
ERHIAccess GRHIMergeableAccessMask = ERHIAccess::ReadOnlyMask | ERHIAccess::UAVMask;

// By default, only exclusively read only accesses are allowed.
ERHIAccess GRHIMultiPipelineMergeableAccessMask = ERHIAccess::ReadOnlyExclusiveMask;

ERHIAccess GRHIMultiSubresourceDiscardIntermediateAccess = ERHIAccess::SRVCompute;

//
// The current shader platform.
//

EShaderPlatform GMaxRHIShaderPlatform = SP_PCD3D_SM5;

/** The maximum feature level supported on this machine */
ERHIFeatureLevel::Type GMaxRHIFeatureLevel = ERHIFeatureLevel::SM5;

bool IsRHIDeviceAMD()
{
	check(GRHIVendorId != 0);
	return GRHIVendorId == 0x1002;
}

bool IsRHIDeviceIntel()
{
	check(GRHIVendorId != 0);
	return GRHIVendorId == 0x8086;
}

bool IsRHIDeviceNVIDIA()
{
	check(GRHIVendorId != 0);
	return GRHIVendorId == 0x10DE;
}

bool IsRHIDeviceQualcomm()
{
	check(GRHIVendorId != 0);
	// On Windows, the DXGI vendor ID is reported as 0x4D4F4351.
	// On other platforms, it's 0x5143.
	return GRHIVendorId == 0x5143 || GRHIVendorId == 0x4D4F4351;
}

bool IsRHIDeviceApple()
{
    check(GRHIVendorId != 0);
    return GRHIVendorId == (uint32) EGpuVendorId::Apple;
}

EGpuVendorId GetRHIDeviceVendorId()
{
	if (IsRHIDeviceAMD())
	{
		return EGpuVendorId::Amd;
	}
	if (IsRHIDeviceIntel())
	{
		return EGpuVendorId::Intel;
	}
	if (IsRHIDeviceNVIDIA())
	{
		return EGpuVendorId::Nvidia;
	}
	if (IsRHIDeviceQualcomm())
	{
		return EGpuVendorId::Qualcomm;
	}
	if (IsRHIDeviceApple())
	{
		return EGpuVendorId::Apple;
	}

	return EGpuVendorId::Unknown;
}

uint32 RHIGetMetalShaderLanguageVersion(const FStaticShaderPlatform Platform)
{
    if (IsMetalPlatform(Platform))
	{
		if (IsPCPlatform(Platform))
		{
            static int32 MacMetalShaderLanguageVersion = -1;
            if (MacMetalShaderLanguageVersion == -1)
            {
                if (!GConfig->GetInt(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("MetalLanguageVersion"), MacMetalShaderLanguageVersion, GEngineIni))
                {
                    MacMetalShaderLanguageVersion = 0; // 0 means default EMacMetalShaderStandard::MacMetalSLStandard_Minimum
                }
            }
            return MacMetalShaderLanguageVersion;
		}
		else
		{
            static int32 IOSMetalShaderLanguageVersion = -1;
            if (IOSMetalShaderLanguageVersion == -1)
            {
                if (!GConfig->GetInt(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("MetalLanguageVersion"), IOSMetalShaderLanguageVersion, GEngineIni))
                {
                	IOSMetalShaderLanguageVersion = 0;  // 0 means default EIOSMetalShaderStandard::IOSMetalSLStandard_Minimum
                }
            }
            return IOSMetalShaderLanguageVersion;
        }
	}
	return 0;
}

static ERHIFeatureLevel::Type GRHIMobilePreviewFeatureLevel = ERHIFeatureLevel::Num;

void RHISetMobilePreviewFeatureLevel(ERHIFeatureLevel::Type MobilePreviewFeatureLevel)
{
	check(GRHIMobilePreviewFeatureLevel == ERHIFeatureLevel::Num);
	check(!GIsEditor);
	GRHIMobilePreviewFeatureLevel = MobilePreviewFeatureLevel;
}

bool RHIGetPreviewFeatureLevel(ERHIFeatureLevel::Type& PreviewFeatureLevelOUT)
{
	static bool bForceFeatureLevelES3_1 = !GIsEditor && (FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES31")) || FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES3_1")));

	if (bForceFeatureLevelES3_1)
	{
		PreviewFeatureLevelOUT = ERHIFeatureLevel::ES3_1;
	}
	else if (!GIsEditor && GRHIMobilePreviewFeatureLevel != ERHIFeatureLevel::Num)
	{
		PreviewFeatureLevelOUT = GRHIMobilePreviewFeatureLevel;
	}
	else
	{
		return false;
	}
	return true;
}

int32 RHIGetPreferredClearUAVRectPSResourceType(const FStaticShaderPlatform Platform)
{
	// We can't bind Nanite buffers as RWBuffer to perform a clear op.
	if (IsMetalPlatform(Platform) && !FDataDrivenShaderPlatformInfo::GetSupportsNanite(Platform))
	{
		static constexpr uint32 METAL_TEXTUREBUFFER_SHADER_LANGUAGE_VERSION = 4;
		if (METAL_TEXTUREBUFFER_SHADER_LANGUAGE_VERSION <= RHIGetMetalShaderLanguageVersion(Platform))
		{
			return 0; // BUFFER
		}
	}
	return 1; // TEXTURE_2D
}

RHI_API bool RHISupportsVolumeTextureRendering(const FStaticShaderPlatform Platform)
{
#if WITH_EDITOR
	// When preview platforms are supported (when building with editor) we might be previewing an RHI that doesn't support geometry shaders (such as Metal)
	// while rendering with a runtime RHI that doesn't support vertex shader layers as an alternative (such as D3D), so take these DDPI entries into account, too.
	return GSupportsVolumeTextureRendering && (RHISupportsGeometryShaders(Platform) || RHISupportsVertexShaderLayer(Platform));
#else
	return GSupportsVolumeTextureRendering;
#endif
}

void FRHIRenderPassInfo::ConvertToRenderTargetsInfo(FRHISetRenderTargetsInfo& OutRTInfo) const
{
	for (int32 Index = 0; Index < MaxSimultaneousRenderTargets; ++Index)
	{
		if (!ColorRenderTargets[Index].RenderTarget)
		{
			break;
		}

		OutRTInfo.ColorRenderTarget[Index].Texture = ColorRenderTargets[Index].RenderTarget;
		ERenderTargetLoadAction LoadAction = GetLoadAction(ColorRenderTargets[Index].Action);
		OutRTInfo.ColorRenderTarget[Index].LoadAction = LoadAction;
		OutRTInfo.ColorRenderTarget[Index].StoreAction = GetStoreAction(ColorRenderTargets[Index].Action);
		OutRTInfo.ColorRenderTarget[Index].ArraySliceIndex = ColorRenderTargets[Index].ArraySlice;
		OutRTInfo.ColorRenderTarget[Index].MipIndex = ColorRenderTargets[Index].MipIndex;
		++OutRTInfo.NumColorRenderTargets;

		OutRTInfo.bClearColor |= (LoadAction == ERenderTargetLoadAction::EClear);

		if (ColorRenderTargets[Index].ResolveTarget)
		{
			OutRTInfo.bHasResolveAttachments = true;
			OutRTInfo.ColorResolveRenderTarget[Index] = OutRTInfo.ColorRenderTarget[Index];
			OutRTInfo.ColorResolveRenderTarget[Index].Texture = ColorRenderTargets[Index].ResolveTarget;
		}
	}

	ERenderTargetActions DepthActions = GetDepthActions(DepthStencilRenderTarget.Action);
	ERenderTargetActions StencilActions = GetStencilActions(DepthStencilRenderTarget.Action);
	ERenderTargetLoadAction DepthLoadAction = GetLoadAction(DepthActions);
	ERenderTargetStoreAction DepthStoreAction = GetStoreAction(DepthActions);
	ERenderTargetLoadAction StencilLoadAction = GetLoadAction(StencilActions);
	ERenderTargetStoreAction StencilStoreAction = GetStoreAction(StencilActions);

	OutRTInfo.DepthStencilRenderTarget = FRHIDepthRenderTargetView(DepthStencilRenderTarget.DepthStencilTarget,
		DepthLoadAction,
		GetStoreAction(DepthActions),
		StencilLoadAction,
		GetStoreAction(StencilActions),
		DepthStencilRenderTarget.ExclusiveDepthStencil);
	OutRTInfo.bClearDepth = (DepthLoadAction == ERenderTargetLoadAction::EClear);
	OutRTInfo.bClearStencil = (StencilLoadAction == ERenderTargetLoadAction::EClear);

	if (DepthStencilRenderTarget.ResolveTarget && DepthStencilRenderTarget.ResolveTarget != DepthStencilRenderTarget.DepthStencilTarget)
	{
		OutRTInfo.DepthStencilResolveRenderTarget = OutRTInfo.DepthStencilRenderTarget;
		OutRTInfo.DepthStencilResolveRenderTarget.Texture = DepthStencilRenderTarget.ResolveTarget;
	}

	OutRTInfo.ShadingRateTexture = ShadingRateTexture;
	OutRTInfo.ShadingRateTextureCombiner = ShadingRateTextureCombiner;
	OutRTInfo.MultiViewCount = MultiViewCount;
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void FRHIRenderPassInfo::Validate() const
{
	int32 NumSamples = -1;	// -1 means nothing found yet
	int32 ColorIndex = 0;
	for (; ColorIndex < MaxSimultaneousRenderTargets; ++ColorIndex)
	{
		const FColorEntry& Entry = ColorRenderTargets[ColorIndex];
		if (Entry.RenderTarget)
		{
			// Ensure NumSamples matches amongst all color RTs
			if (NumSamples == -1)
			{
				NumSamples = Entry.RenderTarget->GetNumSamples();
			}
			else
			{
				// CustomResolveSubpass can have targets with a different NumSamples
				ensureMsgf(Entry.RenderTarget->GetNumSamples() == NumSamples || SubpassHint == ESubpassHint::CustomResolveSubpass, TEXT("RenderTarget have inconsistent NumSamples: first %d, then %d"), NumSamples, Entry.RenderTarget->GetNumSamples());
			}

			ERenderTargetStoreAction Store = GetStoreAction(Entry.Action);
			// Don't try to resolve a non-msaa
			ensure(Store != ERenderTargetStoreAction::EMultisampleResolve || Entry.RenderTarget->GetNumSamples() > 1);
			// Don't resolve to null
			ensure(Store != ERenderTargetStoreAction::EMultisampleResolve || Entry.ResolveTarget);

			if (Entry.ResolveTarget)
			{
				//ensure(Store == ERenderTargetStoreAction::EMultisampleResolve);
			}
		}
		else
		{
			break;
		}
	}

	int32 NumColorRenderTargets = ColorIndex;
	for (; ColorIndex < MaxSimultaneousRenderTargets; ++ColorIndex)
	{
		// Gap in the sequence of valid render targets (ie RT0, null, RT2, ...)
		ensureMsgf(!ColorRenderTargets[ColorIndex].RenderTarget, TEXT("Missing color render target on slot %d"), ColorIndex - 1);
	}

	if (DepthStencilRenderTarget.DepthStencilTarget)
	{
		// Ensure NumSamples matches with color RT
		if (NumSamples != -1)
		{
			ensure(DepthStencilRenderTarget.DepthStencilTarget->GetNumSamples() == NumSamples);
		}
		ERenderTargetStoreAction DepthStore = GetStoreAction(GetDepthActions(DepthStencilRenderTarget.Action));
		ERenderTargetStoreAction StencilStore = GetStoreAction(GetStencilActions(DepthStencilRenderTarget.Action));
		bool bIsMSAAResolve = (DepthStore == ERenderTargetStoreAction::EMultisampleResolve) || (StencilStore == ERenderTargetStoreAction::EMultisampleResolve);
		// Don't try to resolve a non-msaa
		ensure(!bIsMSAAResolve || DepthStencilRenderTarget.DepthStencilTarget->GetNumSamples() > 1);
		// Don't resolve to null
		//ensure(DepthStencilRenderTarget.ResolveTarget || DepthStore != ERenderTargetStoreAction::EStore);

		// Don't write to depth if read-only
		//ensure(DepthStencilRenderTarget.ExclusiveDepthStencil.IsDepthWrite() || DepthStore != ERenderTargetStoreAction::EStore);
		// This is not true for stencil. VK and Metal specify that the DontCare store action MAY leave the attachment in an undefined state.
		/*ensure(DepthStencilRenderTarget.ExclusiveDepthStencil.IsStencilWrite() || StencilStore != ERenderTargetStoreAction::EStore);*/

		// If we have a depthstencil target we MUST Store it or it will be undefined after rendering.
		if (DepthStencilRenderTarget.DepthStencilTarget->GetFormat() != PF_D24)
		{
			// If this is DepthStencil we must store it out unless we are absolutely sure it will never be used again.
			// it is valid to use a depthbuffer for performance and not need the results later.
			//ensure(StencilStore == ERenderTargetStoreAction::EStore);
		}

		if (DepthStencilRenderTarget.ExclusiveDepthStencil.IsDepthWrite())
		{
			// this check is incorrect for mobile, depth/stencil is intermediate and we don't want to store it to main memory
			//ensure(DepthStore == ERenderTargetStoreAction::EStore);
		}

		if (DepthStencilRenderTarget.ExclusiveDepthStencil.IsStencilWrite())
		{
			// this check is incorrect for mobile, depth/stencil is intermediate and we don't want to store it to main memory
			//ensure(StencilStore == ERenderTargetStoreAction::EStore);
		}
		
		if (SubpassHint == ESubpassHint::DepthReadSubpass || SubpassHint == ESubpassHint::CustomResolveSubpass)
		{
			// for depth read sub-pass
			// 1. render pass must have depth target
			// 2. depth target must support InputAttachement
			ensure(EnumHasAnyFlags(DepthStencilRenderTarget.DepthStencilTarget->GetFlags(), TexCreate_InputAttachmentRead));
		}

		if (DepthStencilRenderTarget.ResolveTarget && DepthStencilRenderTarget.ResolveTarget != DepthStencilRenderTarget.DepthStencilTarget)
		{
			// For depth resolve
			// 1. RHI must support depth stencil resolve
			// 2. Must be using MSAA resolve
			// 3. Resolve target sample count must be 1
			// 4. Resolve target format must be the same as the MSAA target format
			ensureMsgf(GRHISupportsDepthStencilResolve, TEXT("Attempted to resolve depth/stencil target but feature is not supported."));
			ensureMsgf(bIsMSAAResolve, TEXT("Depth/stencil resolve target is bound but resolve was not requested."));
			ensureMsgf(DepthStencilRenderTarget.ResolveTarget->GetNumSamples() == 1, TEXT("Depth/stencil resolve targets must have a sample count of 1."));
			ensureMsgf(DepthStencilRenderTarget.ResolveTarget->GetFormat() == DepthStencilRenderTarget.DepthStencilTarget->GetFormat(),
				TEXT("Depth/stencil resolve targets must have the same format as the MSAA target."));
		}
	}
	else
	{
		ensure(DepthStencilRenderTarget.Action == EDepthStencilTargetActions::DontLoad_DontStore);
		ensure(DepthStencilRenderTarget.ExclusiveDepthStencil == FExclusiveDepthStencil::DepthNop_StencilNop);
		ensure(SubpassHint != ESubpassHint::DepthReadSubpass);
		ensure(SubpassHint != ESubpassHint::CustomResolveSubpass);
	}
}
#endif

#define ValidateResourceDesc(expr, format, ...) \
	if (bFatal) \
	{ \
		checkf(expr, format, ##__VA_ARGS__); \
	} \
	else if (!(expr)) \
	{ \
		return false;\
	} \

// static
bool FRHITextureDesc::Validate(const FRHITextureCreateInfo& Desc, const TCHAR* Name, bool bFatal)
{
	// Validate texture's pixel format.
	{
		ValidateResourceDesc(Desc.Format != PF_Unknown, TEXT("Illegal to create texture %s with an invalid pixel format."), Name);
		ValidateResourceDesc(Desc.Format < PF_MAX, TEXT("Illegal to create texture %s with an invalid pixel format."), Name);
		ValidateResourceDesc(GPixelFormats[Desc.Format].Supported,
			TEXT("Failed to create texture %s with pixel format %s because it is not supported."),
			Name,
			GPixelFormats[Desc.Format].Name);
	}

	// Validate texture's extent.
	{
		int32 MaxDimension = (Desc.Dimension == ETextureDimension::TextureCube || Desc.Dimension == ETextureDimension::TextureCubeArray) ? GMaxCubeTextureDimensions : GMaxTextureDimensions;

		ValidateResourceDesc(Desc.Extent.X > 0, TEXT("Texture %s's Extent.X=%d is invalid."), Name, Desc.Extent.X);
		ValidateResourceDesc(Desc.Extent.X <= MaxDimension, TEXT("Texture %s's Extent.X=%d is too large."), Name, Desc.Extent.X);

		ValidateResourceDesc(Desc.Extent.Y > 0, TEXT("Texture %s's Extent.Y=%d is invalid."), Name, Desc.Extent.Y);
		ValidateResourceDesc(Desc.Extent.Y <= MaxDimension, TEXT("Texture %s's Extent.Y=%d is too large."), Name, Desc.Extent.Y);
	}

	// Validate texture's depth
	if (Desc.Dimension == ETextureDimension::Texture3D)
	{
		ValidateResourceDesc(Desc.Depth > 0, TEXT("Texture %s's Depth=%d is invalid."), Name, int32(Desc.Depth));
		ValidateResourceDesc(Desc.Depth <= GMaxTextureDimensions, TEXT("Texture %s's Extent.Depth=%d is too large."), Name, Desc.Depth);
	}
	else
	{
		ValidateResourceDesc(Desc.Depth == 1, TEXT("Texture %s's Depth=%d is invalid for Dimension=%s."), Name, int32(Desc.Depth), GetTextureDimensionString(Desc.Dimension));
	}

	// Validate texture's array size
	if (Desc.Dimension == ETextureDimension::Texture2DArray || Desc.Dimension == ETextureDimension::TextureCubeArray)
	{
		ValidateResourceDesc(Desc.ArraySize > 0, TEXT("Texture %s's ArraySize=%d is invalid."), Name, Desc.ArraySize);
		ValidateResourceDesc(Desc.ArraySize <= GMaxTextureArrayLayers, TEXT("Texture %s's Extent.ArraySize=%d is too large."), Name, int32(Desc.ArraySize));
	}
	else
	{
		ValidateResourceDesc(Desc.ArraySize == 1, TEXT("Texture %s's ArraySize=%d is invalid for Dimension=%s."), Name, Desc.ArraySize,  GetTextureDimensionString(Desc.Dimension));
	}

	// Validate texture's samples count.
	if (Desc.Dimension == ETextureDimension::Texture2D || Desc.Dimension == ETextureDimension::Texture2DArray)
	{
		ValidateResourceDesc(Desc.NumSamples > 0, TEXT("Texture %s's NumSamples=%d is invalid."), Name, Desc.NumSamples);
	}
	else
	{
		ValidateResourceDesc(Desc.NumSamples == 1, TEXT("Texture %s's NumSamples=%d is invalid for Dimension=%s."), Name, Desc.NumSamples, GetTextureDimensionString(Desc.Dimension));
	}

	// Validate texture's mips.
	if (Desc.IsMultisample())
	{
		ValidateResourceDesc(Desc.NumMips == 1, TEXT("MSAA Texture %s's can only have one mip."), Name);
	}
	else
	{
		ValidateResourceDesc(Desc.NumMips > 0, TEXT("Texture %s's NumMips=%d is invalid."), Name, Desc.NumMips);
		ValidateResourceDesc(Desc.NumMips <= GMaxTextureMipCount, TEXT("Texture %s's NumMips=%d is too large."), Name, Desc.NumMips);
	}

	// Validate reserved resource restrictions
	if (EnumHasAnyFlags(Desc.Flags, TexCreate_ReservedResource))
	{
		ValidateResourceDesc(GRHIGlobals.ReservedResources.Supported,
			TEXT("Reserved Texture %s's can't be created because current RHI does not support reserved resources."),
			Name);

		if (Desc.IsTexture3D())
		{
			ValidateResourceDesc(GRHIGlobals.ReservedResources.SupportsVolumeTextures,
				TEXT("Reserved Texture %s's can't be created because current RHI does not support reserved volume textures."),
				Name);
		}
		else
		{
			ValidateResourceDesc(
				Desc.Dimension == ETextureDimension::Texture2D ||
				Desc.Dimension == ETextureDimension::Texture2DArray,
				TEXT("Reserved Texture %s's Desc.Dimension=%s is invalid. Expected Texture2D, Texture2DArray or Texture3D."),
				Name, GetTextureDimensionString(Desc.Dimension));
		}

		if (Desc.Dimension == ETextureDimension::Texture2DArray)
		{
			const uint32 MipShift = FMath::Max<uint32>(1u, Desc.NumMips) - 1;
			const FIntPoint SmallestMipExtent = Desc.Extent / (1 << MipShift);

			ValidateResourceDesc(SmallestMipExtent.X >= GRHIGlobals.ReservedResources.TextureArrayMinimumMipDimension,
				TEXT("Reserved Texture array %s's SmallestMipExtent.X=%d is invalid. It is required to be be no less than %d."),
				Name, SmallestMipExtent.X, GRHIGlobals.ReservedResources.TextureArrayMinimumMipDimension);

			ValidateResourceDesc(SmallestMipExtent.Y >= GRHIGlobals.ReservedResources.TextureArrayMinimumMipDimension,
				TEXT("Reserved Texture array %s's SmallestMipExtent.Y=%d is invalid. It is required to be be no less than %d."),
				Name, SmallestMipExtent.Y, GRHIGlobals.ReservedResources.TextureArrayMinimumMipDimension);
		}
	}

	return true;
}

// static
bool FRHITextureSRVCreateInfo::Validate(const FRHITextureDesc& TextureDesc, const FRHITextureSRVCreateInfo& TextureSRVDesc, const TCHAR* TextureName, bool bFatal)
{
	if (TextureName == nullptr)
	{
		TextureName = TEXT("UnnamedTexture");
	}

	ValidateResourceDesc(TextureDesc.Flags & TexCreate_ShaderResource,
		TEXT("Attempted to create SRV from texture %s which was not created with TexCreate_ShaderResource"),
		TextureName);

	// Validate the pixel format if overridden by the SRV's descriptor.
	if (TextureSRVDesc.Format == PF_X24_G8)
	{
		// PF_X24_G8 is a bit of mess in the RHI, used to read the stencil, but have varying BlockBytes.
		ValidateResourceDesc(TextureDesc.Format == PF_DepthStencil,
			TEXT("PF_X24_G8 is only to read stencil from a PF_DepthStencil texture"));
	}
	else if (TextureSRVDesc.Format != PF_Unknown)
	{
		ValidateResourceDesc(TextureSRVDesc.Format < PF_MAX,
			TEXT("Illegal to create SRV for texture %s with invalid FPooledRenderTargetDesc::Format."),
			TextureName);
		ValidateResourceDesc(GPixelFormats[TextureSRVDesc.Format].Supported,
			TEXT("Failed to create SRV for texture %s with pixel format %s because it is not supported."),
			TextureName, GPixelFormats[TextureSRVDesc.Format].Name);

		EPixelFormat ResourcePixelFormat = TextureDesc.Format;

		ValidateResourceDesc(
			GPixelFormats[TextureSRVDesc.Format].BlockBytes == GPixelFormats[ResourcePixelFormat].BlockBytes &&
			GPixelFormats[TextureSRVDesc.Format].BlockSizeX == GPixelFormats[ResourcePixelFormat].BlockSizeX &&
			GPixelFormats[TextureSRVDesc.Format].BlockSizeY == GPixelFormats[ResourcePixelFormat].BlockSizeY &&
			GPixelFormats[TextureSRVDesc.Format].BlockSizeZ == GPixelFormats[ResourcePixelFormat].BlockSizeZ,
			TEXT("Failed to create SRV for texture %s with pixel format %s because it does not match the byte size of the texture's pixel format %s."),
			TextureName, GPixelFormats[TextureSRVDesc.Format].Name, GPixelFormats[ResourcePixelFormat].Name);
	}

	ValidateResourceDesc((TextureSRVDesc.MipLevel + TextureSRVDesc.NumMipLevels) <= TextureDesc.NumMips,
		TEXT("Failed to create SRV at mips %d-%d: the texture %s has only %d mip levels."),
		TextureSRVDesc.MipLevel, (TextureSRVDesc.MipLevel + TextureSRVDesc.NumMipLevels), TextureName, TextureDesc.NumMips);

	// Validate the array slices
	if (TextureDesc.IsTextureCube() && TextureSRVDesc.DimensionOverride == ETextureDimension::Texture2DArray)
	{
		// Either TextureCube or TextureCubeArray, compute array size appropriately
		int32 CubeArraySize = TextureDesc.Dimension == ETextureDimension::TextureCube ? ECubeFace::CubeFace_MAX : ECubeFace::CubeFace_MAX * TextureDesc.ArraySize;

		ValidateResourceDesc((TextureSRVDesc.FirstArraySlice + TextureSRVDesc.NumArraySlices) <= CubeArraySize,
			TEXT("Failed to create SRV at array slices %d-%d: the cube map texture %s has only %d slices."),
			TextureSRVDesc.FirstArraySlice,
			(TextureSRVDesc.FirstArraySlice + TextureSRVDesc.NumArraySlices),
			TextureName,
			CubeArraySize);
	}
	else if (TextureDesc.IsTextureArray())
	{
		ValidateResourceDesc((TextureSRVDesc.FirstArraySlice + TextureSRVDesc.NumArraySlices) <= TextureDesc.ArraySize,
			TEXT("Failed to create SRV at array slices %d-%d: the texture array %s has only %d slices."),
			TextureSRVDesc.FirstArraySlice,
			(TextureSRVDesc.FirstArraySlice + TextureSRVDesc.NumArraySlices),
			TextureName,
			TextureDesc.ArraySize);
	}
	else
	{
		ValidateResourceDesc(TextureSRVDesc.FirstArraySlice == 0,
			TEXT("Failed to create SRV with FirstArraySlice=%d: the texture %s is not a texture array."),
			TextureSRVDesc.FirstArraySlice, TextureName);
		ValidateResourceDesc(TextureSRVDesc.NumArraySlices == 0,
			TEXT("Failed to create SRV with NumArraySlices=%d: the texture %s is not a texture array."),
			TextureSRVDesc.NumArraySlices, TextureName);
	}

	ValidateResourceDesc(TextureSRVDesc.MetaData != ERHITextureMetaDataAccess::FMask || GRHISupportsExplicitFMask,
		TEXT("Failed to create FMask SRV for texture %s because the current RHI doesn't support it. Be sure to gate the call with GRHISupportsExplicitFMask."),
		TextureName);

	ValidateResourceDesc(TextureSRVDesc.MetaData != ERHITextureMetaDataAccess::HTile || GRHISupportsExplicitHTile,
		TEXT("Failed to create HTile SRV for texture %s because the current RHI doesn't support it. Be sure to gate the call with GRHISupportsExplicitHTile."),
		TextureName);

	return true;
}

#undef ValidateResourceDesc

uint64 FRHITextureDesc::CalcMemorySizeEstimate(uint32 FirstMipIndex, uint32 LastMipIndex) const
{
#if DO_CHECK
	Validate(*this, TEXT("CalcMemorySizeEstimate"), /* bFatal = */true);
#endif
	check(FirstMipIndex < NumMips && FirstMipIndex <= LastMipIndex && LastMipIndex < NumMips);

	uint64 MemorySize = 0;
	for (uint32 MipIndex = FirstMipIndex; MipIndex <= LastMipIndex; ++MipIndex)
	{
		FIntVector MipSizeInBlocks = FIntVector(
			FMath::DivideAndRoundUp(FMath::Max(Extent.X >> MipIndex, 1), GPixelFormats[Format].BlockSizeX),
			FMath::DivideAndRoundUp(FMath::Max(Extent.Y >> MipIndex, 1), GPixelFormats[Format].BlockSizeY),
			FMath::DivideAndRoundUp(FMath::Max(Depth    >> MipIndex, 1), GPixelFormats[Format].BlockSizeZ)
		);

		uint32 NumBlocksInMip = MipSizeInBlocks.X * MipSizeInBlocks.Y * MipSizeInBlocks.Z;
		MemorySize += NumBlocksInMip * GPixelFormats[Format].BlockBytes;
	}

	MemorySize *= ArraySize;
	MemorySize *= NumSamples;

	if (IsTextureCube())
	{
		MemorySize *= 6;
	}

	return MemorySize;
}

static FRHIPanicEvent RHIPanicEvent;
FRHIPanicEvent& RHIGetPanicDelegate()
{
	return RHIPanicEvent;
}



int32 CalculateMSAASampleArrayIndex(int32 NumSamples, int32 SampleIndex)
{
	check(NumSamples > 0 && NumSamples <= 16);
	check(FMath::IsPowerOfTwo(NumSamples));
	check(SampleIndex < NumSamples);

	return NumSamples - 1 + SampleIndex;
}

void RHIInitDefaultPixelFormatCapabilities()
{
	for (FPixelFormatInfo& Info : GPixelFormats)
	{
		if (Info.Supported)
		{
			const EPixelFormat PixelFormat = Info.UnrealFormat;
			if (IsBlockCompressedFormat(PixelFormat))
			{
				// Block compressed formats should have limited capabilities
				EnumAddFlags(Info.Capabilities, EPixelFormatCapabilities::AnyTexture | EPixelFormatCapabilities::TextureMipmaps | EPixelFormatCapabilities::TextureLoad | EPixelFormatCapabilities::TextureSample | EPixelFormatCapabilities::TextureGather | EPixelFormatCapabilities::TextureFilterable);
			}
			else
			{
				EnumAddFlags(Info.Capabilities, EPixelFormatCapabilities::AllTextureFlags | EPixelFormatCapabilities::AllBufferFlags | EPixelFormatCapabilities::UAV);
				if (!IsDepthOrStencilFormat(PixelFormat))
				{
					EnumRemoveFlags(Info.Capabilities, EPixelFormatCapabilities::DepthStencil);
				}
			}
		}
	}
}

//
//	CalculateImageBytes
//

SIZE_T CalculateImageBytes(uint32 SizeX,uint32 SizeY,uint32 SizeZ,uint8 Format)
{
	if ( Format == PF_A1 )
	{		
		// The number of bytes needed to store all 1 bit pixels in a line is the width of the image divided by the number of bits in a byte
		uint32 BytesPerLine = (SizeX + 7) / 8;
		// The number of actual bytes in a 1 bit image is the bytes per line of pixels times the number of lines
		return sizeof(uint8) * BytesPerLine * SizeY;
	}
	else if( SizeZ > 0 )
	{
		return GPixelFormats[Format].Get3DImageSizeInBytes(SizeX, SizeY, SizeZ);
	}
	else
	{
		return GPixelFormats[Format].Get2DImageSizeInBytes(SizeX, SizeY);
	}
}

FRHIShaderResourceView* FRHITextureViewCache::GetOrCreateSRV(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const FRHITextureSRVCreateInfo& SRVCreateInfo)
{
	for (const auto& KeyValue : SRVs)
	{
		if (KeyValue.Key == SRVCreateInfo)
		{
			return KeyValue.Value.GetReference();
		}
	}

    check(Texture);
    ETextureDimension Dimension = Texture->GetDesc().Dimension;
    if(SRVCreateInfo.DimensionOverride.IsSet())
    {
        Dimension = *SRVCreateInfo.DimensionOverride;
    }

	FShaderResourceViewRHIRef RHIShaderResourceView = RHICmdList.CreateShaderResourceView(Texture, FRHIViewDesc::CreateTextureSRV()
		.SetDimension   (Dimension)
		.SetFormat      (SRVCreateInfo.Format)
		.SetMipRange    (SRVCreateInfo.MipLevel, SRVCreateInfo.NumMipLevels)
		.SetDisableSRGB (SRVCreateInfo.SRGBOverride == SRGBO_ForceDisable)
		.SetArrayRange  (SRVCreateInfo.FirstArraySlice, SRVCreateInfo.NumArraySlices)
		.SetPlane       (SRVCreateInfo.MetaData)
	);

	check(RHIShaderResourceView);
	FRHIShaderResourceView* View = RHIShaderResourceView.GetReference();
	SRVs.Emplace(SRVCreateInfo, MoveTemp(RHIShaderResourceView));
	return View;
}

FRHIUnorderedAccessView* FRHITextureViewCache::GetOrCreateUAV(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const FRHITextureUAVCreateInfo& UAVCreateInfo)
{
	for (const auto& KeyValue : UAVs)
	{
		if (KeyValue.Key == UAVCreateInfo)
		{
			return KeyValue.Value.GetReference();
		}
	}

    check(Texture);
    ETextureDimension Dimension = Texture->GetDesc().Dimension;
    if(UAVCreateInfo.DimensionOverride.IsSet())
    {
        Dimension = *UAVCreateInfo.DimensionOverride;
    }
    
	FUnorderedAccessViewRHIRef RHIUnorderedAccessView = RHICmdList.CreateUnorderedAccessView(Texture, FRHIViewDesc::CreateTextureUAV()
		.SetDimension (Dimension)
		.SetFormat    (UAVCreateInfo.Format)
		.SetMipLevel  (UAVCreateInfo.MipLevel)
		.SetArrayRange(UAVCreateInfo.FirstArraySlice, UAVCreateInfo.NumArraySlices)
		.SetPlane     (UAVCreateInfo.MetaData)
	);

	check(RHIUnorderedAccessView);
	FRHIUnorderedAccessView* View = RHIUnorderedAccessView.GetReference();
	UAVs.Emplace(UAVCreateInfo, MoveTemp(RHIUnorderedAccessView));
	return View;
}

FRHIShaderResourceView* FRHIBufferViewCache::GetOrCreateSRV(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, const FRHIBufferSRVCreateInfo& SRVCreateInfo)
{
	// Bypass the cache when using the ray tracing scene, as there is no guarantee the scene will not be recreated, which would grow the cache unbounded.
	if (!SRVCreateInfo.RayTracingScene)
	{
		for (const auto& KeyValue : SRVs)
		{
			if (KeyValue.Key == SRVCreateInfo)
			{
				return KeyValue.Value.GetReference();
			}
		}
	}

	auto CreateDesc = FRHIViewDesc::CreateBufferSRV();

	if (EnumHasAnyFlags(Buffer->GetUsage(), BUF_ByteAddressBuffer))
	{
		CreateDesc.SetType(FRHIViewDesc::EBufferType::Raw);
	}
	else if (EnumHasAnyFlags(Buffer->GetUsage(), BUF_StructuredBuffer))
	{
		CreateDesc.SetType(FRHIViewDesc::EBufferType::Structured);
	}
	else if (EnumHasAnyFlags(Buffer->GetUsage(), BUF_AccelerationStructure))
	{
		CreateDesc.SetType(FRHIViewDesc::EBufferType::AccelerationStructure);
		CreateDesc.SetRayTracingScene(SRVCreateInfo.RayTracingScene);
	}
	else
	{
		CreateDesc.SetType(FRHIViewDesc::EBufferType::Typed);
		CreateDesc.SetFormat(SRVCreateInfo.Format);
	}

	CreateDesc.SetOffsetInBytes(SRVCreateInfo.StartOffsetBytes);

	if (SRVCreateInfo.NumElements != UINT32_MAX)
	{
		CreateDesc.SetNumElements(SRVCreateInfo.NumElements);
	}

	FShaderResourceViewRHIRef RHIShaderResourceView = RHICmdList.CreateShaderResourceView(Buffer, CreateDesc);

	FRHIShaderResourceView* View = RHIShaderResourceView.GetReference();

	if (!SRVCreateInfo.RayTracingScene)
	{
		SRVs.Emplace(SRVCreateInfo, MoveTemp(RHIShaderResourceView));
	}

	return View;
}

FRHIUnorderedAccessView* FRHIBufferViewCache::GetOrCreateUAV(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, const FRHIBufferUAVCreateInfo& UAVCreateInfo)
{
	for (const auto& KeyValue : UAVs)
	{
		if (KeyValue.Key == UAVCreateInfo)
		{
			return KeyValue.Value.GetReference();
		}
	}

	auto CreateDesc = FRHIViewDesc::CreateBufferUAV();
	CreateDesc.SetAtomicCounter(UAVCreateInfo.bSupportsAtomicCounter);
	CreateDesc.SetAppendBuffer(UAVCreateInfo.bSupportsAppendBuffer);

	if (EnumHasAnyFlags(Buffer->GetUsage(), BUF_ByteAddressBuffer))
	{
		CreateDesc.SetType(FRHIViewDesc::EBufferType::Raw);
	}
	else if (EnumHasAnyFlags(Buffer->GetUsage(), BUF_StructuredBuffer))
	{
		CreateDesc.SetType(FRHIViewDesc::EBufferType::Structured);
	}
	else if (EnumHasAnyFlags(Buffer->GetUsage(), BUF_AccelerationStructure))
	{
		CreateDesc.SetType(FRHIViewDesc::EBufferType::AccelerationStructure);
	}
	else
	{
		CreateDesc.SetType(FRHIViewDesc::EBufferType::Typed);
		CreateDesc.SetFormat(UAVCreateInfo.Format);
	}

	FUnorderedAccessViewRHIRef RHIUnorderedAccessView = RHICmdList.CreateUnorderedAccessView(Buffer, CreateDesc);

	FRHIUnorderedAccessView* View = RHIUnorderedAccessView.GetReference();
	UAVs.Emplace(UAVCreateInfo, MoveTemp(RHIUnorderedAccessView));
	return View;
}

#if RHI_USE_RESOURCE_DEBUG_NAME

void FRHITextureViewCache::SetDebugName(FRHICommandListBase& RHICmdList, const TCHAR* DebugName)
{
	for (const auto& KeyValue : UAVs)
	{
		RHICmdList.BindDebugLabelName(KeyValue.Value, DebugName);
	}
}

void FRHIBufferViewCache::SetDebugName(FRHICommandListBase& RHICmdList, const TCHAR* DebugName)
{
	for (const auto& KeyValue : UAVs)
	{
		RHICmdList.BindDebugLabelName(KeyValue.Value, DebugName);
	}
}

#endif

void FRHITransientTexture::BindDebugLabelName(FRHICommandListBase& RHICmdList)
{
	ViewCache.SetDebugName(RHICmdList, GetName());

#if RHI_USE_RESOURCE_DEBUG_NAME
	RHICmdList.BindDebugLabelName(GetRHI(), GetName());
#endif
}

void FRHITransientBuffer::BindDebugLabelName(FRHICommandListBase& RHICmdList)
{
	ViewCache.SetDebugName(RHICmdList, GetName());

	// TODO: Add method to rename a buffer.
}

FDebugName::FDebugName()
	: Name()
	, Number(NAME_NO_NUMBER_INTERNAL)
{
}

FDebugName::FDebugName(FName InName)
	: Name(InName)
	, Number(NAME_NO_NUMBER_INTERNAL)
{
}

FDebugName::FDebugName(FName InName, int32 InNumber)
	: Name(InName)
	, Number(InNumber)
{
}

FDebugName& FDebugName::operator=(FName Other)
{
	Name = Other;
	Number = NAME_NO_NUMBER_INTERNAL;
	return *this;
}

FString FDebugName::ToString() const
{
	FString Out;
	Name.AppendString(Out);
	if (Number != NAME_NO_NUMBER_INTERNAL)
	{
		Out.Appendf(TEXT("_%u"), Number);
	}
	return Out;
}

uint32 FDebugName::ToStringInternal(TCHAR* Out, const uint32 OutSize) const
{
	check(OutSize > 0);
	const uint32 OutLen = OutSize - 1;

	uint32 NameLen = Name.ToStringTruncate(Out, OutSize);
	if (Number != NAME_NO_NUMBER_INTERNAL && OutLen > NameLen)
	{
		int32 SuffixLen = FCString::Snprintf(Out + NameLen, OutSize - NameLen, TEXT("_%d"), Number);
		if (SuffixLen < 0)
		{
			// Suffix was truncated.
			return OutLen;
		}
		else
		{
			// Suffix was fully written
			return NameLen + SuffixLen;
		}
	}
	else
	{
		// No suffix, or no room to add it
		return FMath::Min<uint32>(OutLen, NameLen);
	}
}

void FDebugName::AppendString(FStringBuilderBase& Builder) const
{
	Name.AppendString(Builder);
	if (Number != NAME_NO_NUMBER_INTERNAL)
	{
		Builder << '_' << Number;
	}
}

namespace UE::RHI
{
	static TAutoConsoleVariable<int32> CVarGPUCrashDebugging(
		TEXT("r.GPUCrashDebugging"),
		0,
		TEXT("Enable vendor specific GPU crash analysis tools"),
		ECVF_ReadOnly
	);

	RHI_API bool UseGPUCrashDebugging()
	{
		static const bool bNoGpuCrashDebugging = FParse::Param(FCommandLine::Get(), TEXT("nogpucrashdebugging"));
		static const bool bGpuCrashDebugging   = FParse::Param(FCommandLine::Get(), TEXT("gpucrashdebugging"));

		// Command line takes precedence
		if (bNoGpuCrashDebugging)
		{
			return false;
		}
		else if (bGpuCrashDebugging)
		{
			return true;
		}
		else
		{
			return CVarGPUCrashDebugging.GetValueOnAnyThread() != 0;
		}
	}

	RHI_API bool ShouldEnableGPUCrashFeature(IConsoleVariable& CVar, TCHAR const* CommandLineSwitch)
	{
		static const bool bNoGpuCrashDebugging = FParse::Param(FCommandLine::Get(), TEXT("nogpucrashdebugging"));
		static const bool bGpuCrashDebugging   = FParse::Param(FCommandLine::Get(), TEXT("gpucrashdebugging"));

		bool bEnabled;
		if (bNoGpuCrashDebugging)
		{
			// Command line switch is forcing everything off
			bEnabled = false;
		}
		else if (bGpuCrashDebugging)
		{
			// Command line switch is forcing everything on
			bEnabled = true;
		}
		else if (CVarGPUCrashDebugging->GetInt() > 0)
		{
			// Otherwise, switch everything on when opt-ed in via the r.GPUCrashDebugging cvar.
			bEnabled = true;
		}
		else
		{
			// If none of the above apply, check the individual feature cvar.
			bEnabled = CVar.GetInt() > 0;
		}

		// Allow additional command line switches to force on/off the feature via "-feature=1" / "-feature=0", or simply "-feature".
		int32 Value = 0;
		if (FParse::Value(FCommandLine::Get(), *FString::Printf(TEXT("%s="), CommandLineSwitch), Value))
		{
			bEnabled = Value > 0;
		}
		else if (FParse::Param(FCommandLine::Get(), CommandLineSwitch))
		{
			bEnabled = true;
		}

		return bEnabled;
	}

	static TAutoConsoleVariable<int32> CVarGPUCrashDebuggingBreadcrumbs(
		TEXT("r.GPUCrashDebugging.Breadcrumbs"),
		1,
		TEXT("Enable RHI breadcrumbs, a vendor-agnostic method for determining which passes were active when a GPU crash occurs"),
		ECVF_ReadOnly
	);

	RHI_API bool UseGPUCrashBreadcrumbs()
	{
		static bool bEnabled = ShouldEnableGPUCrashFeature(*CVarGPUCrashDebuggingBreadcrumbs, TEXT("gpubreadcrumbs"));
		return bEnabled;
	}

	RHI_API void CopySharedMips(FRHICommandList& RHICmdList, FRHITexture* SrcTexture, FRHITexture* DstTexture)
	{
		FRHITextureDesc const& Desc = DstTexture->GetNumMips() < SrcTexture->GetNumMips()
			? DstTexture->GetDesc()
			: SrcTexture->GetDesc();

		FRHICopyTextureInfo CopyInfo;
		CopyInfo.Size.X         = Desc.Extent.X;
		CopyInfo.Size.Y         = Desc.Extent.Y;
		CopyInfo.Size.Z         = Desc.Depth;
		CopyInfo.NumSlices      = Desc.ArraySize;
		CopyInfo.NumMips        = Desc.NumMips;
		CopyInfo.SourceMipIndex = SrcTexture->GetNumMips() - CopyInfo.NumMips;
		CopyInfo.DestMipIndex   = DstTexture->GetNumMips() - CopyInfo.NumMips;

		RHICmdList.CopyTexture(SrcTexture, DstTexture, CopyInfo);
	}

	RHI_API void CopySharedMips_AssumeSRVMaskState(FRHICommandList& RHICmdList, FRHITexture* SrcTexture, FRHITexture* DstTexture)
	{
		// Transition to copy source and dest
		{
			FRHITransitionInfo TransitionsBefore[] =
			{
				FRHITransitionInfo(SrcTexture, ERHIAccess::SRVMask, ERHIAccess::CopySrc),
				FRHITransitionInfo(DstTexture, ERHIAccess::SRVMask, ERHIAccess::CopyDest)
			};
			RHICmdList.Transition(MakeArrayView(TransitionsBefore, UE_ARRAY_COUNT(TransitionsBefore)));
		}

		CopySharedMips(RHICmdList, SrcTexture, DstTexture);

		// Transition to SRV
		{
			FRHITransitionInfo TransitionsAfter[] =
			{
				FRHITransitionInfo(SrcTexture, ERHIAccess::CopySrc, ERHIAccess::SRVMask),
				FRHITransitionInfo(DstTexture, ERHIAccess::CopyDest, ERHIAccess::SRVMask)
			};
			RHICmdList.Transition(MakeArrayView(TransitionsAfter, UE_ARRAY_COUNT(TransitionsAfter)));
		}
	}

} //! UE::RHI
