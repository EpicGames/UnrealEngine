// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIBreadcrumbs.h"
#include "Misc/MemStack.h"
#include "RHI.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Trace/Trace.inl"

#if WITH_RHI_BREADCRUMBS

thread_local FRHIBreadcrumbAllocatorArray FRHIBreadcrumbAllocator::PendingDestruction;
thread_local int32 FRHIBreadcrumbAllocator::DestructionDepth = 0;

FRHIBreadcrumbAllocator::~FRHIBreadcrumbAllocator()
{
	//
	// There can be frames, particularly in the editor, where there are hundreds of RHI breadcrumb allocators
	// chained together via the Parents field. Limit the recursive stack depth of the destructor to prevent
	// stack overflow in these cases.
	// 
	// Allowing some recursion avoids paying the cost of using the PendingDestruction array in the general
	// gameplay case where the chain is only a few allocators long.
	//
	if (DestructionDepth > 2)
	{
		if (PendingDestruction.IsEmpty())
		{
			PendingDestruction = MoveTemp(Parents);
		}
		else
		{
			for (TSharedRef<FRHIBreadcrumbAllocator>& Item : Parents)
			{
				PendingDestruction.Emplace(Item);
			}
		}
	}
	else
	{
		TGuardValue Guard(DestructionDepth, DestructionDepth + 1);
		Parents.Reset();

		while (PendingDestruction.Num())
		{
			FRHIBreadcrumbAllocatorArray ToDelete = MoveTemp(PendingDestruction);
		}
	}
}

// Constructor for the sentinel node
FRHIBreadcrumbNode::FRHIBreadcrumbNode(FRHIBreadcrumbData const& Data)
	: Data(Data)
{}

RHI_API FRHIBreadcrumbNode* const FRHIBreadcrumbNode::Sentinel = []()
{
	using namespace UE::RHI::Breadcrumbs::Private;
	static TRHIBreadcrumbDesc<0> StaticData(FRHIBreadcrumbData(TEXT("Sentinel"), __FILE__, __LINE__), nullptr);
	static TRHIBreadcrumb<TRHIBreadcrumbDesc<0>> SentinelNode(StaticData);
	return &SentinelNode;
}();

RHI_API std::atomic<uint32> FRHIBreadcrumbNode::NextID;

RHI_API uint32 FRHIBreadcrumbNode::GetLevel(FRHIBreadcrumbNode const* Node)
{
	uint32 Result = 0;
	while (Node)
	{
		Node = Node->GetParent();
		Result++;
	}
	return Result;
}

RHI_API FRHIBreadcrumbNode const* FRHIBreadcrumbNode::GetNonNullRoot(FRHIBreadcrumbNode const* Node)
{
	if (Node == nullptr || Node == Sentinel)
	{
		return nullptr;
	}

	while (Node->GetParent() != nullptr && Node->GetParent() != Sentinel)
	{
		Node = Node->GetParent();
	}

	return Node;
}

RHI_API FRHIBreadcrumbNode const* FRHIBreadcrumbNode::FindCommonAncestor(FRHIBreadcrumbNode const* Node0, FRHIBreadcrumbNode const* Node1)
{
	uint32 Level0 = GetLevel(Node0);
	uint32 Level1 = GetLevel(Node1);

	while (Level1 > Level0)
	{
		Node1 = Node1->GetParent();
		Level1--;
	}

	while (Level0 > Level1)
	{
		Node0 = Node0->GetParent();
		Level0--;
	}

	while (Node0 != Node1)
	{
		Node0 = Node0->GetParent();
		Node1 = Node1->GetParent();
	}

	return Node0;
}

RHI_API FString FRHIBreadcrumbNode::GetFullPath() const
{
	FString Result;
	FRHIBreadcrumb::FBuffer Buffer;

	auto Recurse = [&Result, &Buffer, this](auto& Recurse, FRHIBreadcrumbNode const* Current) -> void
	{
		if (!Current || Current == Sentinel)
			return;

		Recurse(Recurse, Current->Parent);

		Result += Current->GetTCHAR(Buffer);
		if (Current != this)
		{
			Result += TEXT("/");
		}
	};
	Recurse(Recurse, this);

	return Result;
}

#if WITH_ADDITIONAL_CRASH_CONTEXTS

RHI_API void FRHIBreadcrumbNode::WriteCrashData(struct FCrashContextExtendedWriter& Writer, const TCHAR* ThreadName) const
{
	TCHAR String[4096];
	size_t Length = FCString::Snprintf(String, UE_ARRAY_COUNT(String), TEXT("Breadcrumbs '%s'\n"), ThreadName);

	FRHIBreadcrumb::FBuffer Buffer;
	for (FRHIBreadcrumbNode const* Breadcrumb = this; Breadcrumb && Length < UE_ARRAY_COUNT(String); Breadcrumb = Breadcrumb->Parent)
	{
		TCHAR const* Str = Breadcrumb->GetTCHAR(Buffer);
		Length += FCString::Snprintf(&String[Length], UE_ARRAY_COUNT(String) - Length, TEXT(" - %s\n"), Str);
	}

	static int32 ReportID = 0;
	TCHAR ReportName[128];
	FCString::Snprintf(ReportName, UE_ARRAY_COUNT(ReportName), TEXT("Breadcrumbs_%s_%d"), ThreadName, ReportID++);

	Writer.AddString(ReportName, String);
	UE_LOGF(LogRHI, Error, "%ls", String);
}

RHI_API void FRHIBreadcrumbState::DumpActiveBreadcrumbs(TMap<FQueueID, TArray<FRHIBreadcrumbRange>> const& QueueRanges, EVerbosity Verbosity) const
{
	FRHIBreadcrumb::FBuffer Buffer;
	FGPUBreadcrumbCrashData CrashData(TEXT("RHI"));

	FString Tree;
	for (auto const& [QueueID, Ranges] : QueueRanges)
	{
		auto const& Data = Devices[QueueID.DeviceIndex].Pipelines[QueueID.Pipeline];

		Tree += FString::Printf(TEXT("\r\n\r\n\tDevice %d, Pipeline %s: (In: 0x%08x, Out: 0x%08x)")
			, QueueID.DeviceIndex
			, *GetRHIPipelineName(QueueID.Pipeline)
			, Data.MarkerIn
			, Data.MarkerOut
		);
		
		// Merge overlapping ranges into one unique range.
		
		// Build a [Node] -> [Next] map.
		TMap<FRHIBreadcrumbNode*, FRHIBreadcrumbNode*> ForwardMap;
		for (FRHIBreadcrumbRange const& Range : Ranges)
		{
			FRHIBreadcrumbNode** Next = nullptr;
			for (FRHIBreadcrumbNode* Node : Range.Enumerate(QueueID.Pipeline))
			{
				check(Node && Node != FRHIBreadcrumbNode::Sentinel);

				if (Next)
				{
					check(*Next == nullptr || *Next == Node);
					*Next = Node;
				}

				Next = &ForwardMap.FindOrAdd(Node);
			}
		}

		// Reverse the map, and find the node with a nullptr [Next] (there should only be one).
		FRHIBreadcrumbNode* EndNode = nullptr;
		TMap<FRHIBreadcrumbNode*, FRHIBreadcrumbNode*> ReverseMap;
		for (auto const& Pair : ForwardMap)
		{
			if (Pair.Value == nullptr)
			{
				check(!EndNode);
				EndNode = Pair.Key;
			}
			else
			{
				FRHIBreadcrumbNode*& Prev = ReverseMap.FindOrAdd(Pair.Value);
				check(Prev == nullptr); // Should not already be in the map
				Prev = Pair.Key;
			}
		}

		if (!EndNode)
		{
			Tree += TEXT("\r\n\t\tNo breadcrumb nodes found for this queue.");
		}
		else
		{
			// Find the start of the linked list.
			FRHIBreadcrumbNode* First = EndNode;
			while (FRHIBreadcrumbNode* const* Prev = ReverseMap.Find(First))
			{
				First = *Prev;
			}

			FRHIBreadcrumbRange SearchRange = { First, EndNode };
			FRHIBreadcrumbRange ActiveRange = SearchRange;

			using EState = FGPUBreadcrumbCrashData::EState;
			EState State = EState::Finished;

			TMap<FRHIBreadcrumbNode*, EState> NodeStates;
			for (FRHIBreadcrumbNode* Node : SearchRange.Enumerate(QueueID.Pipeline))
			{
				// Add this node and all it's parents to the node state map.
				for (FRHIBreadcrumbNode* Current = Node; Current; Current = Current->GetParent())
				{
					NodeStates.FindOrAdd(Current) = EState::Finished;
				}

				// Scan for the MarkerOut. Everything before this marker has been completed by the GPU.
				if (Node->ID == Data.MarkerOut)
				{
					check(ActiveRange.First == SearchRange.First);
					ActiveRange.First = Node;
				}

				// Scan for the MarkerIn. Everything after this marker has not been started by the GPU.
				if (Node->ID == Data.MarkerIn)
				{
					check(ActiveRange.Last == SearchRange.Last);
					ActiveRange.Last = Node;
				}
			}

			bool bNextIsNotStarted = false;
			for (FRHIBreadcrumbNode* Node : SearchRange.Enumerate(QueueID.Pipeline))
			{
				if (Node == ActiveRange.First)
				{
					check(State == EState::Finished);
					State = EState::Active;
				}

				if (Node == ActiveRange.Last)
				{
					check(State == EState::Active);
					bNextIsNotStarted = true;
				}
				else if (bNextIsNotStarted)
				{
					check(State == EState::Active);
					State = EState::NotStarted;
					bNextIsNotStarted = false;
				}

				switch (State)
				{
				case EState::Active:
					// Mark all nodes from current up to the root as Active
					for (FRHIBreadcrumbNode* Current = Node; Current; Current = Current->GetParent())
					{
						EState& NodeState = NodeStates.FindChecked(Current);
						if (NodeState == EState::Active)
							break;

						NodeState = EState::Active;
					}
					break;

				case EState::NotStarted:
					// Mark all nodes from current up to the root as NotStarted, assuming they're not already marked as Active
					for (FRHIBreadcrumbNode* Current = Node; Current; Current = Current->GetParent())
					{
						EState& NodeState = NodeStates.FindChecked(Current);
						if (NodeState == EState::NotStarted || NodeState == EState::Active)
							break;

						NodeState = EState::NotStarted;
					}
					break;
				}
			}

			// Now the node states have been assigned, dump out the tree.
			FGPUBreadcrumbCrashData::FSerializer CrashSerializer;
			int32 LastLevel = 0;

			auto WriteNode = [&](FRHIBreadcrumbNode* Node)
			{
				int32 Level = FRHIBreadcrumbNode::GetLevel(Node);

				FString Tabs;
				for (int32 Index = 0; Index < Level - 1; ++Index)
				{
					Tabs += TEXT("\t");
				}
				TCHAR const* Name = Node->GetTCHAR(Buffer);
				EState NodeState = NodeStates.FindChecked(Node);

				TCHAR const* StateStr; 
				switch (NodeState)
				{
				default: checkNoEntry(); [[fallthrough]];
				case EState::NotStarted: StateStr = TEXT("Not Started"); break;
				case EState::Active	   : StateStr = TEXT("     Active"); break;
				case EState::Finished  : StateStr = TEXT("   Finished"); break;
				}

				Tree += FString::Printf(TEXT("\r\n\t\t(ID: 0x%08x) [%s]\t%s%s"), Node->ID, StateStr, *Tabs, Name);

				while (LastLevel >= Level)
				{
					CrashSerializer.EndNode();
					--LastLevel;
				}

				CrashSerializer.BeginNode(Name, NodeState);
				LastLevel = Level;
			};

			// Walk into the tree to hit the first node in the search range
			auto Recurse = [&](FRHIBreadcrumbNode* Current, auto& Recurse) -> void
			{
				if (!Current)
					return;

				Recurse(Current->GetParent(), Recurse);
				WriteNode(Current);
			};
			Recurse(SearchRange.First->GetParent(), Recurse);

			// Then iterate the search range
			for (FRHIBreadcrumbNode* Node : SearchRange.Enumerate(QueueID.Pipeline))
			{
				WriteNode(Node);
			}

			while (LastLevel)
			{
				CrashSerializer.EndNode();
				--LastLevel;
			}

			CrashData.Queues.Add(FString::Printf(TEXT("%s Queue %d"), *GetRHIPipelineName(QueueID.Pipeline), QueueID.DeviceIndex), CrashSerializer.GetResult());
		}
	}

	if (CrashData.Queues.Num())
	{
		FGenericCrashContext::SetGPUBreadcrumbs(MoveTemp(CrashData));
	}

	switch (Verbosity)
	{
	case EVerbosity::Log    : UE_LOGF(LogRHI, Log    , "Active GPU breadcrumbs:%ls\r\n", *Tree); break; 
	case EVerbosity::Warning: UE_LOGF(LogRHI, Warning, "Active GPU breadcrumbs:%ls\r\n", *Tree); break;
	case EVerbosity::Error  : UE_LOGF(LogRHI, Error  , "Active GPU breadcrumbs:%ls\r\n", *Tree); break;
	}
	
}

#endif // WITH_ADDITIONAL_CRASH_CONTEXTS

#endif // WITH_RHI_BREADCRUMBS
