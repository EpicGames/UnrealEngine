// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIBufferInitializer.h"
#include "RHICommandList.h"

FRHIBufferInitializer::FRHIBufferInitializer(FRHICommandListBase& RHICmdList, FRHIBuffer* InBuffer, void* InWritableData, uint64 InWritableSize, FRHIBufferInitializer::FFinalizeCallback&& InFinalizeCallback)
	: FinalizeCallback(Forward<FFinalizeCallback>(InFinalizeCallback))
	, CommandList(&RHICmdList)
	, Buffer(InBuffer)
	, WritableData(InWritableData)
	, WritableSize(InWritableSize)
{
	check(InBuffer != nullptr);
	RHICmdList.AddPendingBufferUpload(InBuffer);
}

FBufferRHIRef FRHIBufferInitializer::Finalize()
{
	FBufferRHIRef Result;
	if (FinalizeCallback)
	{
		check(Buffer != nullptr);

		FRHICommandListScopedPipelineGuard ScopedPipeline(*CommandList);
		Result = FinalizeCallback(*CommandList);

		RemovePendingBufferUpload();

		Reset();
	}
	return Result;
}

void FRHIBufferInitializer::RemovePendingBufferUpload()
{
	CommandList->RemovePendingBufferUpload(Buffer);
}
