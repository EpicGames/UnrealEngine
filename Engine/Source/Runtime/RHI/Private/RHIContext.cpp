// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIContext.h"
#include "DynamicRHI.h"
#include "RHI.h"
#include "RHIStats.h"
#include "RHIShaderBindingLayout.h"

void RHIGenerateCrossGPUPreTransferFences(const TArrayView<const FTransferResourceParams> Params, TArray<FCrossGPUTransferFence*>& OutPreTransfer)
{
	// Generate destination GPU masks by source GPU
	uint32 DestGPUMasks[MAX_NUM_GPUS];
	for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
	{
		DestGPUMasks[GPUIndex] = 0;
	}

	for (const FTransferResourceParams& Param : Params)
	{
		check(Param.SrcGPUIndex != Param.DestGPUIndex && Param.SrcGPUIndex < GNumExplicitGPUsForRendering&& Param.DestGPUIndex < GNumExplicitGPUsForRendering);
		DestGPUMasks[Param.SrcGPUIndex] |= 1u << Param.DestGPUIndex;
	}

	// Count total number of bits in all the masks, and allocate that number of fences
	uint32 NumFences = 0;
	for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
	{
		NumFences += FMath::CountBits(DestGPUMasks[GPUIndex]);
	}
	OutPreTransfer.SetNumUninitialized(NumFences);

	// Allocate and initialize fence data
	uint32 FenceIndex = 0;
	for (uint32 SrcGPUIndex = 0; SrcGPUIndex < GNumExplicitGPUsForRendering; SrcGPUIndex++)
	{
		uint32 DestGPUMask = DestGPUMasks[SrcGPUIndex];
		for (uint32 DestGPUIndex = FMath::CountTrailingZeros(DestGPUMask); DestGPUMask; DestGPUIndex = FMath::CountTrailingZeros(DestGPUMask))
		{
			FCrossGPUTransferFence* TransferSyncPoint = RHICreateCrossGPUTransferFence();
			TransferSyncPoint->SignalGPUIndex = DestGPUIndex;
			TransferSyncPoint->WaitGPUIndex = SrcGPUIndex;

			OutPreTransfer[FenceIndex] = TransferSyncPoint;
			FenceIndex++;

			DestGPUMask &= ~(1u << DestGPUIndex);
		}
	}
}

FUniformBufferStaticBindings::FUniformBufferStaticBindings(const FRHIShaderBindingLayout* InShaderBindingLayout) : ShaderBindingLayout(InShaderBindingLayout) 
{
	UniformBuffers.SetNum(ShaderBindingLayout ? ShaderBindingLayout->GetNumUniformBufferEntries() : 0);
}

void FUniformBufferStaticBindings::AddUniformBuffer(FRHIUniformBuffer* UniformBuffer)
{
	checkf(UniformBuffer, TEXT("Attemped to assign a null uniform buffer to the global uniform buffer bindings."));		
	const FRHIUniformBufferLayout& Layout = UniformBuffer->GetLayout();

	// Only care about the static slots if no shader desc is used, otherwise the desc is used to validate that it contains the uniform buffer
	if (ShaderBindingLayout)
	{
		const FRHIUniformBufferShaderBindingLayout* UniformBufferEntry = ShaderBindingLayout->FindEntry(Layout.Name);
		checkf(UniformBufferEntry, TEXT("Attempted to set a static uniform buffer %s which is not defined in the ShaderBindingLayout provided."), *Layout.GetDebugName());
		UniformBuffers[UniformBufferEntry->CBVResourceIndex] = UniformBuffer;
	}
	else
	{
		const FUniformBufferStaticSlot Slot = Layout.StaticSlot;
		checkf(IsUniformBufferStaticSlotValid(Slot), TEXT("Attempted to set a global uniform buffer %s with an invalid slot."), *Layout.GetDebugName());

		#if VALIDATE_UNIFORM_BUFFER_STATIC_BINDINGS
		if (int32 SlotIndex = Slots.Find(Slot); SlotIndex != INDEX_NONE)
		{
			checkf(UniformBuffers[SlotIndex] == UniformBuffer, TEXT("Uniform Buffer %s was added multiple times to the binding array but with different values."), *Layout.GetDebugName());
		}
		#endif

		Slots.Add(Slot);
		UniformBuffers.Add(UniformBuffer);
		SlotCount = FMath::Max(SlotCount, Slot + 1);
	}
}

void FUniformBufferStaticBindings::Bind(TArray<FRHIUniformBuffer*>& Bindings) const
{
	Bindings.Reset();

	if (ShaderBindingLayout)
	{
		Bindings.SetNumZeroed(UniformBuffers.Num());

		for (int32 Index = 0; Index < UniformBuffers.Num(); ++Index)
		{
			Bindings[Index] = UniformBuffers[Index];
		}
	}
	else
	{
		Bindings.SetNumZeroed(SlotCount);

		for (int32 Index = 0; Index < UniformBuffers.Num(); ++Index)
		{
			Bindings[Slots[Index]] = UniformBuffers[Index];
		}
	}
}

void IRHICommandContextPSOFallback::SetGraphicsPipelineStateFromInitializer(const FGraphicsPipelineStateInitializer& PsoInit, uint32 StencilRef, bool bApplyAdditionalState)
{
#if PLATFORM_SUPPORTS_MESH_SHADERS && PLATFORM_USE_FALLBACK_PSO
	if (PsoInit.BoundShaderState.GetMeshShader())
	{
		RHISetBoundShaderState(
			RHICreateBoundShaderState(
				PsoInit.BoundShaderState.GetAmplificationShader(),
				PsoInit.BoundShaderState.GetMeshShader(),
				PsoInit.BoundShaderState.GetPixelShader()
			).GetReference()
		);
	}
	else
#endif
	{
		RHISetBoundShaderState(
			RHICreateBoundShaderState(
				PsoInit.BoundShaderState.VertexDeclarationRHI,
				PsoInit.BoundShaderState.VertexShaderRHI,
				PsoInit.BoundShaderState.PixelShaderRHI,
				PsoInit.BoundShaderState.GetGeometryShader()
			).GetReference()
		);
	}

	RHISetDepthStencilState(PsoInit.DepthStencilState, StencilRef);
	RHISetRasterizerState(PsoInit.RasterizerState);
	RHISetBlendState(PsoInit.BlendState, FLinearColor(1.0f, 1.0f, 1.0f));
	if (GSupportsDepthBoundsTest)
	{
		RHIEnableDepthBoundsTest(PsoInit.bDepthBounds);
	}
}
