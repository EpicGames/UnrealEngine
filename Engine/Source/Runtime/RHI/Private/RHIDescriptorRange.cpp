// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIDescriptorRange.h"

FRHIDescriptorRange::FRHIDescriptorRange(const FRHIDescriptorRangeDesc& InDesc, TConstArrayView<FRHIDescriptorResource> InResources)
	: FRHIResource(RRT_DescriptorRange)
	, Desc(InDesc)
{
	Resources.SetNum(InDesc.Count);
	ResourceTypes.SetNum(InDesc.Count);
	for (int32 Index = 0; Index < InResources.Num(); Index++)
	{
		UpdateResource(Index, InResources[Index]);
	}
}

FRHIDescriptorRange::~FRHIDescriptorRange()
{
}

void FRHIDescriptorRange::UpdateResource(int32 Index, FRHIDescriptorResource InResource)
{
	if (Index >= 0 && Index < Resources.Num())
	{
		Resources[Index] = InResource.Resource;
		ResourceTypes[Index] = InResource.Type;
	}
}

void FRHIDescriptorRange::UpdateResources(int32 StartIndex, TConstArrayView<FRHIDescriptorResource> InResources)
{
	for (int32 NewResourceIndex = 0; NewResourceIndex < InResources.Num(); NewResourceIndex++)
	{
		UpdateResource(StartIndex + NewResourceIndex, InResources[NewResourceIndex]);
	}
}
