// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHIGPUReadback.cpp: Convenience function implementations for async GPU 
	memory updates and readbacks
=============================================================================*/

#include "RHIGPUReadback.h"
#include "RHITransition.h"

///////////////////////////////////////////////////////////////////////////////
////////////////////     FGenericRHIStagingBuffer    //////////////////////////
///////////////////////////////////////////////////////////////////////////////

void* FGenericRHIStagingBuffer::Lock(uint32 InOffset, uint32 NumBytes)
{
	check(ShadowBuffer);
	check(!bIsLocked);
	bIsLocked = true;
	return reinterpret_cast<void*>(reinterpret_cast<uint8*>(FRHICommandListImmediate::Get().LockBuffer(ShadowBuffer, InOffset, NumBytes, RLM_ReadOnly)) + Offset);
}

void FGenericRHIStagingBuffer::Unlock()
{
	check(bIsLocked);
	FRHICommandListImmediate::Get().UnlockBuffer(ShadowBuffer);
	bIsLocked = false;
}

///////////////////////////////////////////////////////////////////////////////
////////////////////     FRHIGPUBufferReadback    /////////////////////////////
///////////////////////////////////////////////////////////////////////////////

FRHIGPUBufferReadback::FRHIGPUBufferReadback(FName RequestName) : FRHIGPUMemoryReadback(RequestName)
{
}

void FRHIGPUBufferReadback::EnqueueCopy(FRHICommandList& RHICmdList, FRHIBuffer* SourceBuffer, uint32 NumBytes)
{
	Fence->Clear();
	LastCopyGPUMask = RHICmdList.GetGPUMask();

	for (uint32 GPUIndex : LastCopyGPUMask)
	{
		SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::FromIndex(GPUIndex));

		if (!DestinationStagingBuffers[GPUIndex])
		{
			DestinationStagingBuffers[GPUIndex] = RHICreateStagingBuffer();
		}

		RHICmdList.CopyToStagingBuffer(SourceBuffer, DestinationStagingBuffers[GPUIndex], 0, NumBytes ? NumBytes : SourceBuffer->GetSize());
		RHICmdList.WriteGPUFence(Fence);
	}
}

void* FRHIGPUBufferReadback::Lock(uint32 NumBytes)
{
	// We arbitrarily read from the first GPU set in the mask.  The assumption is that in cases where the buffer is written on multiple GPUs,
	// that it will have the same data generated in lockstep on all GPUs, so it doesn't matter which GPU we read the data from.  We could
	// easily in the future allow the caller to pass in a GPU index (defaulting to INDEX_NONE) to allow reading from a specific GPU index.
	uint32 GPUIndex = LastCopyGPUMask.GetFirstIndex();

	if (DestinationStagingBuffers[GPUIndex])
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		LastLockGPUIndex = GPUIndex;
		return RHICmdList.LockStagingBuffer(DestinationStagingBuffers[GPUIndex], Fence.GetReference(), 0, NumBytes);
	}
	else
	{
		return nullptr;
	}
}

void FRHIGPUBufferReadback::Unlock()
{
	ensure(DestinationStagingBuffers[LastLockGPUIndex]);
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.UnlockStagingBuffer(DestinationStagingBuffers[LastLockGPUIndex]);
}

uint64 FRHIGPUBufferReadback::GetGPUSizeBytes() const
{
	uint64 TotalSize = 0;
	for (uint32 BufferIndex = 0; BufferIndex < UE_ARRAY_COUNT(DestinationStagingBuffers); BufferIndex++)
	{
		if (DestinationStagingBuffers[BufferIndex].IsValid())
		{
			TotalSize += DestinationStagingBuffers[BufferIndex]->GetGPUSizeBytes();
		}
	}
	return TotalSize;
}


///////////////////////////////////////////////////////////////////////////////
////////////////////     FRHIGPUTextureReadback    ////////////////////////////
///////////////////////////////////////////////////////////////////////////////

FRHIGPUTextureReadback::FRHIGPUTextureReadback(FName RequestName) : FRHIGPUMemoryReadback(RequestName)
{
}

void FRHIGPUTextureReadback::EnqueueCopy(FRHICommandList& RHICmdList, FRHITexture* SourceTexture, const FIntVector& SourcePosition, uint32 SourceSlice, const FIntVector& Size)
{
	Fence->Clear();
	LastCopyGPUMask = RHICmdList.GetGPUMask();

	if (SourceTexture)
	{
		check(!SourceTexture->IsMultisampled());

		for (uint32 GPUIndex : LastCopyGPUMask)
		{
			SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::FromIndex(GPUIndex));

			// Assume for now that every enqueue happens on a texture of the same format and size (when reused).
			bool bNeedTextureCreation = true;

			if (DestinationStagingTextures[GPUIndex])
			{
				if (GRHIGlobals.SupportLinearTextureVolumeFormat)
				{
					bNeedTextureCreation = DestinationStagingTextures[GPUIndex]->GetDesc().Dimension != SourceTexture->GetDesc().Dimension;
				}
				else
				{
					// Some platform support only 2d texture for readback, so assuming that if we have a staging texture is always a 2d texture 
					bNeedTextureCreation = false;
				}
			}

			if (bNeedTextureCreation)
			{
				FIntVector StagingTextureSize;

				// Passing 0 or negative for size means read back the entire texture.
				if (Size.X > 0 && Size.Y > 0)
				{
					StagingTextureSize = Size;
				}
				else
				{
					StagingTextureSize = SourceTexture->GetSizeXYZ();
				}

				FString FenceName = Fence->GetFName().ToString();

				FRHITextureCreateDesc Desc = FRHITextureCreateDesc(SourceTexture->GetDesc(), ERHIAccess::CopyDest, *FenceName)
					.SetExtent(StagingTextureSize.X, StagingTextureSize.Y)
					.SetFlags(ETextureCreateFlags::CPUReadback | ETextureCreateFlags::HideInVisualizeTexture);

				if (GRHIGlobals.SupportLinearTextureVolumeFormat)
				{
					switch (Desc.Dimension)
					{
					case ETextureDimension::Texture2DArray:
					case ETextureDimension::TextureCubeArray:
						ensureMsgf(Size.Z <= 1, TEXT("Readback for texture arrays supports only one slice at a time. Texture Name: %s, SourcePosition: (%d, %d, %d), SourceSlice: %u, Size: (%d, %d, %d)."),
							*SourceTexture->GetName().ToString(), SourcePosition.X, SourcePosition.Y, SourcePosition.Z, SourceSlice, Size.X, Size.Y, Size.Z);
						Desc.SetArraySize(1);
						break;
					case ETextureDimension::Texture3D:
						Desc.SetDepth(StagingTextureSize.Z);
						break;
					}
				}
				else
				{
					ensureMsgf(Size.Z <= 1, TEXT("Readback for texture arrays/volume texture supports only one slice at a time. Texture Name: %s, SourcePosition: (%d, %d, %d), SourceSlice: %u, Size: (%d, %d, %d)."),
						*SourceTexture->GetName().ToString(), SourcePosition.X, SourcePosition.Y, SourcePosition.Z, SourceSlice, Size.X, Size.Y, Size.Z);

					// Some platforms only support 2d texture for readback, create a 2d texture staging texture and force the slice size to 1
					Desc.SetArraySize(1);
					Desc.SetDepth(1);
					Desc.SetDimension(ETextureDimension::Texture2D);
				}

				DestinationStagingTextures[GPUIndex] = RHICmdList.CreateTexture(Desc);
			}
			else
			{
				RHICmdList.Transition(FRHITransitionInfo(DestinationStagingTextures[GPUIndex], ERHIAccess::CPURead, ERHIAccess::CopyDest));
			}
			
			FRHICopyTextureInfo CopyInfo;

			// Make sure we're not passing negative coordinates or size.
			if (SourcePosition.X >= 0 && SourcePosition.Y >= 0)
			{
				CopyInfo.SourcePosition = SourcePosition;
			}

			if (Size.X > 0 && Size.Y > 0)
			{
				CopyInfo.Size = Size;
			}

			CopyInfo.SourceSliceIndex = SourceSlice;
			CopyInfo.DestSliceIndex = 0;
			CopyInfo.NumSlices = 1;

			RHICmdList.CopyTexture(SourceTexture, DestinationStagingTextures[GPUIndex], CopyInfo);
			RHICmdList.Transition(FRHITransitionInfo(DestinationStagingTextures[GPUIndex], ERHIAccess::CopyDest, ERHIAccess::CPURead));
			RHICmdList.WriteGPUFence(Fence);
		}
	}
}

void* FRHIGPUTextureReadback::Lock(int32& OutRowPitchInPixels, int32 *OutBufferHeight)
{
	uint32 GPUIndex = LastCopyGPUMask.GetFirstIndex();

	if (DestinationStagingTextures[GPUIndex])
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

		LastLockGPUIndex = GPUIndex;

		void* ResultsBuffer = nullptr;
		int32 BufferWidth = 0, BufferHeight = 0;
		RHICmdList.MapStagingSurface(DestinationStagingTextures[GPUIndex], Fence.GetReference(), ResultsBuffer, BufferWidth, BufferHeight, LastLockGPUIndex);

		if (OutBufferHeight)
		{
			*OutBufferHeight = BufferHeight;
		}

		OutRowPitchInPixels = BufferWidth;

		return ResultsBuffer;
	}

	OutRowPitchInPixels = 0;
	if (OutBufferHeight)
	{
		*OutBufferHeight = 0;
	}
	return nullptr;
}

void FRHIGPUTextureReadback::Unlock()
{
	ensure(DestinationStagingTextures[LastLockGPUIndex]);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.UnmapStagingSurface(DestinationStagingTextures[LastLockGPUIndex], LastLockGPUIndex);
}

uint64 FRHIGPUTextureReadback::GetGPUSizeBytes() const
{
	uint64 TotalSize = 0;
	for (uint32 TextureIndex = 0; TextureIndex < UE_ARRAY_COUNT(DestinationStagingTextures); TextureIndex++)
	{
		if (DestinationStagingTextures[TextureIndex].IsValid())
		{
			TotalSize += DestinationStagingTextures[TextureIndex]->GetDesc().CalcMemorySizeEstimate();
		}
	}
	return TotalSize;
}
