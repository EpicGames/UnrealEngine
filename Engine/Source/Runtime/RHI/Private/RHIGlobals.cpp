// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIGlobals.h"
#include "RHIShaderPlatform.h"

FRHIGlobals GRHIGlobals;

//
//	MSAA sample offsets.
//
// https://docs.microsoft.com/en-us/windows/win32/api/d3d11/ne-d3d11-d3d11_standard_multisample_quality_levels
static FVector2f GDefaultMSAASampleOffsets[1 + 2 + 4 + 8 + 16] = {
	// MSAA x1
	FVector2f(+0.0f / 8.0f, +0.0f / 8.0f),

	// MSAA x2
	FVector2f(+4.0f / 8.0f, +4.0f / 8.0f),
	FVector2f(-4.0f / 8.0f, -4.0f / 8.0f),

	// MSAA x4
	FVector2f(-2.0f / 8.0f, -6.0f / 8.0f),
	FVector2f(+6.0f / 8.0f, -2.0f / 8.0f),
	FVector2f(-6.0f / 8.0f, +2.0f / 8.0f),
	FVector2f(+2.0f / 8.0f, +6.0f / 8.0f),

	// MSAA x8
	FVector2f(+1.0f / 8.0f, -3.0f / 8.0f),
	FVector2f(-1.0f / 8.0f, +3.0f / 8.0f),
	FVector2f(+5.0f / 8.0f, +1.0f / 8.0f),
	FVector2f(-3.0f / 8.0f, -5.0f / 8.0f),
	FVector2f(-5.0f / 8.0f, +5.0f / 8.0f),
	FVector2f(-7.0f / 8.0f, -1.0f / 8.0f),
	FVector2f(+3.0f / 8.0f, +7.0f / 8.0f),
	FVector2f(+7.0f / 8.0f, -7.0f / 8.0f),

	// MSAA x16
	FVector2f(+1.0f / 8.0f, +1.0f / 8.0f),
	FVector2f(-1.0f / 8.0f, -3.0f / 8.0f),
	FVector2f(-3.0f / 8.0f, +2.0f / 8.0f),
	FVector2f(+4.0f / 8.0f, -1.0f / 8.0f),
	FVector2f(-5.0f / 8.0f, -2.0f / 8.0f),
	FVector2f(+2.0f / 8.0f, +5.0f / 8.0f),
	FVector2f(+5.0f / 8.0f, +3.0f / 8.0f),
	FVector2f(+3.0f / 8.0f, -5.0f / 8.0f),
	FVector2f(-2.0f / 8.0f, +6.0f / 8.0f),
	FVector2f(+0.0f / 8.0f, -7.0f / 8.0f),
	FVector2f(-4.0f / 8.0f, -6.0f / 8.0f),
	FVector2f(-6.0f / 8.0f, +4.0f / 8.0f),
	FVector2f(-8.0f / 8.0f, +0.0f / 8.0f),
	FVector2f(+7.0f / 8.0f, -4.0f / 8.0f),
	FVector2f(+6.0f / 8.0f, +7.0f / 8.0f),
	FVector2f(-7.0f / 8.0f, -8.0f / 8.0f),
};

FRHIGlobals::FRHIGlobals()
{
	FMemory::Memcpy(DefaultMSAASampleOffsets, GDefaultMSAASampleOffsets, sizeof(GDefaultMSAASampleOffsets));

	for (EShaderPlatform& Platform : ShaderPlatformForFeatureLevel)
	{
		Platform = SP_NumPlatforms;
	}
}

FRHIGlobals::~FRHIGlobals() = default;
