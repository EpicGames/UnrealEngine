// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHILockTracker.h"
#include "RHI.h"

FRHILockTracker GRHILockTracker;

void FRHILockTracker::RaiseMismatchError()
{
	UE_LOGF(LogRHI, Fatal, "Mismatched RHI buffer locks.");
}
