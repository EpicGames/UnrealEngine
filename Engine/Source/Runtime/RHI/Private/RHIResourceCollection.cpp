// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIResourceCollection.h"


FRHIResourceCollection::FRHIResourceCollection(TConstArrayView<FRHIResourceCollectionMember> InMembers)
	: FRHIResource(RRT_ResourceCollection)
	, Members(InMembers)
{
	for (const FRHIResourceCollectionMember& Member : Members)
	{
		if (Member.Resource)
		{
			Member.Resource->AddRef();
		}
	}
}

FRHIResourceCollection::~FRHIResourceCollection()
{
	for (const FRHIResourceCollectionMember& Member : Members)
	{
		if (Member.Resource)
		{
			Member.Resource->Release();
		}
	}
}

void FRHIResourceCollection::UpdateMember(int32 Index, FRHIResourceCollectionMember InMember)
{
	if (Index >= 0 && Index < Members.Num())
	{
		FRHIResourceCollectionMember& ExistingMember = Members[Index];
		if (ExistingMember.Resource != InMember.Resource)
		{
			if (InMember.Resource)
			{
				InMember.Resource->AddRef();
			}

			if (ExistingMember.Resource)
			{
				ExistingMember.Resource->Release();
			}

			ExistingMember = InMember;
		}
	}
}

void FRHIResourceCollection::UpdateMembers(int32 StartIndex, TConstArrayView<FRHIResourceCollectionMember> NewMembers)
{
	for (int32 NewMemberIndex = 0; NewMemberIndex < NewMembers.Num(); NewMemberIndex++)
	{
		UpdateMember(StartIndex + NewMemberIndex, NewMembers[NewMemberIndex]);
	}
}

FRHIDescriptorHandle FRHIResourceCollection::GetBindlessHandle() const
{
	return FRHIDescriptorHandle();
}
