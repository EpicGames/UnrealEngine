// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIResources.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Experimental/Containers/HazardPointer.h"
#include "Misc/MemStack.h"
#include "RHICommandList.h"
#include "RHIUniformBufferLayoutInitializer.h"
#include "Stats/Stats.h"
#include "RHIGlobals.h"

UE::TConsumeAllMpmcQueue<FRHIResource*> PendingDeletes;
UE::TConsumeAllMpmcQueue<FRHIResource*> PendingDeletesWithLifetimeExtension;

#if DO_CHECK
// Used to check that we only enter the FRHIResource destructor via the FRHIResource::DeleteResources() function.
thread_local FRHIResource const* FRHIResource::CurrentlyDeleting = nullptr;
#endif

FRHIResource::FRHIResource(ERHIResourceType InResourceType)
	: ResourceType(InResourceType)
	, bCommitted(true)
	, bAllowExtendLifetime(true)
#if RHI_ENABLE_RESOURCE_INFO
	, bBeingTracked(false)
#endif
{
#if RHI_ENABLE_RESOURCE_INFO
	BeginTrackingResource(this);
#endif
}

FRHIResource::~FRHIResource()
{
#if DO_CHECK
	check(IsEngineExitRequested() || CurrentlyDeleting == this);
	check(AtomicFlags.GetNumRefs(std::memory_order_relaxed) == 0); // this should not have any outstanding refs
	CurrentlyDeleting = nullptr;
#endif

#if RHI_ENABLE_RESOURCE_INFO
	EndTrackingResource(this);
#endif
}

void FRHIResource::MarkForDelete() const
{
	if (!AtomicFlags.MarkForDelete(std::memory_order_release))
	{
		if (bAllowExtendLifetime)
		{
			PendingDeletesWithLifetimeExtension.ProduceItem(const_cast<FRHIResource*>(this));
		}
		else
		{
			PendingDeletes.ProduceItem(const_cast<FRHIResource*>(this));
		}
	}
}

void FRHIResource::DeleteResources(TArray<FRHIResource*> const& Resources)
{
	for (FRHIResource* Resource : Resources)
	{
		if (Resource->AtomicFlags.Deleting())
		{
#if DO_CHECK
			CurrentlyDeleting = Resource;
#endif
			delete Resource;

#if DO_CHECK
			check(CurrentlyDeleting == nullptr);
#endif
		}
	}
}

DECLARE_CYCLE_STAT(TEXT("Gather Deleted Resources"), STAT_GatherDeletedResources, STATGROUP_RHICMDLIST);

int32 GRHIResourceLifetimeRefCount = 0;

void RHIResourceLifetimeAddRef(int32 NumRefs)
{
	check(IsInRenderingThread());
	GRHIResourceLifetimeRefCount += NumRefs;
}

void RHIResourceLifetimeReleaseRef(FRHICommandListImmediate& RHICmdList, int32 NumRefs)
{
	check(IsInRenderingThread());

	int32 RefCount = GRHIResourceLifetimeRefCount -= NumRefs;
	check(RefCount >= 0);

	if (!RefCount)
	{
		RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread, ERHISubmitFlags::DeleteResources);
	}
}

void FRHIResource::GatherResourcesToDelete(TArray<FRHIResource*>& OutResources, bool bIncludeExtendedLifetimeResources)
{
	SCOPE_CYCLE_COUNTER(STAT_GatherDeletedResources);
	if (bIncludeExtendedLifetimeResources)
	{
		PendingDeletesWithLifetimeExtension.ConsumeAllLifo([&OutResources](FRHIResource* Resource)
		{
			OutResources.Emplace(Resource);
		});
	}

	PendingDeletes.ConsumeAllLifo([&OutResources](FRHIResource* Resource)
	{
		OutResources.Emplace(Resource);
	});
}

FRHIBuffer::FRHIBuffer(const FRHIBufferCreateDesc& CreateDesc)
	: FRHIViewableResource(RRT_Buffer, CreateDesc.InitialState, CreateDesc.DebugName, CreateDesc.OwnerName)
#if ENABLE_RHI_VALIDATION
	, RHIValidation::FBufferResource(CreateDesc)
#endif
	, Desc(CreateDesc)
{
}

FRHITexture::FRHITexture(const FRHITextureCreateDesc& CreateDesc)
	: FRHIViewableResource(RRT_Texture, CreateDesc.InitialState, CreateDesc.DebugName, CreateDesc.OwnerName)
#if ENABLE_RHI_VALIDATION
	, RHIValidation::FTextureResource(CreateDesc)
#endif
	, TextureDesc(CreateDesc)
{
}

FRHITexture::FRHITexture(ERHIResourceType InResourceType)
	: FRHIViewableResource(InResourceType, ERHIAccess::Unknown, nullptr, NAME_None)
{
	check(InResourceType == RRT_TextureReference);
}

void FRHITexture::SetName(FName InName)
{
	Name = InName;
}

const TCHAR* FRHIViewDesc::GetBufferTypeString(FRHIViewDesc::EBufferType BufferType)
{
	switch (BufferType)
	{
	case FRHIViewDesc::EBufferType::Unknown:				return TEXT("Unknown");
	case FRHIViewDesc::EBufferType::Typed:					return TEXT("Typed");
	case FRHIViewDesc::EBufferType::Structured:				return TEXT("Structured");
	case FRHIViewDesc::EBufferType::AccelerationStructure:	return TEXT("AccelerationStructure");
	case FRHIViewDesc::EBufferType::Raw:					return TEXT("Raw");
	default:												checkf(false, TEXT("Missing FRHIViewDesc::EBufferType %d"), int32{EnumToUnderlyingType(BufferType)});
	}
	return TEXT("");
}

const TCHAR* FRHIViewDesc::GetTextureDimensionString(FRHIViewDesc::EDimension Dimension)
{
	switch (Dimension)
	{
	case EDimension::Texture2D:				return TEXT("Texture2D");
	case EDimension::Texture2DArray:		return TEXT("Texture2DArray");
	case EDimension::Texture3D:				return TEXT("Texture3D");
	case EDimension::TextureCube:			return TEXT("TextureCube");
	case EDimension::TextureCubeArray:		return TEXT("TextureCubeArray");
	default:								checkf(false, TEXT("Missing FRHIViewDesc::EDimension %d"), int32{EnumToUnderlyingType(Dimension)});
	}

	return TEXT("");
}
FRHIViewDesc::FBuffer::FViewInfo FRHIViewDesc::FBuffer::GetViewInfo(FRHIBuffer* TargetBuffer) const
{
	check(TargetBuffer);
	checkf(BufferType != EBufferType::Unknown, TEXT("A buffer type must be specified when creating a buffer view. Use SetType() or SetTypeFromBuffer()."));

	FRHIBufferDesc const& Desc = TargetBuffer->GetDesc();
	if (Desc.IsNull())
	{
		FViewInfo Info = {};
		Info.bNullView = true;
		return Info;
	}

	FViewInfo Info = {};
	Info.BufferType = BufferType;
	Info.Format = Format;

	// Find the correct stride, and do some validation.
	switch (Info.BufferType)
	{
	default:
		checkNoEntry();
		[[fallthrough]];
		
	case EBufferType::Typed:
		checkf(!EnumHasAnyFlags(Desc.Usage, BUF_StructuredBuffer | BUF_AccelerationStructure), TEXT("Cannot create typed views of structured buffers, or ray tracing acceleration structures."));
		checkf(Format != PF_Unknown, TEXT("Format cannot be unknown for typed buffers."));
		checkf(Stride == 0, TEXT("Do not specify a stride for typed buffer views."));
		checkf((OffsetInBytes % RHIGetMinimumAlignmentForBufferBackedSRV(Info.Format)) == 0, TEXT("Buffer OffsetInBytes (%d) must be a multiple of the minimum alignment (%" UINT64_FMT ") supported by the RHI for format (%d: %s)."),
			OffsetInBytes,
			RHIGetMinimumAlignmentForBufferBackedSRV(Info.Format),
			Info.Format,
			GPixelFormats[Info.Format].Name
		);

		// Stride is determined by the format
		Info.StrideInBytes = GPixelFormats[Info.Format].BlockBytes;
		break;

	case EBufferType::Structured:
		checkf(EnumHasAnyFlags(Desc.Usage, BUF_StructuredBuffer), TEXT("The buffer descriptor is not a structured buffer, so is incompatible with this view type."));
		checkf(Format == PF_Unknown, TEXT("Structured buffer views should not specify a format."));

		// Stride is taken from the view, or the underlying buffer if not provided.
		Info.StrideInBytes = Stride == 0 ? Desc.Stride : Stride;
		checkf(Info.StrideInBytes > 0, TEXT("Stride for structured buffers must be set by the view, or on the underlying buffer resource."));
		checkf((OffsetInBytes % Info.StrideInBytes) == 0, TEXT("OffsetInBytes (%d) must be a multiple of element stride (%d)."), OffsetInBytes, Info.StrideInBytes);
		break;

	case EBufferType::AccelerationStructure:
		checkf(EnumHasAnyFlags(Desc.Usage, BUF_AccelerationStructure), TEXT("The buffer descriptor is not a ray tracing acceleration structure, so is incompatible with this view type."));
		checkf(Format == PF_Unknown, TEXT("Acceleration structure views should not specify a format."));
		checkf(RayTracingScene != nullptr, TEXT("RayTracingScene must be specified when creating view of ray tracing acceleration structures."));

		// Treat acceleration structures as a byte array.
		Info.StrideInBytes = 1;
		break;

	case EBufferType::Raw:
		checkf(GRHIGlobals.SupportsRawViewsForAnyBuffer || EnumHasAnyFlags(Desc.Usage, BUF_ByteAddressBuffer), TEXT("The current RHI does not support raw access to buffers created without the BUF_ByteAddressBuffer usage flag."));
		checkf(Format == PF_Unknown, TEXT("Raw buffer views should not specify a format."));
		checkf(Stride == 0, TEXT("Do not specify a stride for raw buffer views."));
		checkf((OffsetInBytes % 16) == 0, TEXT("The byte offset of raw views must be a multiple of 16 (specified offset: %d)."), OffsetInBytes);

		// Raw buffers are always an array of 32-bit ints.
		Info.StrideInBytes = sizeof(uint32);
		Info.Format = PF_Unknown;
		break;
	}

	checkf(OffsetInBytes < Desc.Size, TEXT("Buffer byte offset (%d) is out of bounds (size: %d)."), OffsetInBytes, Desc.Size);
	Info.OffsetInBytes = OffsetInBytes;

	// OffsetInBytes == 0 && NumElements == 0 is a special case to mean "whole resource". If offset is non-zero, we need the caller to pass the required number of elements, except for acceleration structures.
	checkf(Info.BufferType == EBufferType::AccelerationStructure || (OffsetInBytes == 0 || NumElements > 0), TEXT("NumElements field must be non-zero if a byte offset is used."));
		
	// If BufferType is AccelerationStructure or NumElements is zero, use "whole buffer".
	Info.NumElements = (Info.BufferType == EBufferType::AccelerationStructure || NumElements == 0) ? (Desc.Size - OffsetInBytes) / Info.StrideInBytes : NumElements;
	Info.SizeInBytes = Info.NumElements * Info.StrideInBytes;

	checkf(Info.OffsetInBytes + Info.SizeInBytes <= Desc.Size,
		TEXT("The bounds of the view (offset: %d, size in bytes: %d, stride: %d, num elements: %d) exceeds the size of the underlying buffer (%d bytes)."),
		Info.OffsetInBytes,
		Info.SizeInBytes,
		Info.StrideInBytes,
		Info.NumElements,
		Desc.Size);
		
	checkf(Info.Format == PF_Unknown || GPixelFormats[Info.Format].Supported, TEXT("Unsupported format (%d: %s) requested in view of buffer resource: %s")
		, Info.Format
		, GPixelFormats[Info.Format].Name
		, *TargetBuffer->GetName().ToString()
	);

	return Info;
}

RHI_API FRHIViewDesc::FBufferSRV::FViewInfo FRHIViewDesc::FBufferSRV::GetViewInfo(FRHIBuffer* TargetBuffer) const
{
	check(ViewType == EViewType::BufferSRV);
	return FViewInfo { FBuffer::GetViewInfo(TargetBuffer) };
}

RHI_API FRHIViewDesc::FBufferUAV::FViewInfo FRHIViewDesc::FBufferUAV::GetViewInfo(FRHIBuffer* TargetBuffer) const
{
	check(ViewType == EViewType::BufferUAV);
	FViewInfo Info = { FBuffer::GetViewInfo(TargetBuffer) };

	// @todo checks to see if these flags are valid
	Info.bAtomicCounter = bAtomicCounter;
	Info.bAppendBuffer = bAppendBuffer;

	return Info;
}

FRHIViewDesc::FTexture::FViewInfo FRHIViewDesc::FTexture::GetViewInfo(FRHITexture* TargetTexture) const
{
	check(TargetTexture);
	checkf(Dimension != EDimension::Unknown, TEXT("A texture dimension must be specified when creating a texture view. Use SetDimension() or SetDimensionFromTexture()."));

	FRHITextureDesc const& Desc = TargetTexture->GetDesc();

	checkf(!Desc.IsTexture3D() || Dimension == EDimension::Texture3D, TEXT("Views of 3D textures must use 3D dimension."));
	checkf(Desc.IsTexture3D() || Dimension != EDimension::Texture3D, TEXT("The underlying texture resource must be a 3D texture to create a 3D dimension view."));
	checkf(Desc.IsTextureCube() || (Dimension != EDimension::TextureCube && Dimension != EDimension::TextureCubeArray), TEXT("The underlying texture resource must be a cube (or cube array) to create a cube dimension view."));

	checkf(MipRange.Num > 0 || MipRange.First == 0, TEXT("MipRange.Num cannot be zero, unless creating a view of the entire range."));
	checkf(MipRange.First + MipRange.Num <= Desc.NumMips, TEXT("Mip range (first: %d, num: %d) is out of bounds for texture description (num mips: %d)."),
		MipRange.First,
		MipRange.Num,
		Desc.NumMips
	);

	checkf(ArrayRange.Num > 0 || ArrayRange.First == 0, TEXT("ArrayRange.Num cannot be zero, unless creating a view of the entire range."));

#if DO_CHECK
	// make sure the view fits in the texture
	{
		uint16 TextureArraySize = Desc.IsTextureCube() ? Desc.ArraySize * 6 : Desc.ArraySize;
		uint16 ViewArrayFirst = ArrayRange.First;
		uint16 ViewArrayNum = ArrayRange.Num == 0 ? Desc.ArraySize : ArrayRange.Num;

		bool bIsCubeView = false;
		if (Dimension == EDimension::TextureCube || Dimension == EDimension::TextureCubeArray)
		{
			bIsCubeView = true;
			ViewArrayFirst *= 6;
			ViewArrayNum *= 6;
		}

		uint16 SliceDividerForCheckMessage = bIsCubeView ? 6 : 1; // We want the message to report the number of elements in the units of the view
		checkf(ViewArrayFirst + ViewArrayNum <= TextureArraySize, TEXT("Array range (first: %d, num: %d) is out of bounds for texture description (array size: %d)."),
			ViewArrayFirst / SliceDividerForCheckMessage,
			ViewArrayNum / SliceDividerForCheckMessage,
			TextureArraySize / SliceDividerForCheckMessage
		);
	}
#endif

	// When ArrayRange.Num == 0, we use the number of elements from the texture. If the view is a 2D array and the texture a cube (array), we need to do x6 on the number of slices
	// We already checked that we can only create cube views on cube textures, so we only need to take into account the 2D view on cube texture case
	uint16 AdjustedTextureArraySize = Desc.ArraySize;
	if (Dimension == EDimension::Texture2DArray && Desc.IsTextureCube())
	{
		AdjustedTextureArraySize *= 6;
	}

	FViewInfo Info = {};
	Info.Format = Format == PF_Unknown ? Desc.Format : Format;
	Info.Plane = Plane;
	Info.Dimension = Dimension;

	switch (Plane)
	{
	default:
		checkNoEntry();
		[[fallthrough]];

	case ERHITexturePlane::Primary:
	case ERHITexturePlane::PrimaryCompressed:
		// Override the returned plane when using the PF_X24_G8 format.
		// @todo remove use of PF_X24_G8 in the engine.
		if (Format == PF_X24_G8)
		{
			check(EnumHasAnyFlags(Desc.Flags, TexCreate_DepthStencilTargetable));
			Info.Plane = ERHITexturePlane::Stencil;
		}
		break;

	case ERHITexturePlane::Stencil:
		check(EnumHasAnyFlags(Desc.Flags, TexCreate_DepthStencilTargetable));
		if (Format == PF_Unknown)
		{
			Info.Format = PF_X24_G8;
		}
		break;

	case ERHITexturePlane::HTile:
		check(EnumHasAnyFlags(Desc.Flags, TexCreate_DepthStencilTargetable));
		break;

	case ERHITexturePlane::Depth:
	case ERHITexturePlane::FMask:
	case ERHITexturePlane::CMask:
		break;
	}

	checkf(Info.Format == PF_Unknown || GPixelFormats[Info.Format].Supported, TEXT("Unsupported format (%d: %s) requested in view of texture resource: %s")
		, Info.Format
		, GPixelFormats[Info.Format].Name
		, *TargetTexture->GetName().ToString()
	);

	Info.ArrayRange.First = ArrayRange.First;
	Info.ArrayRange.Num = ArrayRange.Num == 0 ? AdjustedTextureArraySize : ArrayRange.Num;
	Info.bAllSlices = Info.ArrayRange.First == 0 && Info.ArrayRange.Num == AdjustedTextureArraySize;

	// Temporarily remove check while we fix issues.
	/*checkf(Desc.Format == Format ||
		Desc.UAVFormat == Format && EnumHasAnyFlags(Desc.Flags, TexCreate_UAV) ||
		Desc.AliasableFormats.Contains(Format) ||
		Format == PF_Unknown,
		TEXT("Attempting to Alias texture format and texture %s's AliasbleFormat or UAVFormat does not contain %s"), *TargetTexture->GetName().ToString(), GPixelFormats[Format].Name);*/

	return Info;
}

FRHIViewDesc::FTextureSRV::FViewInfo FRHIViewDesc::FTextureSRV::GetViewInfo(FRHITexture* TargetTexture) const
{
	check(ViewType == EViewType::TextureSRV);
	check(TargetTexture);
	FRHITextureDesc const& Desc = TargetTexture->GetDesc();

	FViewInfo Info = { FTexture::GetViewInfo(TargetTexture) };

	Info.MipRange.First = MipRange.First;
	Info.MipRange.Num = MipRange.Num == 0 ? Desc.NumMips : MipRange.Num;
	Info.bAllMips = Info.MipRange.First == 0 && Info.MipRange.Num == Desc.NumMips;

	Info.bSRGB = bDisableSRGB ? false : EnumHasAnyFlags(Desc.Flags, TexCreate_SRGB);

	return Info;
}

RHI_API FRHIViewDesc::FTextureUAV::FViewInfo FRHIViewDesc::FTextureUAV::GetViewInfo(FRHITexture* TargetTexture) const
{
	check(ViewType == EViewType::TextureUAV);
	check(TargetTexture);
	FRHITextureDesc const& Desc = TargetTexture->GetDesc();

	FViewInfo Info = { FTexture::GetViewInfo(TargetTexture) };
	Info.MipLevel = MipRange.First;
	check(MipRange.Num == 1);

	checkf(Desc.NumSamples == 1, TEXT("Cannot create an unordered access view of a multisampled texture."));
	checkf(Info.MipLevel <= Desc.NumMips, TEXT("Mip level (%d) is out of bounds for texture description (Num Mips: %d)."), Info.MipLevel, Desc.NumMips);

	// UAVs only cover 1 mip, so bAllMips is true if the base resource only has 1.
	Info.bAllMips = Desc.NumMips == 1;

	return Info;
}

static_assert(offsetof(FRHIUniformBufferResource, MemberOffset) == offsetof(FRHIUniformBufferResourceInitializer, MemberOffset), "FRHIUniformBufferResource must be identical to FRHIUniformBufferResourceInitializer");
static_assert(offsetof(FRHIUniformBufferResource, MemberType  ) == offsetof(FRHIUniformBufferResourceInitializer, MemberType  ), "FRHIUniformBufferResource must be identical to FRHIUniformBufferResourceInitializer");
static_assert(sizeof(FRHIUniformBufferResource) == sizeof(FRHIUniformBufferResourceInitializer), "FRHIUniformBufferResource must be identical to FRHIUniformBufferResourceInitializer");

template <>
struct TIsBitwiseConstructible<FRHIUniformBufferResource, FRHIUniformBufferResourceInitializer>
{
	enum { Value = true };
};

FRHIUniformBufferLayout::FRHIUniformBufferLayout(const FRHIUniformBufferLayoutInitializer& Initializer)
	: FRHIResource(RRT_UniformBufferLayout)
	, Name(Initializer.GetDebugName())
	, Resources(TConstArrayView<FRHIUniformBufferResourceInitializer>(Initializer.Resources))
	, GraphResources(TConstArrayView<FRHIUniformBufferResourceInitializer>(Initializer.GraphResources))
	, GraphTextures(TConstArrayView<FRHIUniformBufferResourceInitializer>(Initializer.GraphTextures))
	, GraphBuffers(TConstArrayView<FRHIUniformBufferResourceInitializer>(Initializer.GraphBuffers))
	, GraphUniformBuffers(TConstArrayView<FRHIUniformBufferResourceInitializer>(Initializer.GraphUniformBuffers))
	, UniformBuffers(TConstArrayView<FRHIUniformBufferResourceInitializer>(Initializer.UniformBuffers))
	, Hash(Initializer.GetHash())
	, ConstantBufferSize(Initializer.ConstantBufferSize)
	, RenderTargetsOffset(Initializer.RenderTargetsOffset)
	, StaticSlot(Initializer.StaticSlot)
	, BindingFlags(Initializer.BindingFlags)
	, Flags(Initializer.Flags)
{
}

FRHIUniformBuffer::~FRHIUniformBuffer()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE::RHI::UniformBufferContentMap::Remove(this);
#endif
}

static bool GRHIUniformBufferContentMapEnable = false;
static FAutoConsoleVariableRef CVarRHIUniformBufferContentMapEnable(
	TEXT("r.RHI.UniformBufferContentMap.Enable"),
	GRHIUniformBufferContentMapEnable,
	TEXT("When true, a copy of each FUniformBuffer content is maintained and updated on CPU inside FRHIUniformBufferContentMap."),
	ECVF_ReadOnly);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
namespace UE::RHI::UniformBufferContentMap
{
	static FCriticalSection CriticalSection;
	static TMap<const FRHIUniformBuffer*, TArray<uint8>> UniformBufferContentMap;

	void Update(const FRHIUniformBuffer* InUniformBuffer, const void* Contents)
	{
		if (GRHIUniformBufferContentMapEnable)
		{
			check(InUniformBuffer);
			FScopeLock ScopeLock(&CriticalSection);
			if (Contents)
			{
				UniformBufferContentMap.Emplace(InUniformBuffer, TArray<uint8>((const uint8*)Contents, InUniformBuffer->GetSize()));
			}
			else
			{
				UniformBufferContentMap.Emplace(InUniformBuffer, TArray<uint8>());
			}
		}
	}

	void Remove(const FRHIUniformBuffer* InUniformBuffer)
	{
		if (GRHIUniformBufferContentMapEnable)
		{
			check(InUniformBuffer);
			FScopeLock ScopeLock(&CriticalSection);
			UniformBufferContentMap.Remove(InUniformBuffer);
		}
	}

	const void* Get(const FRHIUniformBuffer* InUniformBuffer)
	{
		if (GRHIUniformBufferContentMapEnable)
		{
			check(InUniformBuffer);
			FScopeLock ScopeLock(&CriticalSection);
			TArray<uint8>* Result = UniformBufferContentMap.Find(InUniformBuffer);
			return (Result != nullptr) ? (void*)Result->GetData() : nullptr;
		}
		return nullptr;
	}
}
#endif

void FRHIRayTracingScene::SetReferencedResources(TConstArrayView<FRHIRayTracingGeometry*> InReferencedGeometries, TConstArrayView<FRHIBuffer*> InReferencedBuffers)
{
	ReferencedGeometries.Empty(InReferencedGeometries.Num());

	for (FRHIRayTracingGeometry* ReferencedGeometry : InReferencedGeometries)
	{
		ReferencedGeometries.Add(ReferencedGeometry);
	}

	ReferencedBuffers.Empty(InReferencedBuffers.Num());

	for (FRHIBuffer* ReferencedBuffer : InReferencedBuffers)
	{
		ReferencedBuffers.Add(ReferencedBuffer);
	}
}

uint32 FRayTracingPipelineStateInitializer::GetMaxLocalBindingDataSize() const
{
	uint32 MaxLocalBindingDataSize = 0;
	
	// Take max size of all miss, hit and callable shaders
	for (FRHIRayTracingShader* Shader : MissTable)
	{
		MaxLocalBindingDataSize = FMath::Max(MaxLocalBindingDataSize, Shader->LocalBindingDataSize);
	}
	for (FRHIRayTracingShader* Shader : HitGroupTable)
	{
		MaxLocalBindingDataSize = FMath::Max(MaxLocalBindingDataSize, Shader->LocalBindingDataSize);
	}
	for (FRHIRayTracingShader* Shader : CallableTable)
	{
		MaxLocalBindingDataSize = FMath::Max(MaxLocalBindingDataSize, Shader->LocalBindingDataSize);
	}

	return MaxLocalBindingDataSize;
}
