// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIShaderPlatformConfig.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreMisc.h"
#include "DataDrivenShaderPlatformInfo.h"

#if WITH_EDITOR
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif

FConfigCacheIni* UE::RHIShaderPlatformConfig::GetShaderFormatConfigCacheIni(FName ShaderFormat)
{
#if WITH_EDITOR
	if (ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatformWithSupport(TEXT("ShaderFormat"), ShaderFormat))
	{
		return TargetPlatform->GetConfigSystem();
	}
#endif

	return GConfig;
}

void UE::RHIShaderPlatformConfig::GetConfigSectionNameForShaderPlatform(FStringBuilderBase& OutSectionName, EShaderPlatform ShaderPlatform)
{
	FDataDrivenShaderPlatformInfo::GetName(ShaderPlatform).ToString(OutSectionName);
	OutSectionName.InsertAt(0, TEXT("ShaderPlatformConfig "));
}

const FConfigSection* UE::RHIShaderPlatformConfig::GetConfigSectionForShaderPlatform(FConfigCacheIni* Config, EShaderPlatform ShaderPlatform, FStringBuilderBase& OutSectionName)
{
	GetConfigSectionNameForShaderPlatform(OutSectionName, ShaderPlatform);

	return Config->GetSection(*OutSectionName, false, GEngineIni);
}

const FConfigSection* UE::RHIShaderPlatformConfig::GetConfigSectionForShaderPlatform(EShaderPlatform ShaderPlatform, FStringBuilderBase& OutSectionName)
{
	const FName ShaderFormat = FDataDrivenShaderPlatformInfo::GetShaderFormat(ShaderPlatform);
	if (FConfigCacheIni* Config = UE::RHIShaderPlatformConfig::GetShaderFormatConfigCacheIni(ShaderFormat))
	{
		return GetConfigSectionForShaderPlatform(Config, ShaderPlatform, OutSectionName);
	}

	return nullptr;
}

const FConfigSection* UE::RHIShaderPlatformConfig::GetConfigSectionForShaderPlatform(FConfigCacheIni* Config, EShaderPlatform ShaderPlatform)
{
	TStringBuilder<64> SectionName;
	return GetConfigSectionForShaderPlatform(Config, ShaderPlatform, SectionName);
}

const FConfigSection* UE::RHIShaderPlatformConfig::GetConfigSectionForShaderPlatform(EShaderPlatform ShaderPlatform)
{
	TStringBuilder<64> SectionName;
	return GetConfigSectionForShaderPlatform(ShaderPlatform, SectionName);
}

const FConfigValue* UE::RHIShaderPlatformConfig::GetConfigValueForShaderPlatform(FConfigCacheIni* Config, EShaderPlatform ShaderPlatform, FStringView SettingName)
{
	if (const FConfigSection* ConfigSection = GetConfigSectionForShaderPlatform(Config, ShaderPlatform))
	{
		return ConfigSection->Find(FName(SettingName));
	}

	return nullptr;
}

const FConfigValue* UE::RHIShaderPlatformConfig::GetConfigValueForShaderPlatform(EShaderPlatform ShaderPlatform, FStringView SettingName)
{
	if (const FConfigSection* ConfigSection = GetConfigSectionForShaderPlatform(ShaderPlatform))
	{
		return ConfigSection->Find(FName(SettingName));
	}

	return nullptr;
}
