// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIStaticShaderPlatformNames.h"
#include "RHIStaticShaderPlatformNames.gen.h"

FStaticShaderPlatformNames::FStaticShaderPlatformNames()
{
	for (const FStaticNameMapEntry* Entry = GStaticShaderNames; Entry->Name; ++Entry)
	{
		check(IsStaticPlatform(EShaderPlatform(Entry->Enum)));
		uint32 PlatformIndex = Entry->Enum - SP_StaticPlatform_First;

		FPlatform& Platform = Platforms[PlatformIndex];
		check(Platform.Name == NAME_None); // Check we've not already seen this platform

		Platform.Name = Entry->Platform;
		Platform.ShaderPlatform = FName(*FString::Printf(TEXT("SP_%s"), Entry->Name), FNAME_Add);
		Platform.ShaderFormat = FName(*FString::Printf(TEXT("SF_%s"), Entry->Name), FNAME_Add);
	}
}

const FStaticShaderPlatformNames& FStaticShaderPlatformNames::Get()
{
	static FStaticShaderPlatformNames Names;
	return Names;
}
