// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIStats.h"
#include "HAL/Platform.h"
#include "HAL/PlatformAtomics.h"
#include "Templates/Atomic.h"
#include "ProfilingDebugging/CsvProfiler.h"

int32 GNumDrawCallsRHI[MAX_NUM_GPUS] = {};
int32 GNumPrimitivesDrawnRHI[MAX_NUM_GPUS] = {};

// Define counter stats.
#if HAS_GPU_STATS

	DEFINE_STAT(STAT_RHIDraws);
	DEFINE_STAT(STAT_RHIDispatches);
	DEFINE_STAT(STAT_RHIPrimitives);
	DEFINE_STAT(STAT_RHIVertices);

#endif

// Define memory stats.
DEFINE_STAT(STAT_RenderTargetMemory2D);
DEFINE_STAT(STAT_RenderTargetMemory3D);
DEFINE_STAT(STAT_RenderTargetMemoryCube);
DEFINE_STAT(STAT_UAVTextureMemory);
DEFINE_STAT(STAT_TextureMemory2D);
DEFINE_STAT(STAT_TextureMemory3D);
DEFINE_STAT(STAT_TextureMemoryCube);
DEFINE_STAT(STAT_UniformBufferMemory);
DEFINE_STAT(STAT_IndexBufferMemory);
DEFINE_STAT(STAT_VertexBufferMemory);
DEFINE_STAT(STAT_RTAccelerationStructureMemory);
DEFINE_STAT(STAT_StructuredBufferMemory);
DEFINE_STAT(STAT_ByteAddressBufferMemory);
DEFINE_STAT(STAT_DrawIndirectBufferMemory);
DEFINE_STAT(STAT_MiscBufferMemory);

DEFINE_STAT(STAT_ReservedUncommittedBufferMemory);
DEFINE_STAT(STAT_ReservedCommittedBufferMemory);
DEFINE_STAT(STAT_ReservedUncommittedTextureMemory);
DEFINE_STAT(STAT_ReservedCommittedTextureMemory);

DEFINE_STAT(STAT_SamplerDescriptorsAllocated);
DEFINE_STAT(STAT_ResourceDescriptorsAllocated);

DEFINE_STAT(STAT_BindlessSamplerHeapMemory);
DEFINE_STAT(STAT_BindlessResourceHeapMemory);
DEFINE_STAT(STAT_BindlessSamplerDescriptorsAllocated);
DEFINE_STAT(STAT_BindlessResourceDescriptorsAllocated);

#if PLATFORM_MICROSOFT

// Define D3D memory stats.
DEFINE_STAT(STAT_D3DUpdateVideoMemoryStats);
DEFINE_STAT(STAT_D3DTotalVideoMemory);
DEFINE_STAT(STAT_D3DTotalSystemMemory);
DEFINE_STAT(STAT_D3DUsedVideoMemory);
DEFINE_STAT(STAT_D3DUsedSystemMemory);
DEFINE_STAT(STAT_D3DAvailableVideoMemory);
DEFINE_STAT(STAT_D3DAvailableSystemMemory);
DEFINE_STAT(STAT_D3DDemotedVideoMemory);
DEFINE_STAT(STAT_D3DDemotedSystemMemory);

CSV_DEFINE_CATEGORY(GPUMem, true);

void UpdateD3DMemoryStatsAndCSV(const FRHIMemoryStats& MemoryStats, bool bUpdateCSV)
{
#if STATS || CSV_PROFILER_STATS
	SCOPE_CYCLE_COUNTER(STAT_D3DUpdateVideoMemoryStats);

#if STATS
	SET_MEMORY_STAT(STAT_D3DTotalVideoMemory, MemoryStats.BudgetLocal);	
	SET_MEMORY_STAT(STAT_D3DUsedVideoMemory, MemoryStats.UsedLocal);
	SET_MEMORY_STAT(STAT_D3DAvailableVideoMemory, MemoryStats.AvailableLocal);
	SET_MEMORY_STAT(STAT_D3DDemotedVideoMemory, MemoryStats.DemotedLocal);
	
	if (MemoryStats.BudgetSystem > 0)
	{
		SET_MEMORY_STAT(STAT_D3DTotalSystemMemory, MemoryStats.BudgetSystem);
		SET_MEMORY_STAT(STAT_D3DUsedSystemMemory, MemoryStats.UsedSystem);
		SET_MEMORY_STAT(STAT_D3DAvailableSystemMemory, MemoryStats.AvailableSystem);
		SET_MEMORY_STAT(STAT_D3DDemotedSystemMemory, MemoryStats.DemotedSystem);
	}
#endif // STATS

#if CSV_PROFILER_STATS
	if (bUpdateCSV)
	{
		// Just output the two main stats (budget and used) to avoid bloating the CSV, since the rest can be inferred from them.
		CSV_CUSTOM_STAT(GPUMem, LocalBudgetMB, float(MemoryStats.BudgetLocal / 1024.0 / 1024.0), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(GPUMem, LocalUsedMB, float(MemoryStats.UsedLocal / 1024.0 / 1024.0), ECsvCustomStatOp::Set);

		if (MemoryStats.BudgetSystem > 0)
		{
			CSV_CUSTOM_STAT(GPUMem, SystemBudgetMB, float(MemoryStats.BudgetSystem / 1024.0 / 1024.0), ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(GPUMem, SystemUsedMB, float(MemoryStats.UsedSystem / 1024.0 / 1024.0), ECsvCustomStatOp::Set);
		}
	}
#endif // CSV_PROFILER_STATS
#endif // STATS || CSV_PROFILER_STATS
}

#endif // PLATFORM_MICROSOFT