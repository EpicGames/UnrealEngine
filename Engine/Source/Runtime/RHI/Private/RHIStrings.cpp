// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIStrings.h"
#include "Containers/BitArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIAccess.h"
#include "RHIGlobals.h"
#include "RHIShaderFormatDefinitions.inl"
#include "RHIStaticShaderPlatformNames.h"
#include "RHIPipeline.h"

static FName NAME_PLATFORM_WINDOWS("Windows");
static FName NAME_PLATFORM_ANDROID("Android");
static FName NAME_PLATFORM_IOS("IOS");
static FName NAME_PLATFORM_MAC("Mac");
static FName NAME_PLATFORM_TVOS("TVOS");

static FName InvalidFeatureLevelName("InvalidFeatureLevel");
static FName InvalidShadingPathName("InvalidShadingPath");

template<typename EnumType>
inline FString BuildEnumNameBitList(EnumType Value, const TCHAR* (*GetEnumName)(EnumType))
{
	if (Value == EnumType(0))
	{
		return GetEnumName(Value);
	}

	using T = __underlying_type(EnumType);
	T StateValue = (T)Value;

	FString Name;

	int32 BitIndex = 0;
	while (StateValue)
	{
		if (StateValue & 1)
		{
			if (Name.Len() > 0 && StateValue > 0)
			{
				Name += TEXT("|");
			}

			Name += GetEnumName(EnumType(T(1) << BitIndex));
		}

		BitIndex++;
		StateValue >>= 1;
	}

	return MoveTemp(Name);
}

const TCHAR* RHIVendorIdToString()
{
	return RHIVendorIdToString((EGpuVendorId)GRHIVendorId);
}

const TCHAR* RHIVendorIdToString(EGpuVendorId VendorId)
{
	// Special case for alternate Qualcomm vendor id.
	if (static_cast<int32>(VendorId) == 0x4D4F4351)
	{
		return TEXT("Qualcomm");
	}

	switch (VendorId)
	{
	case EGpuVendorId::Amd:			return TEXT("AMD");
	case EGpuVendorId::ImgTec:		return TEXT("ImgTec");
	case EGpuVendorId::Nvidia:		return TEXT("NVIDIA");
	case EGpuVendorId::Arm:			return TEXT("ARM");
	case EGpuVendorId::Broadcom:	return TEXT("Broadcom");
	case EGpuVendorId::Qualcomm:	return TEXT("Qualcomm");
	case EGpuVendorId::Apple:		return TEXT("Apple");
	case EGpuVendorId::Intel:		return TEXT("Intel");
	case EGpuVendorId::Vivante:		return TEXT("Vivante");
	case EGpuVendorId::VeriSilicon:	return TEXT("VeriSilicon");
	case EGpuVendorId::Kazan:		return TEXT("Kazan");
	case EGpuVendorId::Codeplay:	return TEXT("Codeplay");
	case EGpuVendorId::Mesa:		return TEXT("Mesa");
	case EGpuVendorId::SamsungAMD:	return TEXT("SamsungAMD");
	case EGpuVendorId::Microsoft:	return TEXT("Microsoft");
	case EGpuVendorId::NotQueried:	return TEXT("Not Queried");
	default:                        return TEXT("Unknown");
	}
}

FString LexToString(EShaderPlatform Platform, bool bError)
{
	switch (Platform)
	{
	case SP_PCD3D_SM6: return TEXT("PCD3D_SM6");
	case SP_PCD3D_SM5: return TEXT("PCD3D_SM5");
	case SP_PCD3D_ES3_1: return TEXT("PCD3D_ES3_1");
	case SP_OPENGL_PCES3_1: return TEXT("OPENGL_PCES3_1");
	case SP_OPENGL_ES3_1_ANDROID: return TEXT("OPENGL_ES3_1_ANDROID");
	case SP_METAL_ES3_1_IOS: return TEXT("METAL_ES3_1_IOS");
	case SP_METAL_SM5_IOS: return TEXT("METAL_SM5_IOS");
	case SP_METAL_SM6_IOS: return TEXT("METAL_SM6_IOS");
	case SP_METAL_ES3_1_TVOS: return TEXT("METAL_ES3_1_TVOS");
	case SP_METAL_SM5_TVOS: return TEXT("METAL_SM5_TVOS");
	case SP_METAL_SM5: return TEXT("METAL_SM5");
    case SP_METAL_SM6: return TEXT("METAL_SM6");
    case SP_METAL_SIM: return TEXT("METAL_SIM");
	case SP_METAL_ES3_1: return TEXT("METAL_ES3_1");
	case SP_VULKAN_ES3_1_ANDROID: return TEXT("VULKAN_ES3_1_ANDROID");
	case SP_VULKAN_PCES3_1: return TEXT("VULKAN_PCES3_1");
	case SP_VULKAN_SM5: return TEXT("VULKAN_SM5");
	case SP_VULKAN_SM6: return TEXT("VULKAN_SM6");
	case SP_VULKAN_SM5_ANDROID: return TEXT("VULKAN_SM5_ANDROID");
	case SP_CUSTOM_PLATFORM_FIRST: return TEXT("SP_CUSTOM_PLATFORM_FIRST");
	case SP_CUSTOM_PLATFORM_LAST: return TEXT("SP_CUSTOM_PLATFORM_LAST");

	default:
		if (FStaticShaderPlatformNames::IsStaticPlatform(Platform))
		{
			return FStaticShaderPlatformNames::Get().GetShaderPlatform(Platform).ToString();
		}
		else if (Platform > SP_CUSTOM_PLATFORM_FIRST && Platform < SP_CUSTOM_PLATFORM_LAST)
		{
			return FString::Printf(TEXT("SP_CUSTOM_PLATFORM_%d"), Platform - SP_CUSTOM_PLATFORM_FIRST);
		}
		else
		{
			checkf(!bError, TEXT("Unknown or removed EShaderPlatform %d!"), (int32)Platform);
			return FString::Printf(TEXT("Unknown or removed EShaderPlatform %d"), (int32)Platform);
		}
	}
}

FString LexToString(ERHIFeatureLevel::Type Level)
{
	switch (Level)
	{
	case ERHIFeatureLevel::ES2_REMOVED: return TEXT("ES2_REMOVED");
	case ERHIFeatureLevel::ES3_1:       return TEXT("ES3_1");
	case ERHIFeatureLevel::SM4_REMOVED: return TEXT("SM4_REMOVED");
	case ERHIFeatureLevel::SM5:         return TEXT("SM5");
	case ERHIFeatureLevel::SM6:         return TEXT("SM6");
	default:                            return TEXT("UnknownFeatureLevel");
	}
}

static const TCHAR* FeatureLevelNames[] =
{
	TEXT("ES2"),
	TEXT("ES3_1"),
	TEXT("SM4_REMOVED"),
	TEXT("SM5"),
	TEXT("SM6"),
};
static_assert(UE_ARRAY_COUNT(FeatureLevelNames) == ERHIFeatureLevel::Num, "Missing entry from feature level names.");

bool GetFeatureLevelFromName(const FString& Name, ERHIFeatureLevel::Type& OutFeatureLevel)
{
	for (int32 NameIndex = 0; NameIndex < UE_ARRAY_COUNT(FeatureLevelNames); NameIndex++)
	{
		if (FCString::Strcmp(FeatureLevelNames[NameIndex], *Name) == 0)
		{
			OutFeatureLevel = (ERHIFeatureLevel::Type)NameIndex;
			return true;
		}
	}

	OutFeatureLevel = ERHIFeatureLevel::Num;
	return false;
}

bool GetFeatureLevelFromName(FName Name, ERHIFeatureLevel::Type& OutFeatureLevel)
{
	const FString StringName(Name.ToString());
	return GetFeatureLevelFromName(StringName, OutFeatureLevel);
}

void GetFeatureLevelName(ERHIFeatureLevel::Type InFeatureLevel, FString& OutName)
{
	check(InFeatureLevel < UE_ARRAY_COUNT(FeatureLevelNames));
	if (InFeatureLevel < UE_ARRAY_COUNT(FeatureLevelNames))
	{
		OutName = FeatureLevelNames[(int32)InFeatureLevel];
	}
	else
	{
		OutName = TEXT("InvalidFeatureLevel");
	}
}

void GetFeatureLevelName(ERHIFeatureLevel::Type InFeatureLevel, FName& OutName)
{
	check(InFeatureLevel < UE_ARRAY_COUNT(FeatureLevelNames));
	if (InFeatureLevel < UE_ARRAY_COUNT(FeatureLevelNames))
	{
		OutName = FName(FeatureLevelNames[(int32)InFeatureLevel]);
	}
	else
	{
		OutName = InvalidFeatureLevelName;
	}
}

// @todo platplug: This is still here, only being used now by UMaterialShaderQualitySettings::GetOrCreatePlatformSettings
// since I have moved the other uses to FindTargetPlatformWithSupport
// But I'd like to delete it anyway!
FName ShaderPlatformToPlatformName(EShaderPlatform Platform)
{
	switch (Platform)
	{
	case SP_PCD3D_SM6:
	case SP_PCD3D_SM5:
	case SP_PCD3D_ES3_1:
	case SP_OPENGL_PCES3_1:
	case SP_VULKAN_PCES3_1:
	case SP_VULKAN_SM5:
	case SP_VULKAN_SM6:
		return NAME_PLATFORM_WINDOWS;
	case SP_VULKAN_ES3_1_ANDROID:
	case SP_VULKAN_SM5_ANDROID:
	case SP_OPENGL_ES3_1_ANDROID:
		return NAME_PLATFORM_ANDROID;
	case SP_METAL_ES3_1_IOS:
	case SP_METAL_SM5_IOS:
	case SP_METAL_SM6_IOS:
	case SP_METAL_SIM:
		return NAME_PLATFORM_IOS;
	case SP_METAL_SM5:
    case SP_METAL_SM6:
	case SP_METAL_ES3_1:
		return NAME_PLATFORM_MAC;
	case SP_METAL_ES3_1_TVOS:
	case SP_METAL_SM5_TVOS:
		return NAME_PLATFORM_TVOS;

	default:
		if (FStaticShaderPlatformNames::IsStaticPlatform(Platform))
		{
			return FStaticShaderPlatformNames::Get().GetPlatformName(Platform);
		}
		else
		{
			return NAME_None;
		}
	}
}

FName LegacyShaderPlatformToShaderFormat(EShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetShaderFormat(Platform);
}

EShaderPlatform ShaderFormatToLegacyShaderPlatform(FName ShaderFormat)
{
	return ShaderFormatNameToShaderPlatform(ShaderFormat);
}

const TCHAR* LexToString(ERHIDescriptorHeapType InHeapType)
{
	switch (InHeapType)
	{
	default: checkNoEntry();                   return TEXT("UnknownDescriptorHeapType");
	case ERHIDescriptorHeapType::Standard:     return TEXT("Standard");
	case ERHIDescriptorHeapType::RenderTarget: return TEXT("RenderTarget");
	case ERHIDescriptorHeapType::DepthStencil: return TEXT("DepthStencil");
	case ERHIDescriptorHeapType::Sampler:      return TEXT("Sampler");
	}
}

static const FName ShadingPathNames[] =
{
	FName("Deferred"),
	FName("Forward"),
	FName("Mobile"),
};
static_assert(UE_ARRAY_COUNT(ShadingPathNames) == ERHIShadingPath::Num, "Missing entry from shading path names.");

bool GetShadingPathFromName(FName Name, ERHIShadingPath::Type& OutShadingPath)
{
	for (int32 NameIndex = 0; NameIndex < UE_ARRAY_COUNT(ShadingPathNames); NameIndex++)
	{
		if (ShadingPathNames[NameIndex] == Name)
		{
			OutShadingPath = (ERHIShadingPath::Type)NameIndex;
			return true;
		}
	}

	OutShadingPath = ERHIShadingPath::Num;
	return false;
}

void GetShadingPathName(ERHIShadingPath::Type InShadingPath, FString& OutName)
{
	check(InShadingPath < UE_ARRAY_COUNT(ShadingPathNames));
	if (InShadingPath < UE_ARRAY_COUNT(ShadingPathNames))
	{
		ShadingPathNames[(int32)InShadingPath].ToString(OutName);
	}
	else
	{
		OutName = TEXT("InvalidShadingPath");
	}
}

void GetShadingPathName(ERHIShadingPath::Type InShadingPath, FName& OutName)
{
	check(InShadingPath < UE_ARRAY_COUNT(ShadingPathNames));
	if (InShadingPath < UE_ARRAY_COUNT(ShadingPathNames))
	{
		OutName = ShadingPathNames[(int32)InShadingPath];
	}
	else
	{
		OutName = InvalidShadingPathName;
	}
}

FString GetBufferUsageFlagsName(EBufferUsageFlags BufferUsage)
{
	return BuildEnumNameBitList<EBufferUsageFlags>(BufferUsage, [](EBufferUsageFlags BufferUsage)
	{
		switch (BufferUsage)
		{
		case BUF_None:					return TEXT("BUF_None");
		case BUF_Static:				return TEXT("BUF_Static");
		case BUF_Dynamic:				return TEXT("BUF_Dynamic");
		case BUF_Volatile:				return TEXT("BUF_Volatile");
		case BUF_UnorderedAccess:		return TEXT("BUF_UnorderedAccess");
		case BUF_ByteAddressBuffer:		return TEXT("BUF_ByteAddressBuffer");
		case BUF_SourceCopy:			return TEXT("BUF_SourceCopy");
		case BUF_DrawIndirect:			return TEXT("BUF_DrawIndirect");
		case BUF_ShaderResource:		return TEXT("BUF_ShaderResource");
		case BUF_KeepCPUAccessible:		return TEXT("BUF_KeepCPUAccessible");
		case BUF_FastVRAM:				return TEXT("BUF_FastVRAM");
		case EBufferUsageFlags::NNE:	return TEXT("BUF_NNE");
		case BUF_Shared:				return TEXT("BUF_Shared");
		case BUF_AccelerationStructure:	return TEXT("BUF_AccelerationStructure");
		case BUF_VertexBuffer:			return TEXT("BUF_VertexBuffer");
		case BUF_IndexBuffer:			return TEXT("BUF_IndexBuffer");
		case BUF_StructuredBuffer:		return TEXT("BUF_StructuredBuffer");
		case BUF_MultiGPUAllocate:		return TEXT("BUF_MultiGPUAllocate");
		case BUF_MultiGPUGraphIgnore:	return TEXT("BUF_MultiGPUGraphIgnore");
		case BUF_RayTracingScratch:		return TEXT("BUF_RayTracingScratch");
		case BUF_NullResource:			return TEXT("BUF_NullResource");
		case BUF_UniformBuffer:			return TEXT("BUF_UniformBuffer");
		case BUF_ReservedResource:		return TEXT("BUF_ReservedResource");
		}
		ensureMsgf(false, TEXT("Unexpected buffer usage flag bit: %x"), uint32{EnumToUnderlyingType(BufferUsage)});
		return TEXT("BUF_Unknown");
	});
}

FString GetTextureCreateFlagsName(ETextureCreateFlags TextureCreateFlags)
{
	return BuildEnumNameBitList<ETextureCreateFlags>(TextureCreateFlags, [](ETextureCreateFlags TextureCreateFlags)
	{
		switch (TextureCreateFlags)
		{
		case ETextureCreateFlags::None:								return TEXT("None");
		case ETextureCreateFlags::RenderTargetable:					return TEXT("RenderTargetable");
		case ETextureCreateFlags::ResolveTargetable:				return TEXT("ResolveTargetable");
		case ETextureCreateFlags::DepthStencilTargetable:			return TEXT("DepthStencilTargetable");
		case ETextureCreateFlags::ShaderResource:					return TEXT("ShaderResource");
		case ETextureCreateFlags::SRGB:								return TEXT("SRGB");
		case ETextureCreateFlags::CPUWritable:						return TEXT("CPUWritable");
		case ETextureCreateFlags::NoTiling:							return TEXT("NoTiling");
		case ETextureCreateFlags::VideoDecode:						return TEXT("VideoDecode");
		case ETextureCreateFlags::Dynamic:							return TEXT("Dynamic");
		case ETextureCreateFlags::InputAttachmentRead:				return TEXT("InputAttachmentRead");
		case ETextureCreateFlags::Foveation:						return TEXT("Foveation");
		case ETextureCreateFlags::Tiling3D:							return TEXT("Tiling3D");
		case ETextureCreateFlags::Memoryless:						return TEXT("Memoryless");
		case ETextureCreateFlags::FastVRAMPartialAlloc:				return TEXT("FastVRAMPartialAlloc");
		case ETextureCreateFlags::DisableSRVCreation:				return TEXT("DisableSRVCreation");
		case ETextureCreateFlags::DisableDCC:						return TEXT("DisableDCC");
		case ETextureCreateFlags::UAV:								return TEXT("UAV");
		case ETextureCreateFlags::Presentable:						return TEXT("Presentable");
		case ETextureCreateFlags::CPUReadback:						return TEXT("CPUReadback");
		case ETextureCreateFlags::OfflineProcessed:					return TEXT("OfflineProcessed");
		case ETextureCreateFlags::FastVRAM:							return TEXT("FastVRAM");
		case ETextureCreateFlags::HideInVisualizeTexture:			return TEXT("HideInVisualizeTexture");
		case ETextureCreateFlags::Virtual:							return TEXT("Virtual");
		case ETextureCreateFlags::TargetArraySlicesIndependently:	return TEXT("TargetArraySlicesIndependently");
		case ETextureCreateFlags::Shared:							return TEXT("Shared");
		case ETextureCreateFlags::NoFastClear:						return TEXT("NoFastClear");
		case ETextureCreateFlags::DepthStencilResolveTarget:		return TEXT("DepthStencilResolveTarget");
		case ETextureCreateFlags::Streamable:						return TEXT("Streamable");
		case ETextureCreateFlags::NoFastClearFinalize:				return TEXT("NoFastClearFinalize");
		case ETextureCreateFlags::ReduceMemoryWithTilingMode:		return TEXT("ReduceMemoryWithTilingMode");
		case ETextureCreateFlags::AtomicCompatible:					return TEXT("AtomicCompatible");
		case ETextureCreateFlags::Atomic64Compatible:				return TEXT("Atomic64Compatible");
		case ETextureCreateFlags::External:							return TEXT("External");
		case ETextureCreateFlags::MultiGPUGraphIgnore:				return TEXT("MultiGPUGraphIgnore");
		case ETextureCreateFlags::ReservedResource:					return TEXT("ReservedResource");
		case ETextureCreateFlags::ImmediateCommit:					return TEXT("ImmediateCommit");
		}
		ensureMsgf(false, TEXT("Unexpected texture creation flag bit: %" UINT64_x_FMT), uint64{EnumToUnderlyingType(TextureCreateFlags)});
		return TEXT("Unknown");
	});
}

struct FRHIResourceTypeName
{
	ERHIResourceType Type;
	const TCHAR* Name;
};
static const FRHIResourceTypeName GRHIResourceTypeNames[] =
{
#define RHI_RESOURCE_TYPE_DEF(Name) { RRT_##Name, TEXT(#Name) }

	RHI_RESOURCE_TYPE_DEF(None),
	RHI_RESOURCE_TYPE_DEF(SamplerState),
	RHI_RESOURCE_TYPE_DEF(RasterizerState),
	RHI_RESOURCE_TYPE_DEF(DepthStencilState),
	RHI_RESOURCE_TYPE_DEF(BlendState),
	RHI_RESOURCE_TYPE_DEF(VertexDeclaration),
	RHI_RESOURCE_TYPE_DEF(VertexShader),
	RHI_RESOURCE_TYPE_DEF(MeshShader),
	RHI_RESOURCE_TYPE_DEF(AmplificationShader),
	RHI_RESOURCE_TYPE_DEF(PixelShader),
	RHI_RESOURCE_TYPE_DEF(GeometryShader),
	RHI_RESOURCE_TYPE_DEF(RayTracingShader),
	RHI_RESOURCE_TYPE_DEF(ComputeShader),
	RHI_RESOURCE_TYPE_DEF(GraphicsPipelineState),
	RHI_RESOURCE_TYPE_DEF(ComputePipelineState),
	RHI_RESOURCE_TYPE_DEF(RayTracingPipelineState),
	RHI_RESOURCE_TYPE_DEF(BoundShaderState),
	RHI_RESOURCE_TYPE_DEF(UniformBufferLayout),
	RHI_RESOURCE_TYPE_DEF(UniformBuffer),
	RHI_RESOURCE_TYPE_DEF(Buffer),
	RHI_RESOURCE_TYPE_DEF(Texture),
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RHI_RESOURCE_TYPE_DEF(Texture2D),
	RHI_RESOURCE_TYPE_DEF(Texture2DArray),
	RHI_RESOURCE_TYPE_DEF(Texture3D),
	RHI_RESOURCE_TYPE_DEF(TextureCube),
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	RHI_RESOURCE_TYPE_DEF(TextureReference),
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RHI_RESOURCE_TYPE_DEF(TimestampCalibrationQuery),
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	RHI_RESOURCE_TYPE_DEF(GPUFence),
	RHI_RESOURCE_TYPE_DEF(RenderQuery),
	RHI_RESOURCE_TYPE_DEF(RenderQueryPool),
	RHI_RESOURCE_TYPE_DEF(Viewport),
	RHI_RESOURCE_TYPE_DEF(UnorderedAccessView),
	RHI_RESOURCE_TYPE_DEF(ShaderResourceView),
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RHI_RESOURCE_TYPE_DEF(RayTracingAccelerationStructure),
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	RHI_RESOURCE_TYPE_DEF(RayTracingGeometry),
	RHI_RESOURCE_TYPE_DEF(RayTracingScene),
	RHI_RESOURCE_TYPE_DEF(StagingBuffer),
	RHI_RESOURCE_TYPE_DEF(CustomPresent),
	RHI_RESOURCE_TYPE_DEF(ShaderLibrary),
	RHI_RESOURCE_TYPE_DEF(StreamSourceSlot)
};

ERHIResourceType RHIResourceTypeFromString(const FString& InString)
{
	for (const auto& TypeName : GRHIResourceTypeNames)
	{
		if (InString.Equals(TypeName.Name, ESearchCase::IgnoreCase))
		{
			return TypeName.Type;
		}
	}
	return RRT_None;
}

const TCHAR* StringFromRHIResourceType(ERHIResourceType ResourceType)
{
	for (const auto& TypeName : GRHIResourceTypeNames)
	{
		if (TypeName.Type == ResourceType)
		{
			return TypeName.Name;
		}
	}
	return TEXT("<unknown>");
}

FString GetRHIAccessName(ERHIAccess Access)
{
	return BuildEnumNameBitList<ERHIAccess>(Access, [](ERHIAccess AccessBit)
	{
		switch (AccessBit)
		{
		default: checkNoEntry(); // fall through
		case ERHIAccess::Unknown:             return TEXT("Unknown");
		case ERHIAccess::CPURead:             return TEXT("CPURead");
		case ERHIAccess::Present:             return TEXT("Present");
		case ERHIAccess::IndirectArgs:        return TEXT("IndirectArgs");
		case ERHIAccess::VertexOrIndexBuffer: return TEXT("VertexOrIndexBuffer");
		case ERHIAccess::SRVCompute:          return TEXT("SRVCompute");
		case ERHIAccess::SRVGraphicsPixel:    return TEXT("SRVGraphicsPixel");
		case ERHIAccess::SRVGraphicsNonPixel: return TEXT("SRVGraphicsNonPixel");
		case ERHIAccess::CopySrc:             return TEXT("CopySrc");
		case ERHIAccess::ResolveSrc:          return TEXT("ResolveSrc");
		case ERHIAccess::DSVRead:             return TEXT("DSVRead");
		case ERHIAccess::UAVCompute:          return TEXT("UAVCompute");
		case ERHIAccess::UAVGraphics:         return TEXT("UAVGraphics");
		case ERHIAccess::RTV:                 return TEXT("RTV");
		case ERHIAccess::CopyDest:            return TEXT("CopyDest");
		case ERHIAccess::ResolveDst:          return TEXT("ResolveDst");
		case ERHIAccess::DSVWrite:            return TEXT("DSVWrite");
		case ERHIAccess::BVHRead:             return TEXT("BVHRead");
		case ERHIAccess::BVHWrite:            return TEXT("BVHWrite");
		case ERHIAccess::Discard:             return TEXT("Discard");
		case ERHIAccess::ShadingRateSource:	  return TEXT("ShadingRateSource");
		}
	});
}

FString GetResourceTransitionFlagsName(EResourceTransitionFlags Flags)
{
	return BuildEnumNameBitList<EResourceTransitionFlags>(Flags, [](EResourceTransitionFlags Value)
	{
		switch (Value)
		{
		default: checkNoEntry(); // fall through
		case EResourceTransitionFlags::None:                return TEXT("None");
		case EResourceTransitionFlags::MaintainCompression: return TEXT("MaintainCompression");
		case EResourceTransitionFlags::IgnoreAfterState:	return TEXT("IgnoreAfterState");
		}
	});
}

FString GetRHIPipelineName(ERHIPipeline Pipeline)
{
	return BuildEnumNameBitList<ERHIPipeline>(Pipeline, [](ERHIPipeline Value)
	{
		if (Value == ERHIPipeline(0))
		{
			return TEXT("None");
		}

		switch (Value)
		{
		default: checkNoEntry(); // fall through
		case ERHIPipeline::Graphics:     return TEXT("Graphics");
		case ERHIPipeline::AsyncCompute: return TEXT("AsyncCompute");
		}
	});
}

const TCHAR* GetTextureDimensionString(ETextureDimension Dimension)
{
	switch (Dimension)
	{
	case ETextureDimension::Texture2D:        return TEXT("Texture2D");
	case ETextureDimension::Texture2DArray:   return TEXT("Texture2DArray");
	case ETextureDimension::Texture3D:        return TEXT("Texture3D");
	case ETextureDimension::TextureCube:      return TEXT("TextureCube");
	case ETextureDimension::TextureCubeArray: return TEXT("TextureCubeArray");
	default:                                  return TEXT("");
	}
}

const TCHAR* GetTextureCreateFlagString(ETextureCreateFlags TextureCreateFlag)
{
	switch (TextureCreateFlag)
	{
	case ETextureCreateFlags::None:
		return TEXT("None");
	case ETextureCreateFlags::RenderTargetable:
		return TEXT("RenderTargetable");
	case ETextureCreateFlags::ResolveTargetable:
		return TEXT("ResolveTargetable");
	case ETextureCreateFlags::DepthStencilTargetable:
		return TEXT("DepthStencilTargetable");
	case ETextureCreateFlags::ShaderResource:
		return TEXT("ShaderResource");
	case ETextureCreateFlags::SRGB:
		return TEXT("SRGB");
	case ETextureCreateFlags::CPUWritable:
		return TEXT("CPUWritable");
	case ETextureCreateFlags::NoTiling:
		return TEXT("NoTiling");
	case ETextureCreateFlags::VideoDecode:
		return TEXT("VideoDecode");
	case ETextureCreateFlags::Dynamic:
		return TEXT("Dynamic");
	case ETextureCreateFlags::InputAttachmentRead:
		return TEXT("InputAttachmentRead");
	case ETextureCreateFlags::Foveation:
		return TEXT("Foveation");
	case ETextureCreateFlags::Tiling3D:
		return TEXT("Tiling3D");
	case ETextureCreateFlags::Memoryless:
		return TEXT("Memoryless");
	case ETextureCreateFlags::FastVRAMPartialAlloc:
		return TEXT("FastVRAMPartialAlloc");
	case ETextureCreateFlags::DisableSRVCreation:
		return TEXT("DisableSRVCreation");
	case ETextureCreateFlags::DisableDCC:
		return TEXT("DisableDCC");
	case ETextureCreateFlags::UAV:
		return TEXT("UAV");
	case ETextureCreateFlags::Presentable:
		return TEXT("Presentable");
	case ETextureCreateFlags::CPUReadback:
		return TEXT("CPUReadback");
	case ETextureCreateFlags::OfflineProcessed:
		return TEXT("OfflineProcessed");
	case ETextureCreateFlags::FastVRAM:
		return TEXT("FastVRAM");
	case ETextureCreateFlags::HideInVisualizeTexture:
		return TEXT("HideInVisualizeTexture");
	case ETextureCreateFlags::Virtual:
		return TEXT("Virtual");
	case ETextureCreateFlags::TargetArraySlicesIndependently:
		return TEXT("TargetArraySlicesIndependently");
	case ETextureCreateFlags::Shared:
		return TEXT("Shared");
	case ETextureCreateFlags::NoFastClear:
		return TEXT("NoFastClear");
	case ETextureCreateFlags::DepthStencilResolveTarget:
		return TEXT("DepthStencilResolveTarget");
	case ETextureCreateFlags::Streamable:
		return TEXT("Streamable");
	case ETextureCreateFlags::NoFastClearFinalize:
		return TEXT("NoFastClearFinalize");
	case ETextureCreateFlags::Atomic64Compatible:
		return TEXT("Atomic64Compatible");
	case ETextureCreateFlags::ReduceMemoryWithTilingMode:
		return TEXT("ReduceMemoryWithTilingMode");
	case ETextureCreateFlags::AtomicCompatible:
		return TEXT("AtomicCompatible");
	case ETextureCreateFlags::External:
		return TEXT("External");
	case ETextureCreateFlags::MultiGPUGraphIgnore:
		return TEXT("MultiGPUGraphIgnore");
	}
	return TEXT("");
}

const TCHAR* GetBufferUsageFlagString(EBufferUsageFlags BufferUsage)
{
	switch (BufferUsage)
	{
	case EBufferUsageFlags::None:
		return TEXT("None");
	case EBufferUsageFlags::Static:
		return TEXT("Static");
	case EBufferUsageFlags::Dynamic:
		return TEXT("Dynamic");
	case EBufferUsageFlags::Volatile:
		return TEXT("Volatile");
	case EBufferUsageFlags::UnorderedAccess:
		return TEXT("UnorderedAccess");
	case EBufferUsageFlags::ByteAddressBuffer:
		return TEXT("ByteAddressBuffer");
	case EBufferUsageFlags::SourceCopy:
		return TEXT("SourceCopy");
	case EBufferUsageFlags::DrawIndirect:
		return TEXT("DrawIndirect");
	case EBufferUsageFlags::ShaderResource:
		return TEXT("ShaderResource");
	case EBufferUsageFlags::KeepCPUAccessible:
		return TEXT("KeepCPUAccessible");
	case EBufferUsageFlags::FastVRAM:
		return TEXT("FastVRAM");
	case EBufferUsageFlags::Shared:
		return TEXT("Shared");
	case EBufferUsageFlags::AccelerationStructure:
		return TEXT("AccelerationStructure");
	case EBufferUsageFlags::VertexBuffer:
		return TEXT("VertexBuffer");
	case EBufferUsageFlags::IndexBuffer:
		return TEXT("IndexBuffer");
	case EBufferUsageFlags::StructuredBuffer:
		return TEXT("StructuredBuffer");
	case EBufferUsageFlags::ReservedResource:
		return TEXT("ReservedResource");
	}
	return TEXT("");
}

const TCHAR* GetUniformBufferBaseTypeString(EUniformBufferBaseType BaseType)
{
	switch (BaseType)
	{
	case UBMT_INVALID:
		return TEXT("UBMT_INVALID");
	case UBMT_BOOL:
		return TEXT("UBMT_BOOL");
	case UBMT_INT32:
		return TEXT("UBMT_INT32");
	case UBMT_UINT32:
		return TEXT("UBMT_UINT32");
	case UBMT_FLOAT32:
		return TEXT("UBMT_FLOAT32");
	case UBMT_TEXTURE:
		return TEXT("UBMT_TEXTURE");
	case UBMT_SRV:
		return TEXT("UBMT_SRV");
	case UBMT_UAV:
		return TEXT("UBMT_UAV");
	case UBMT_SAMPLER:
		return TEXT("UBMT_SAMPLER");
	case UBMT_RDG_TEXTURE:
		return TEXT("UBMT_RDG_TEXTURE");
	case UBMT_RDG_TEXTURE_ACCESS:
		return TEXT("UBMT_RDG_TEXTURE_ACCESS");
	case UBMT_RDG_TEXTURE_ACCESS_ARRAY:
		return TEXT("UBMT_RDG_TEXTURE_ACCESS_ARRAY");
	case UBMT_RDG_TEXTURE_SRV:
		return TEXT("UBMT_RDG_TEXTURE_SRV");
	case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
		return TEXT("UBMT_RDG_TEXTURE_NON_PIXEL_SRV");
	case UBMT_RDG_TEXTURE_UAV:
		return TEXT("UBMT_RDG_TEXTURE_UAV");
	case UBMT_RDG_BUFFER_ACCESS:
		return TEXT("UBMT_RDG_BUFFER_ACCESS");
	case UBMT_RDG_BUFFER_ACCESS_ARRAY:
		return TEXT("UBMT_RDG_BUFFER_ACCESS_ARRAY");
	case UBMT_RDG_BUFFER_SRV:
		return TEXT("UBMT_RDG_BUFFER_SRV");
	case UBMT_RDG_BUFFER_UAV:
		return TEXT("UBMT_RDG_BUFFER_UAV");
	case UBMT_RDG_UNIFORM_BUFFER:
		return TEXT("UBMT_RDG_UNIFORM_BUFFER");
	case UBMT_NESTED_STRUCT:
		return TEXT("UBMT_NESTED_STRUCT");
	case UBMT_INCLUDED_STRUCT:
		return TEXT("UBMT_INCLUDED_STRUCT");
	case UBMT_REFERENCED_STRUCT:
		return TEXT("UBMT_REFERENCED_STRUCT");
	case UBMT_RENDER_TARGET_BINDING_SLOTS:
		return TEXT("UBMT_RENDER_TARGET_BINDING_SLOTS");
	case UBMT_RESOURCE_COLLECTION:
		return TEXT("UBMT_RESOURCE_COLLECTION");
	case UBMT_DESCRIPTOR_RANGE:
		return TEXT("UBMT_DESCRIPTOR_RANGE");
	}
	return TEXT("");
}

const TCHAR* GetShaderCodeResourceBindingTypeName(EShaderCodeResourceBindingType BindingType)
{
	switch (BindingType)
	{
	case EShaderCodeResourceBindingType::Invalid:							return TEXT("Invalid");
	case EShaderCodeResourceBindingType::SamplerState:						return TEXT("SamplerState");
	case EShaderCodeResourceBindingType::Texture2D:							return TEXT("Texture2D");
	case EShaderCodeResourceBindingType::Texture2DArray:					return TEXT("Texture2DArray");
	case EShaderCodeResourceBindingType::Texture2DMultiView:					return TEXT("Texture2DMultiView");
	case EShaderCodeResourceBindingType::Texture2DMS:						return TEXT("Texture2DMS");
	case EShaderCodeResourceBindingType::Texture3D:							return TEXT("Texture3D");
	case EShaderCodeResourceBindingType::TextureCube:						return TEXT("TextureCube");
	case EShaderCodeResourceBindingType::TextureCubeArray:					return TEXT("TextureCubeArray");
	case EShaderCodeResourceBindingType::TextureMetadata:					return TEXT("TextureMetadata");
	case EShaderCodeResourceBindingType::Buffer:							return TEXT("Buffer");
	case EShaderCodeResourceBindingType::StructuredBuffer:					return TEXT("StructuredBuffer");
	case EShaderCodeResourceBindingType::ByteAddressBuffer:					return TEXT("ByteAddressBuffer");
	case EShaderCodeResourceBindingType::RaytracingAccelerationStructure:	return TEXT("RaytracingAccelerationStructure");
	case EShaderCodeResourceBindingType::RWTexture2D:						return TEXT("RWTexture2D");
	case EShaderCodeResourceBindingType::RWTexture2DArray:					return TEXT("RWTexture2DArray");
	case EShaderCodeResourceBindingType::RWTexture2DMultiView:					return TEXT("RWTexture2DMultiView");
	case EShaderCodeResourceBindingType::RWTexture3D:						return TEXT("RWTexture3D");
	case EShaderCodeResourceBindingType::RWTextureCube:						return TEXT("RWTextureCube");
	case EShaderCodeResourceBindingType::RWTextureMetadata:					return TEXT("RWTextureMetadata");
	case EShaderCodeResourceBindingType::RWBuffer:							return TEXT("RWBuffer");
	case EShaderCodeResourceBindingType::RWStructuredBuffer:				return TEXT("RWStructuredBuffer");
	case EShaderCodeResourceBindingType::RWByteAddressBuffer:				return TEXT("RWByteAddressBuffer");
	case EShaderCodeResourceBindingType::RasterizerOrderedTexture2D:		return TEXT("RasterizerOrderedTexture2D");
	case EShaderCodeResourceBindingType::ResourceCollection:				return TEXT("ResourceCollection");
	case EShaderCodeResourceBindingType::DescriptorRange:					return TEXT("DescriptorRange");
	default:																checkf(false, TEXT("Missing EShaderCodeResourceBindingType %d"), int32{EnumToUnderlyingType(BindingType)});
	}
	return TEXT("");
}

bool GetBindlessConfigurationFromString(FStringView InString, ERHIBindlessConfiguration& OutBindlessConfiguration)
{
	if (InString.Equals(TEXT("Disabled"), ESearchCase::IgnoreCase))
	{
		OutBindlessConfiguration = ERHIBindlessConfiguration::Disabled;
		return true;
	}

	if (InString.Equals(TEXT("RayTracing"), ESearchCase::IgnoreCase))
	{
		OutBindlessConfiguration = ERHIBindlessConfiguration::RayTracing;
		return true;
	}

	if (InString.Equals(TEXT("Minimal"), ESearchCase::IgnoreCase))
	{
		OutBindlessConfiguration = ERHIBindlessConfiguration::Minimal;
		return true;
	}

	if (InString.Equals(TEXT("All"), ESearchCase::IgnoreCase))
	{
		OutBindlessConfiguration = ERHIBindlessConfiguration::All;
		return true;
	}

	return false;
}

const TCHAR* GetBindlessConfigurationString(ERHIBindlessConfiguration InBindlessConfiguration)
{
	switch (InBindlessConfiguration)
	{
	case ERHIBindlessConfiguration::Disabled:   return TEXT("Disabled");
	case ERHIBindlessConfiguration::RayTracing: return TEXT("RayTracing");
	case ERHIBindlessConfiguration::Minimal:    return TEXT("Minimal");
	case ERHIBindlessConfiguration::All:        return TEXT("All");
	default:
		checkf(false, TEXT("Unknown Bindless Configuration: %d"), static_cast<uint32>(InBindlessConfiguration));
		return TEXT("");
	}
}

const TCHAR* GetDescriptorTypeString(ERHIDescriptorType InDescriptorType)
{
	switch (InDescriptorType)
	{
	case ERHIDescriptorType::BufferSRV:             return TEXT("BufferSRV");
	case ERHIDescriptorType::BufferUAV:             return TEXT("BufferUAV");
	case ERHIDescriptorType::TypedBufferSRV:        return TEXT("TypedBufferSRV");
	case ERHIDescriptorType::TypedBufferUAV:        return TEXT("TypedBufferUAV");
	case ERHIDescriptorType::TextureSRV:            return TEXT("TextureSRV");
	case ERHIDescriptorType::TextureUAV:            return TEXT("TextureUAV");
	case ERHIDescriptorType::CBV:                   return TEXT("CBV");
	case ERHIDescriptorType::Sampler:               return TEXT("Sampler");
	case ERHIDescriptorType::AccelerationStructure: return TEXT("AccelerationStructure");
	case ERHIDescriptorType::Invalid:               return TEXT("Invalid");
	default:
		checkf(false, TEXT("Unknown DescriptorType: %d"), static_cast<uint32>(InDescriptorType));
		return TEXT("");
	}
}

FString GetDescriptorMaskString(ERHIDescriptorTypeMask InTypeMask)
{
	TStringBuilder<128> Result;

	if (InTypeMask == ERHIDescriptorTypeMask::None)
	{
		Result.Append(TEXT("<empty>"));
	}
	else
	{
		uint32 IterMask = static_cast<uint32>(InTypeMask);
		while (IterMask)
		{
			const uint32 TypeIndex = FBitSet::GetAndClearNextBit(IterMask);
			const ERHIDescriptorType DescriptorType = static_cast<ERHIDescriptorType>(TypeIndex);

			if (Result.Len())
			{
				Result.Append(TEXT("|"));
			}

			Result.Append(GetDescriptorTypeString(DescriptorType));
		}
	}

	return FString(Result);
}

