// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHITextureInitializer.h"
#include "RHICommandList.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"

FRHITextureInitializer::FRHITextureInitializer(FRHICommandListBase& RHICmdList, FRHITexture* InTexture, void* InWritableData, uint64 InWritableSize, FRHITextureInitializer::FFinalizeCallback&& InFinalizeCallback, FGetSubresourceCallback&& InGetSubresourceCallback)
	: FinalizeCallback(Forward<FFinalizeCallback>(InFinalizeCallback))
	, GetSubresourceCallback(Forward<FGetSubresourceCallback>(InGetSubresourceCallback))
	, CommandList(&RHICmdList)
	, Texture(InTexture)
	, WritableData(InWritableData)
	, WritableSize(InWritableSize)
	, Desc(InTexture->GetDesc())
{
	check(InTexture != nullptr);
	RHICmdList.AddPendingTextureUpload(InTexture);
}

FTextureRHIRef FRHITextureInitializer::Finalize()
{
	FTextureRHIRef Result;
	if (FinalizeCallback)
	{
		check(Texture != nullptr);

		LLM_SCOPE(EnumHasAnyFlags(Texture->GetFlags(), TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable) ? ELLMTag::RenderTargets : ELLMTag::Textures);

		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(Texture->GetOwnerName(), ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FRHITextureCreateDesc().GetTraceClassName(), ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET_FNAME(Texture->GetName(), FRHITextureCreateDesc().GetTraceClassName(), Texture->GetOwnerName());

		FRHICommandListScopedPipelineGuard ScopedPipeline(*CommandList);
		Result = FinalizeCallback(*CommandList);

		RemovePendingTextureUpload();

		Reset();
	}
	return Result;
}

void FRHITextureInitializer::RemovePendingTextureUpload()
{
	CommandList->RemovePendingTextureUpload(Texture);
}
