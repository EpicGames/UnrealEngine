// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHITextureReference.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIShaderPlatform.h"

FRHITextureReference::FRHITextureReference(FRHITexture* InReferencedTexture)
	: FRHITexture(RRT_TextureReference)
	, ReferencedTexture(InReferencedTexture ? InReferencedTexture : DefaultTexture.GetReference())
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	, BindlessHandle(FRHIDescriptorHandle())
#endif
{
	check(DefaultTexture);
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
FRHITextureReference::FRHITextureReference(FRHITexture* InReferencedTexture, FRHIDescriptorHandle InBindlessHandle)
	: FRHITexture(RRT_TextureReference)
	, ReferencedTexture(InReferencedTexture ? InReferencedTexture : DefaultTexture.GetReference())
	, BindlessHandle(InBindlessHandle)
{
	check(DefaultTexture);
}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

FRHITextureReference::~FRHITextureReference() = default;

FRHITextureReference* FRHITextureReference::GetTextureReference()
{
	return this;
}

FRHIDescriptorHandle FRHITextureReference::GetDefaultBindlessHandle() const
{ 
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BindlessHandle.IsValid())
	{
		return BindlessHandle;
	}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

	check(ReferencedTexture);
	return ReferencedTexture->GetDefaultBindlessHandle();
}

void* FRHITextureReference::GetNativeResource() const
{
	check(ReferencedTexture);
	return ReferencedTexture->GetNativeResource();
}

void* FRHITextureReference::GetNativeShaderResourceView() const
{
	check(ReferencedTexture);
	return ReferencedTexture->GetNativeShaderResourceView();
}

void* FRHITextureReference::GetTextureBaseRHI()
{
	check(ReferencedTexture);
	return ReferencedTexture->GetTextureBaseRHI();
}

void FRHITextureReference::GetWriteMaskProperties(void*& OutData, uint32& OutSize)
{
	check(ReferencedTexture);
	return ReferencedTexture->GetWriteMaskProperties(OutData, OutSize);
}

#if ENABLE_RHI_VALIDATION
RHIValidation::FResource* FRHITextureReference::GetTrackerResource()
{
	check(ReferencedTexture);
	return ReferencedTexture->GetTrackerResource();
}
#endif

const FRHITextureDesc& FRHITextureReference::GetDesc() const
{
	check(ReferencedTexture);
	return ReferencedTexture->GetDesc();
}

void FRHITextureReference::SetTrackedAccessFromContext(FRHITrackedAccess InTrackedAccess)
{
	check(ReferencedTexture);
	ReferencedTexture->SetTrackedAccessFromContext(InTrackedAccess);
}