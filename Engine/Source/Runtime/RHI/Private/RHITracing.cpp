// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHITracing.h"
#include "Trace/Trace.inl"
#include "HAL/PlatformTime.h"

#if UE_RENDER_TRACING_ENABLED

RENDER_TRACE_EVENT_BEGIN(CommandListCreated)
	UE_TRACE_EVENT_FIELD(uint64, CmdListID)
	UE_TRACE_EVENT_FIELD(bool, bImmediate)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(CommandListDetach)
	UE_TRACE_EVENT_FIELD(uint64, SourceCmdListID)
	UE_TRACE_EVENT_FIELD(uint64, DetachedCmdListID)
	UE_TRACE_EVENT_FIELD(bool, bIsImmediateFlush)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(CommandListFinishRecording)
	UE_TRACE_EVENT_FIELD(uint64, CmdListID)
	UE_TRACE_EVENT_FIELD(uint8, Flags)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(CommandListDestroyed)
	UE_TRACE_EVENT_FIELD(uint64, CmdListID)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SubmitCommandLists)
	UE_TRACE_EVENT_FIELD(uint64[], CmdListIDs)
	UE_TRACE_EVENT_FIELD(uint32, RTSubmissionID)
	UE_TRACE_EVENT_FIELD(uint16, Flags)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(TranslateJobCreated)
	UE_TRACE_EVENT_FIELD(uint64, TranslateJobID)
	UE_TRACE_EVENT_FIELD(uint8, Flags)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SplitTranslateJob)
	UE_TRACE_EVENT_FIELD(uint64, TranslateJobID)
	UE_TRACE_EVENT_FIELD(uint64, CmdListID)
	UE_TRACE_EVENT_FIELD(uint8, Flags)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(DispatchCommandList)
	UE_TRACE_EVENT_FIELD(uint64, TranslateJobID)
	UE_TRACE_EVENT_FIELD(uint64, CmdListID)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(BeginTranslateCommandList)
	UE_TRACE_EVENT_FIELD(uint64, TranslateJobID)
	UE_TRACE_EVENT_FIELD(uint64, CmdListID)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(ActivatePipelines)
	UE_TRACE_EVENT_FIELD(uint64, CmdListID)
	UE_TRACE_EVENT_FIELD(uint8, Pipelines)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SetTranslateContext)
	UE_TRACE_EVENT_FIELD(uint64, CmdListID)
	UE_TRACE_EVENT_FIELD(uint64, ContextID)
	UE_TRACE_EVENT_FIELD(uint8, Pipeline)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(EndTranslateCommandList)
	UE_TRACE_EVENT_FIELD(uint64, CmdListID)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(FinalizeTranslateJob)
	UE_TRACE_EVENT_FIELD(uint64, TranslateJobID)
	UE_TRACE_EVENT_FIELD(uint8, Flags)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SubmitTranslateJobs)
	UE_TRACE_EVENT_FIELD(uint64[], TranslateJobIDs)
	UE_TRACE_EVENT_FIELD(uint64, SubmissionID)
UE_TRACE_EVENT_END()

namespace RenderTracing
{

static uint32 GNextRTSubmissionID = 1;

void CommandListCreated(const FRHICommandListBase& CmdList, bool bImmediate)
{
	if (!IsEnabled(ERenderTracingChannels::RHICmdLists))
	{
		return;
	}

	RENDER_TRACE_LOG(CommandListCreated)
		<< RENDER_TRACE_VALUE(CommandListCreated, CmdListID, reinterpret_cast<uintptr_t>(&CmdList))
		<< RENDER_TRACE_VALUE(CommandListCreated, bImmediate, bImmediate);
}

void CommandListDetach(const FRHICommandListBase& SourceCmdList, const FRHICommandListBase& DetachedCmdList, bool bIsImmediateFlush)
{
	if (!IsEnabled(ERenderTracingChannels::RHICmdLists))
	{
		return;
	}

	RENDER_TRACE_LOG(CommandListDetach)
		<< RENDER_TRACE_VALUE(CommandListDetach, SourceCmdListID, reinterpret_cast<uintptr_t>(&SourceCmdList))
		<< RENDER_TRACE_VALUE(CommandListDetach, DetachedCmdListID, reinterpret_cast<uintptr_t>(&DetachedCmdList))
		<< RENDER_TRACE_VALUE(CommandListDetach, bIsImmediateFlush, bIsImmediateFlush);
}

void CommandListFinishRecording(const FRHICommandListBase& CmdList, bool bUsesLockFence)
{
	if (!IsEnabled(ERenderTracingChannels::RHICmdLists))
	{
		return;
	}

	uint8 Flags = 0;
	Flags |= (uint8)bUsesLockFence << 0;

	RENDER_TRACE_LOG(CommandListFinishRecording)
		<< RENDER_TRACE_VALUE(CommandListFinishRecording, CmdListID, reinterpret_cast<uintptr_t>(&CmdList))
		<< RENDER_TRACE_VALUE(CommandListFinishRecording, Flags, Flags);
}

void CommandListDestroyed(const FRHICommandListBase& CmdList)
{
	if (!IsEnabled(ERenderTracingChannels::RHICmdLists))
	{
		return;
	}

	RENDER_TRACE_LOG(CommandListDestroyed)
		<< RENDER_TRACE_VALUE(CommandListDestroyed, CmdListID, reinterpret_cast<uintptr_t>(&CmdList));
}

uint32 SubmitCommandLists(TConstArrayView<FRHICommandListBase*> CmdLists, ERHISubmitFlags Flags)
{
	if (!IsEnabled(ERenderTracingChannels::RHITranslation))
	{
		return 0;
	}

	// If SubmitToGPU is set, we want to assign a unique ID to this event so that it can be linked later
	// to the RHI submission task.
	const uint32 RTSubmissionID = EnumHasAllFlags(Flags, ERHISubmitFlags::SubmitToGPU) ? GNextRTSubmissionID++ : 0;

	RENDER_TRACE_LOG(SubmitCommandLists)
		<< RENDER_TRACE_ARRAY_VALUE(SubmitCommandLists, CmdListIDs, CmdLists, uint64)
		<< RENDER_TRACE_VALUE(SubmitCommandLists, RTSubmissionID, RTSubmissionID)
		<< RENDER_TRACE_VALUE(SubmitCommandLists, Flags, static_cast<uint16>(Flags));

	return RTSubmissionID;
}

void SplitTranslateJob(const void* TranslateJobID, FRHICommandListBase& CmdList, bool bSplitParallel, bool bSplitForThreshold, bool bSplitForParentChild)
{
	if (!IsEnabled(ERenderTracingChannels::RHITranslation))
	{
		return;
	}

	uint8 Flags = 0;
	Flags |= (uint8)bSplitParallel << 0;
	Flags |= (uint8)bSplitForThreshold << 1;
	Flags |= (uint8)bSplitForParentChild << 2;

	RENDER_TRACE_LOG(SplitTranslateJob)
		<< RENDER_TRACE_VALUE(SplitTranslateJob, TranslateJobID, reinterpret_cast<uintptr_t>(TranslateJobID))
		<< RENDER_TRACE_VALUE(SplitTranslateJob, CmdListID, reinterpret_cast<uintptr_t>(&CmdList))
		<< RENDER_TRACE_VALUE(SplitTranslateJob, Flags, Flags);
}

void DispatchCommandList(const void* TranslateJobID, const FRHICommandListBase& CmdList)
{
	if (!IsEnabled(ERenderTracingChannels::RHITranslation))
	{
		return;
	}

	RENDER_TRACE_LOG(DispatchCommandList)
		<< RENDER_TRACE_VALUE(DispatchCommandList, TranslateJobID, reinterpret_cast<uintptr_t>(TranslateJobID))
		<< RENDER_TRACE_VALUE(DispatchCommandList, CmdListID, reinterpret_cast<uintptr_t>(&CmdList));
}

void BeginTranslateCommandList(const void* TranslateJobID, const FRHICommandListBase& CmdList)
{
	if (!IsEnabled(ERenderTracingChannels::RHITranslation))
	{
		return;
	}

	RENDER_TRACE_LOG(BeginTranslateCommandList)
		<< RENDER_TRACE_VALUE(BeginTranslateCommandList, TranslateJobID, reinterpret_cast<uintptr_t>(TranslateJobID))
		<< RENDER_TRACE_VALUE(BeginTranslateCommandList, CmdListID, reinterpret_cast<uintptr_t>(&CmdList));
}

void ActivatePipelines(const FRHICommandListBase& CmdList, uint8 Pipelines)
{
	if (!IsEnabled(ERenderTracingChannels::RHITranslation))
	{
		return;
	}

	RENDER_TRACE_LOG(ActivatePipelines)
		<< RENDER_TRACE_VALUE(ActivatePipelines, CmdListID, reinterpret_cast<uintptr_t>(&CmdList))
		<< RENDER_TRACE_VALUE(ActivatePipelines, Pipelines, Pipelines);
}

void SetTranslateContext(const FRHICommandListBase& CmdList, const IRHIComputeContext* Context, uint8 Pipeline)
{
	if (!IsEnabled(ERenderTracingChannels::RHITranslation))
	{
		return;
	}

	RENDER_TRACE_LOG(SetTranslateContext)
		<< RENDER_TRACE_VALUE(SetTranslateContext, CmdListID, reinterpret_cast<uintptr_t>(&CmdList))
		<< RENDER_TRACE_VALUE(SetTranslateContext, ContextID, reinterpret_cast<uintptr_t>(Context))
		<< RENDER_TRACE_VALUE(SetTranslateContext, Pipeline, Pipeline);
}

void EndTranslateCommandList(const FRHICommandListBase& CmdList)
{
	if (!IsEnabled(ERenderTracingChannels::RHITranslation))
	{
		return;
	}

	RENDER_TRACE_LOG(EndTranslateCommandList)
		<< RENDER_TRACE_VALUE(EndTranslateCommandList, CmdListID, reinterpret_cast<uintptr_t>(&CmdList));
}

void FinalizeTranslateJob(const void* TranslateJobID, bool bParallel, bool bUsingSubCmdLists)
{
	if (!IsEnabled(ERenderTracingChannels::RHITranslation))
	{
		return;
	}

	uint8 Flags = 0;
	Flags |= (uint8)bParallel << 0;
	Flags |= (uint8)bUsingSubCmdLists << 1;

	RENDER_TRACE_LOG(FinalizeTranslateJob)
		<< RENDER_TRACE_VALUE(FinalizeTranslateJob, TranslateJobID, reinterpret_cast<uintptr_t>(TranslateJobID))
		<< RENDER_TRACE_VALUE(FinalizeTranslateJob, Flags, Flags);
}

void SubmitTranslateJobs(TConstArrayView<void*> TranslateJobIDs, uint64 SubmissionID)
{
	if (!IsEnabled(ERenderTracingChannels::RHITranslation))
	{
		return;
	}

	RENDER_TRACE_LOG(SubmitTranslateJobs)
		<< RENDER_TRACE_ARRAY_VALUE(SubmitTranslateJobs, TranslateJobIDs, TranslateJobIDs, uint64)
		<< RENDER_TRACE_VALUE(SubmitTranslateJobs, SubmissionID, SubmissionID);
}

}
#endif
