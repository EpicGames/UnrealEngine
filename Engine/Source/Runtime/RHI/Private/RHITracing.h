// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderTracing.h"
#include "RHICommandList.h"
#include "RHIContext.h"

#if UE_RENDER_TRACING_ENABLED

namespace RenderTracing
{

void CommandListCreated(const FRHICommandListBase& CmdList, bool bImmediate);
void CommandListDetach(const FRHICommandListBase& SourceCmdList, const FRHICommandListBase& DetachedCmdList, bool bIsImmediateFlush);
void CommandListFinishRecording(const FRHICommandListBase& CmdList, bool bUsesLockFence);
void CommandListDestroyed(const FRHICommandListBase& CmdList);

uint32 SubmitCommandLists(TConstArrayView<FRHICommandListBase*> CmdLists, ERHISubmitFlags Flags);
void SplitTranslateJob(const void* TranslateJobID, FRHICommandListBase& CmdList, bool bSplitParallel, bool bSplitForThreshold, bool bSplitForParentChild);
void DispatchCommandList(const void* TranslateJobID, const FRHICommandListBase& CmdList);
void BeginTranslateCommandList(const void* TranslateJobID, const FRHICommandListBase& CmdList);
void EndTranslateCommandList(const FRHICommandListBase& CmdList);
void ActivatePipelines(const FRHICommandListBase& CmdList, uint8 Pipelines);
void SetTranslateContext(const FRHICommandListBase& CmdList, const IRHIComputeContext* Context, uint8 Pipeline);
void FinalizeTranslateJob(const void* TranslateJobID, bool bParallel, bool bUsingSubCmdLists);
void SubmitTranslateJobs(TConstArrayView<void*> TranslateJobIDs, uint64 SubmissionID);

}

#endif
