// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHITransientResourceAllocator.h"

FRHITransientResource::FRHITransientResource(
	FRHIResource* InResource,
	uint64 InGpuVirtualAddress,
	uint64 InHash,
	uint64 InSize,
	ERHITransientAllocationType InAllocationType,
	ERHITransientResourceType InResourceType)
	: Resource(InResource)
	, GpuVirtualAddress(InGpuVirtualAddress)
	, Hash(InHash)
	, Size(InSize)
	, AllocationType(InAllocationType)
	, ResourceType(InResourceType)
{}

FRHITransientResource::FRHITransientResource(
	const FResourceTask& InResourceTask,
	uint64 InHash,
	uint64 InSize,
	ERHITransientAllocationType InAllocationType,
	ERHITransientResourceType InResourceType)
	: ResourceTask(InResourceTask)
	, Hash(InHash)
	, Size(InSize)
	, AllocationType(InAllocationType)
	, ResourceType(InResourceType)
{}

FRHITransientResource::~FRHITransientResource() = default;

FRHITransientTexture::FRHITransientTexture(
	const FResourceTask& InResourceTask,
	uint64 InHash,
	uint64 InSize,
	ERHITransientAllocationType InAllocationType,
	const FRHITextureCreateInfo& InCreateInfo)
	: FRHITransientResource(InResourceTask, InHash, InSize, InAllocationType, ERHITransientResourceType::Texture)
	, CreateInfo(InCreateInfo)
{}

FRHITransientTexture::FRHITransientTexture(
	FRHIResource* InTexture,
	uint64 InGpuVirtualAddress,
	uint64 InHash,
	uint64 InSize,
	ERHITransientAllocationType InAllocationType,
	const FRHITextureCreateInfo& InCreateInfo)
	: FRHITransientResource(InTexture, InGpuVirtualAddress, InHash, InSize, InAllocationType, ERHITransientResourceType::Texture)
	, CreateInfo(InCreateInfo)
{}

FRHITransientTexture::~FRHITransientTexture() = default;

FRHITransientBuffer::~FRHITransientBuffer() = default;

