// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHITransition.h"
#include "DynamicRHI.h"

uint64 GRHITransitionPrivateData_SizeInBytes = 0;
uint64 GRHITransitionPrivateData_AlignInBytes = 0;

FRHIViewableResource* GetViewableResource(const FRHITransitionInfo& Info)
{
	switch (Info.Type)
	{
		case FRHITransitionInfo::EType::Buffer:
		{
			return Info.Buffer;
		}

		case FRHITransitionInfo::EType::Texture:
		{
			if (!Info.Texture)
			{
				return nullptr;
			}

			FRHITextureReference* TextureReference = Info.Texture->GetTextureReference(); 
			if (TextureReference)
			{
				return TextureReference->GetReferencedTexture();
			}

			return Info.Texture;
		}

		case FRHITransitionInfo::EType::UAV:
		{
			return Info.UAV ? Info.UAV->GetResource() : nullptr;
		}
	}

	return nullptr;
}

void FRHITransition::Cleanup() const
{
	FRHITransition* Transition = const_cast<FRHITransition*>(this);
	RHIReleaseTransition(Transition);

	// Explicit destruction of the transition.
	Transition->~FRHITransition();
	FRHICmdListBaseLinearAllocator::Free(Transition);
}