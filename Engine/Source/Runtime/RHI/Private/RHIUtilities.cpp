// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
RHIUtilities.cpp:
=============================================================================*/

#include "RHIUtilities.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/PlatformStackWalk.h"
#include "RHI.h"
#include "GenericPlatform/GenericPlatformFramePacer.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformFramePacer.h"
#include "Misc/CommandLine.h"
#include "RHIAccess.h"
#include "RHIFwd.h"
#include "RHIStrings.h"
#include "Tasks/Task.h"
#include "ProfilingDebugging/MiscTrace.h"

#define USE_FRAME_OFFSET_THREAD 1

#if UE_INPUT_LATENCY_TRACKING
int32 GVsyncInformationInsights = 0;
static FAutoConsoleVariableRef CVarVsyncInformationInsights(
	TEXT("r.VsyncInformationInsights"),
	GVsyncInformationInsights,
	TEXT("Whether to show Vsync and Input events in UnrealInsights:")
	TEXT("0 - No information (default)")
	TEXT("1 - Input / Vsync frame counter information")
	TEXT("2 - Input / Vsync frame counter and present counter information")
);

static FRHIFrameInfo GRHIFrameInfo;
#endif

TAutoConsoleVariable<FString> FDumpTransitionsHelper::CVarDumpTransitionsForResource(
	TEXT("r.DumpTransitionsForResource"),
	TEXT(""),
	TEXT("Prints callstack when the given resource is transitioned. Only implemented for DX11 at the moment.")
	TEXT("Name of the resource to dump"),
	ECVF_Default);

FName FDumpTransitionsHelper::DumpTransitionForResource = NAME_None;
void FDumpTransitionsHelper::DumpTransitionForResourceHandler()
{
	const FString NewValue = CVarDumpTransitionsForResource.GetValueOnGameThread();
	DumpTransitionForResource = FName(*NewValue);
}

void FDumpTransitionsHelper::DumpResourceTransition(const FName& ResourceName, const ERHIAccess TransitionType)
{
	const FName ResourceDumpName = FDumpTransitionsHelper::DumpTransitionForResource;
	if ((ResourceDumpName != NAME_None) && (ResourceDumpName == ResourceName))
	{
		const uint32 DumpCallstackSize = 2047;
		ANSICHAR DumpCallstack[DumpCallstackSize] = { 0 };

		FPlatformStackWalk::StackWalkAndDump(DumpCallstack, DumpCallstackSize, 2);
		UE_LOGF(LogRHI, Log, "%ls transition to: %ls", *ResourceDumpName.ToString(), *GetRHIAccessName(TransitionType));
		UE_LOGF(LogRHI, Log, "%ls", ANSI_TO_TCHAR(DumpCallstack));
	}
}

FAutoConsoleVariableSink FDumpTransitionsHelper::CVarDumpTransitionsForResourceSink(FConsoleCommandDelegate::CreateStatic(&FDumpTransitionsHelper::DumpTransitionForResourceHandler));

void SetDepthBoundsTest(FRHICommandList& RHICmdList, float WorldSpaceDepthNear, float WorldSpaceDepthFar, const FMatrix& ProjectionMatrix)
{
	if (GSupportsDepthBoundsTest)
	{
		FVector4 Near = ProjectionMatrix.TransformFVector4(FVector4(0, 0, WorldSpaceDepthNear));
		FVector4 Far = ProjectionMatrix.TransformFVector4(FVector4(0, 0, WorldSpaceDepthFar));
		float DepthNear = float(Near.Z / Near.W);
		float DepthFar = float(Far.Z / Far.W);

		DepthFar = FMath::Clamp(DepthFar, 0.0f, 1.0f);
		DepthNear = FMath::Clamp(DepthNear, 0.0f, 1.0f);

		if (DepthNear <= DepthFar)
		{
			DepthNear = 1.0f;
			DepthFar = 0.0f;
		}

		// Note, using a reversed z depth surface
		RHICmdList.SetDepthBounds(DepthFar, DepthNear);
	}
}

TAutoConsoleVariable<int32> CVarRHISyncInterval(
	TEXT("rhi.SyncInterval"),
	1,
	TEXT("Determines the frequency of VSyncs in supported RHIs.\n")
	TEXT("This is in multiples of 16.66 on a 60hz display, but some platforms support higher refresh rates.\n")
	TEXT("Assuming 60fps, the values correspond to:\n")
	TEXT("  0 - Unlocked (present immediately)\n")
	TEXT("  1 - Present every vblank interval\n")
	TEXT("  2 - Present every 2 vblank intervals\n")
	TEXT("  3 - etc...\n"),
	ECVF_Default
);

TAutoConsoleVariable<float> CVarRHIPresentThresholdTop(
	TEXT("rhi.PresentThreshold.Top"),
	0.0f,
	TEXT("Specifies the percentage of the screen from the top where tearing is allowed.\n")
	TEXT("Only effective on supported platforms.\n")
	TEXT("Range: 0.0 - 1.0\n"),
	ECVF_Default
);

TAutoConsoleVariable<float> CVarRHIPresentThresholdBottom(
	TEXT("rhi.PresentThreshold.Bottom"),
	0.0f,
	TEXT("Specifies the percentage of the screen from the bottom where tearing is allowed.\n")
	TEXT("Only effective on supported platforms.\n")
	TEXT("Range: 0.0 - 1.0\n"),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarRHISyncAllowEarlyKick(
	TEXT("rhi.SyncAllowEarlyKick"),
	1,
	TEXT("When 1, allows the RHI vsync thread to kick off the next frame early if we've missed the vsync."),
	ECVF_Default
);

#if USE_FRAME_OFFSET_THREAD
TAutoConsoleVariable<float> CVarRHISyncSlackMS(
	TEXT("rhi.SyncSlackMS"),
	10,
	TEXT("Increases input latency by this many milliseconds, to help performance (trade-off tunable). Gamethread will be kicked off this many milliseconds before the vsync"),
	ECVF_Default
);
#endif

TAutoConsoleVariable<int32> CVarRHISyncAllowVariable(
	TEXT("rhi.SyncAllowVariable"),
	1,
	TEXT("When 1, allows the RHI to use variable refresh rate, if supported by the output hardware."),
	ECVF_Default
);

float GRHIFrameTimeMS = 0.0f;
double GLastRHITimeInSeconds = 0.0;

int32 GEnableConsole120Fps = 0;
int32 InternalEnableConsole120Fps = 0;
static void OnEnableConsole120FpsCVarRHIChanged(IConsoleVariable* Variable)
{
	InternalEnableConsole120Fps &= (FPlatformMisc::GetMaxSupportedRefreshRate() >= 120);
	if (InternalEnableConsole120Fps != GEnableConsole120Fps)
	{
		GEnableConsole120Fps = InternalEnableConsole120Fps;
		// needs to update the FramePace since it updates the SyncInterval based on the RefreshRate.
		FPlatformRHIFramePacer::SetFramePace(GEnableConsole120Fps ? 120 : 60);
		UE_LOGF(LogRHI, Log, "Console 120Fps = %d", GEnableConsole120Fps);
	}
}

static FAutoConsoleVariableRef CVarRHInableConsole120Fps(
	TEXT("rhi.EnableConsole120Fps"),
	InternalEnableConsole120Fps,
	TEXT("Enable Console 120fps if Monitor supports it and Console is properly setup"),
	FConsoleVariableDelegate::CreateStatic(&OnEnableConsole120FpsCVarRHIChanged),
	ECVF_Default
);

class FRHIFrameFlipTrackingRunnable : public FRunnable
{
	static FRunnableThread* Thread;
	static FRHIFrameFlipTrackingRunnable Singleton;
	static bool bInitialized;
	static bool bRun;

	FCriticalSection CS;
	struct FFramePair
	{
		FFramePair(uint64 InPresentIndex, const UE::Tasks::FTaskEvent& InEvent)
			: PresentIndex(InPresentIndex)
			, Event(InEvent)
		{}

		~FFramePair()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SyncTrigger_Swapchain);
			Event.Trigger();
		}

		uint64 PresentIndex;
		UE::Tasks::FTaskEvent Event;
	};
	TArray<FFramePair> FramePairs;

	FRHIFrameFlipTrackingRunnable();

	virtual uint32 Run() override;
	virtual void Stop() override;

public:
	static void Initialize();
	static void Shutdown();

	static void TriggerTaskEventOnFlip(uint64 PresentIndex, UE::Tasks::FTaskEvent Event);
};

FRHIFrameFlipTrackingRunnable::FRHIFrameFlipTrackingRunnable()
{}

#if USE_FRAME_OFFSET_THREAD
struct FRHIFrameOffsetThread : public FRunnable
{
	static FRunnableThread* Thread;
	static FRHIFrameOffsetThread Singleton;
	static bool bInitialized;
	static bool bRun;

	FCriticalSection CS;
	FRHIFlipDetails LastFlipFrame;

	static FEvent* WaitEvent;

	virtual uint32 Run() override
	{
		while (bRun)
		{
			FRHIFlipDetails NewFlipFrame = GDynamicRHI->RHIWaitForFlip(-1);

			int32 SyncInterval = RHIGetSyncInterval();
			double TargetFrameTimeInSeconds = double(SyncInterval) / double(FPlatformMisc::GetMaxRefreshRate());
			double SlackInSeconds = FMath::Min(RHIGetSyncSlackMS() / 1000.0, TargetFrameTimeInSeconds);			// Clamp slack sync time to at most one full frame interval
			double TargetFlipTime = (NewFlipFrame.VBlankTimeInSeconds + TargetFrameTimeInSeconds) - SlackInSeconds;
			
			double Timeout = FMath::Max(0.0, TargetFlipTime - FPlatformTime::ToSeconds64(FPlatformTime::Cycles64()));

			FPlatformProcess::Sleep(Timeout);

			{
				{
					FScopeLock Lock(&CS);
					LastFlipFrame = NewFlipFrame;
					LastFlipFrame.FlipTimeInSeconds = LastFlipFrame.FlipTimeInSeconds + TargetFrameTimeInSeconds - SlackInSeconds;
					LastFlipFrame.VBlankTimeInSeconds = LastFlipFrame.VBlankTimeInSeconds + TargetFrameTimeInSeconds - SlackInSeconds;
					LastFlipFrame.PresentIndex++;
				}

#if UE_INPUT_LATENCY_TRACKING
				RHIFrameInfoSetVsyncDetails(NewFlipFrame);
#endif
			}

			if (WaitEvent)
			{
				WaitEvent->Trigger();
			}
		}

		return 0;
	}

	virtual void Stop() override
	{
		bRun = false;
		GDynamicRHI->RHISignalFlipEvent();
	}

public:
	FRHIFrameOffsetThread()
	{}

	~FRHIFrameOffsetThread()
	{
	}

	static FRHIFlipDetails WaitForFlip(double Timeout)
	{
		check(Singleton.WaitEvent);
		if (Timeout >= 0)
		{
			Singleton.WaitEvent->Wait((uint32)(Timeout * 1000.0));
		}
		else
		{
			Singleton.WaitEvent->Wait();
		}

		FScopeLock Lock(&Singleton.CS);
		return Singleton.LastFlipFrame;
	}

	static void Signal()
	{
		Singleton.WaitEvent->Trigger();
	}

	static void Initialize()
	{
		bInitialized = true;
		bRun = true;
		Singleton.GetOrInitializeWaitEvent();
		check(Thread == nullptr);
		Thread = FRunnableThread::Create(&Singleton, TEXT("RHIFrameOffsetThread"), 0, TPri_AboveNormal, FPlatformAffinity::GetRHIFrameOffsetThreadMask());
	}

	static void Shutdown()
	{
		// Some platforms call shutdown before initialize has been called, so bail out if that happens
		if (!bInitialized)
		{
			return;
		}
		bInitialized = false;

		if (WaitEvent)
		{
			FPlatformProcess::ReturnSynchEventToPool(WaitEvent);
			WaitEvent = nullptr;
		}

		if (Thread)
		{
			Thread->Kill(true);
			delete Thread;
			Thread = nullptr;
		}
	}

private:
	FEvent* GetOrInitializeWaitEvent()
	{
		// Wait event can't be initialized with the singleton, because it will crash when initialized to early
		if (WaitEvent == nullptr)
		{
			WaitEvent = FPlatformProcess::GetSynchEventFromPool(false);
		}
		return WaitEvent;
	}

};

FRunnableThread* FRHIFrameOffsetThread::Thread = nullptr;
FRHIFrameOffsetThread FRHIFrameOffsetThread::Singleton;
bool FRHIFrameOffsetThread::bInitialized = false;
bool FRHIFrameOffsetThread::bRun = false;
FEvent* FRHIFrameOffsetThread::WaitEvent = nullptr;

#endif // USE_FRAME_OFFSET_THREAD

uint32 FRHIFrameFlipTrackingRunnable::Run()
{
	uint64 SyncFrame = 0;
	double SyncTime = FPlatformTime::Seconds();
	bool bForceFlipSync = true;

	if ( ! FPlatformMisc::UseRenderThread() )
	{
		return 0;
	}

	while (bRun)
	{
		// Determine the next expected flip time, based on the previous flip time we synced to.
		int32 SyncInterval = RHIGetSyncInterval();
		double TargetFrameTimeInSeconds = double(SyncInterval) / double(FPlatformMisc::GetMaxRefreshRate());
		double ExpectedNextFlipTimeInSeconds = SyncTime + (TargetFrameTimeInSeconds * 1.02); // Add 2% to prevent early timeout
		double CurrentTimeInSeconds = FPlatformTime::Seconds();

		double TimeoutInSeconds = (SyncInterval == 0 || bForceFlipSync) ? -1.0 : FMath::Max(ExpectedNextFlipTimeInSeconds - CurrentTimeInSeconds, 0.0);

#if USE_FRAME_OFFSET_THREAD
		FRHIFlipDetails FlippedFrame = FRHIFrameOffsetThread::WaitForFlip(TimeoutInSeconds);
#else
		FRHIFlipDetails FlippedFrame = GDynamicRHI->RHIWaitForFlip(TimeoutInSeconds);
#endif

		CurrentTimeInSeconds = FPlatformTime::Seconds();
		if (FlippedFrame.PresentIndex > SyncFrame)
		{
			// A new frame has flipped
			SyncFrame = FlippedFrame.PresentIndex;
			SyncTime = FlippedFrame.VBlankTimeInSeconds;
			bForceFlipSync = CVarRHISyncAllowEarlyKick.GetValueOnAnyThread() == 0;
		}
		else if (SyncInterval != 0 && !bForceFlipSync && (CurrentTimeInSeconds > ExpectedNextFlipTimeInSeconds))
		{
			// We've missed a flip. Signal the next frame
			// anyway to optimistically recover from a hitch.
			SyncFrame = FlippedFrame.PresentIndex + 1;
			SyncTime = CurrentTimeInSeconds;
		}

		bool bUpdateRHIFrameTime = false;
		// Complete any relevant task events.
		FScopeLock Lock(&CS);
		for (int32 PairIndex = FramePairs.Num() - 1; PairIndex >= 0; --PairIndex)
		{
			auto& Pair = FramePairs[PairIndex];
			if (Pair.PresentIndex <= SyncFrame)
			{
				FramePairs.RemoveAtSwap(PairIndex);
				bUpdateRHIFrameTime = true;
			}
		}

		if(bUpdateRHIFrameTime)
		{
			RHICalculateFrameTime();
		}	
	}

	return 0;
}

void FRHIFrameFlipTrackingRunnable::Stop()
{
	bRun = false;
#if USE_FRAME_OFFSET_THREAD
	FRHIFrameOffsetThread::Signal();
#else
	GDynamicRHI->RHISignalFlipEvent();
#endif
}

void FRHIFrameFlipTrackingRunnable::Initialize()
{
	if ( ! FPlatformMisc::UseRenderThread() )
	{
		return;
	}

	check(Thread == nullptr);
	bInitialized = true;
	bRun = true;
	Thread = FRunnableThread::Create(&Singleton, TEXT("RHIFrameFlipThread"), 0, TPri_AboveNormal);
}

void FRHIFrameFlipTrackingRunnable::Shutdown()
{
	if ( ! FPlatformMisc::UseRenderThread() )
	{
		return;
	}

	if (!bInitialized)
	{
		return;
	}
	bInitialized = false;
	if (Thread)
	{
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}

	FScopeLock Lock(&Singleton.CS);

	Singleton.FramePairs.Empty();

#if USE_FRAME_OFFSET_THREAD
	FRHIFrameOffsetThread::Shutdown();
#endif
}

void FRHIFrameFlipTrackingRunnable::TriggerTaskEventOnFlip(uint64 PresentIndex, UE::Tasks::FTaskEvent Event)
{
	if ( ! FPlatformMisc::UseRenderThread() )
	{
		return;
	}

	FScopeLock Lock(&Singleton.CS);

	if (Thread)
	{
		Singleton.FramePairs.Emplace(PresentIndex, MoveTemp(Event));

#if USE_FRAME_OFFSET_THREAD
		FRHIFrameOffsetThread::Signal();
#else
		GDynamicRHI->RHISignalFlipEvent();
#endif
	}
	else
	{
		// Platform does not support flip tracking.
		// Signal the event now...
		TRACE_CPUPROFILER_EVENT_SCOPE(SyncTrigger_Swapchain);
		Event.Trigger();
	}
}

FRunnableThread* FRHIFrameFlipTrackingRunnable::Thread;
FRHIFrameFlipTrackingRunnable FRHIFrameFlipTrackingRunnable::Singleton;
bool FRHIFrameFlipTrackingRunnable::bInitialized = false;
bool FRHIFrameFlipTrackingRunnable::bRun = false;

uint32 RHIGetSyncInterval()
{
	return FMath::Max(CVarRHISyncInterval.GetValueOnAnyThread(), 0);
}

float RHIGetSyncSlackMS()
{
#if USE_FRAME_OFFSET_THREAD
	const float SyncSlackMS = CVarRHISyncSlackMS.GetValueOnAnyThread();
#else // #if USE_FRAME_OFFSET_THREAD
	const float SyncSlackMS = RHIGetSyncInterval() / float(FPlatformMisc::GetMaxRefreshRate()) * 1000.f;		// Sync slack is entire frame interval if we aren't using the frame offset system
#endif // #else // #if USE_FRAME_OFFSET_THREAD
	return SyncSlackMS;
}

bool RHIGetSyncAllowVariable()
{
	return CVarRHISyncAllowVariable.GetValueOnAnyThread() != 0;
}

void RHIGetPresentThresholds(float& OutTopPercent, float& OutBottomPercent)
{
	OutTopPercent = FMath::Clamp(CVarRHIPresentThresholdTop.GetValueOnAnyThread(), 0.0f, 1.0f);
	OutBottomPercent = FMath::Clamp(CVarRHIPresentThresholdBottom.GetValueOnAnyThread(), 0.0f, 1.0f);
}

void RHITriggerTaskEventOnFlip(uint64 PresentIndex, const UE::Tasks::FTaskEvent& Event)
{
	FRHIFrameFlipTrackingRunnable::TriggerTaskEventOnFlip(PresentIndex, Event);
}


void RHIInitializeFlipTracking()
{
#if USE_FRAME_OFFSET_THREAD
	FRHIFrameOffsetThread::Initialize();
#endif
	FRHIFrameFlipTrackingRunnable::Initialize();
}

void RHICalculateFrameTime()
{
	double CurrentTimeInSeconds = FPlatformTime::Seconds();
	GRHIFrameTimeMS = (float)((CurrentTimeInSeconds - GLastRHITimeInSeconds) * 1000.0);
	GLastRHITimeInSeconds = CurrentTimeInSeconds;
}

float RHIGetFrameTime()
{
	return GRHIFrameTimeMS;
}

void RHIShutdownFlipTracking()
{
	FRHIFrameFlipTrackingRunnable::Shutdown();
#if USE_FRAME_OFFSET_THREAD
	FRHIFrameOffsetThread::Shutdown();
#endif
}

ERHIAccess RHIGetDefaultResourceState(ETextureCreateFlags InUsage, bool bInHasInitialData)
{
	// By default assume it can be bound for reading
	ERHIAccess ResourceState = ERHIAccess::SRVMask;

	if (!bInHasInitialData)
	{
		if (EnumHasAnyFlags(InUsage, TexCreate_RenderTargetable))
		{
			ResourceState = ERHIAccess::RTV;
		}
		else if (EnumHasAnyFlags(InUsage, TexCreate_DepthStencilTargetable))
		{
			ResourceState = ERHIAccess::DSVWrite | ERHIAccess::DSVRead;
		}
		else if (EnumHasAnyFlags(InUsage, TexCreate_UAV))
		{
			ResourceState = ERHIAccess::UAVMask;
		}
		else if (EnumHasAnyFlags(InUsage, TexCreate_Presentable))
		{
			ResourceState = ERHIAccess::Present;
		}
		else if (EnumHasAnyFlags(InUsage, TexCreate_ShaderResource))
		{
			ResourceState = ERHIAccess::SRVMask;
		}
		else if (EnumHasAnyFlags(InUsage, TexCreate_Foveation))
		{
			ResourceState = ERHIAccess::ShadingRateSource;
		}
	}
	
	check(ResourceState != ERHIAccess::Unknown);

	return ResourceState;
}

ERHIAccess RHIGetDefaultResourceState(EBufferUsageFlags InUsage, bool bInHasInitialData)
{
	// Default reading state is different per buffer type
	ERHIAccess DefaultReadingState = ERHIAccess::Unknown;
	if (EnumHasAnyFlags(InUsage, BUF_IndexBuffer))
	{
		DefaultReadingState = ERHIAccess::VertexOrIndexBuffer;
	}
	if (EnumHasAnyFlags(InUsage, BUF_VertexBuffer))
	{
		// Could be vertex buffer or normal DataBuffer
		DefaultReadingState = DefaultReadingState | ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask;
	}
	if (EnumHasAnyFlags(InUsage, BUF_StructuredBuffer))
	{
		DefaultReadingState = DefaultReadingState | ERHIAccess::SRVMask;
	}
	if (EnumHasAnyFlags(InUsage, BUF_AccelerationStructure))
	{
		DefaultReadingState = DefaultReadingState | ERHIAccess::BVHRead;
	}

	// Vertex and index buffers might not have the BUF_ShaderResource flag set and just assume
	// they are readable by default
	ERHIAccess ResourceState = (!EnumHasAnyFlags(DefaultReadingState, ERHIAccess::VertexOrIndexBuffer)) ? ERHIAccess::Unknown : DefaultReadingState;

	// SRV when we have initial data because we can sample the buffer then
	if (bInHasInitialData)
	{
		ResourceState = DefaultReadingState;
	}
	else
	{
		if (EnumHasAnyFlags(InUsage, BUF_UnorderedAccess))
		{
			ResourceState = ERHIAccess::UAVMask;
		}
		else if (EnumHasAnyFlags(InUsage, BUF_AccelerationStructure))
		{
			ResourceState = ERHIAccess::BVHWrite;
		}
		else if (EnumHasAnyFlags(InUsage, BUF_ShaderResource))
		{
			ResourceState = DefaultReadingState | ERHIAccess::SRVMask;
		}
	}

	check(ResourceState != ERHIAccess::Unknown);

	return ResourceState;
}

void DecodeRenderTargetMode(ESimpleRenderTargetMode Mode, ERenderTargetLoadAction& ColorLoadAction, ERenderTargetStoreAction& ColorStoreAction, ERenderTargetLoadAction& DepthLoadAction, ERenderTargetStoreAction& DepthStoreAction, ERenderTargetLoadAction& StencilLoadAction, ERenderTargetStoreAction& StencilStoreAction, FExclusiveDepthStencil DepthStencilUsage)
{
	// set defaults
	ColorStoreAction = ERenderTargetStoreAction::EStore;
	DepthStoreAction = ERenderTargetStoreAction::EStore;
	StencilStoreAction = ERenderTargetStoreAction::EStore;

	switch (Mode)
	{
	case ESimpleRenderTargetMode::EExistingColorAndDepth:
		ColorLoadAction = ERenderTargetLoadAction::ELoad;
		DepthLoadAction = ERenderTargetLoadAction::ELoad;
		break;
	case ESimpleRenderTargetMode::EUninitializedColorAndDepth:
		ColorLoadAction = ERenderTargetLoadAction::ENoAction;
		DepthLoadAction = ERenderTargetLoadAction::ENoAction;
		break;
	case ESimpleRenderTargetMode::EUninitializedColorExistingDepth:
		ColorLoadAction = ERenderTargetLoadAction::ENoAction;
		DepthLoadAction = ERenderTargetLoadAction::ELoad;
		break;
	case ESimpleRenderTargetMode::EUninitializedColorClearDepth:
		ColorLoadAction = ERenderTargetLoadAction::ENoAction;
		DepthLoadAction = ERenderTargetLoadAction::EClear;
		break;
	case ESimpleRenderTargetMode::EClearColorExistingDepth:
		ColorLoadAction = ERenderTargetLoadAction::EClear;
		DepthLoadAction = ERenderTargetLoadAction::ELoad;
		break;
	case ESimpleRenderTargetMode::EClearColorAndDepth:
		ColorLoadAction = ERenderTargetLoadAction::EClear;
		DepthLoadAction = ERenderTargetLoadAction::EClear;
		break;
	case ESimpleRenderTargetMode::EExistingContents_NoDepthStore:
		ColorLoadAction = ERenderTargetLoadAction::ELoad;
		DepthLoadAction = ERenderTargetLoadAction::ELoad;
		DepthStoreAction = ERenderTargetStoreAction::ENoAction;
		break;
	case ESimpleRenderTargetMode::EExistingColorAndClearDepth:
		ColorLoadAction = ERenderTargetLoadAction::ELoad;
		DepthLoadAction = ERenderTargetLoadAction::EClear;
		break;
	case ESimpleRenderTargetMode::EExistingColorAndDepthAndClearStencil:
		ColorLoadAction = ERenderTargetLoadAction::ELoad;
		DepthLoadAction = ERenderTargetLoadAction::ELoad;
		break;
	default:
		UE_LOGF(LogRHI, Fatal, "Using a ESimpleRenderTargetMode that wasn't decoded in DecodeRenderTargetMode [value = %d]", (int32)Mode);
	}

	StencilLoadAction = DepthLoadAction;

	if (!DepthStencilUsage.IsUsingDepth())
	{
		DepthLoadAction = ERenderTargetLoadAction::ENoAction;
	}

	//if we aren't writing to depth, there's no reason to store it back out again.  Should save some bandwidth on mobile platforms.
	if (!DepthStencilUsage.IsDepthWrite())
	{
		DepthStoreAction = ERenderTargetStoreAction::ENoAction;
	}

	if (!DepthStencilUsage.IsUsingStencil())
	{
		StencilLoadAction = ERenderTargetLoadAction::ENoAction;
	}

	//if we aren't writing to stencil, there's no reason to store it back out again.  Should save some bandwidth on mobile platforms.
	if (!DepthStencilUsage.IsStencilWrite())
	{
		StencilStoreAction = ERenderTargetStoreAction::ENoAction;
	}
}

EGpuVendorId RHIGetPreferredAdapterVendor()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("preferAMD")))
	{
		return EGpuVendorId::Amd;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("preferIntel")))
	{
		return EGpuVendorId::Intel;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("preferNvidia")))
	{
		return EGpuVendorId::Nvidia;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("preferMS")) || FParse::Param(FCommandLine::Get(), TEXT("preferMicrosoft")))
	{
		return EGpuVendorId::Microsoft;
	}

	return EGpuVendorId::Unknown;
}

void FRHIFrameInfo::SetInputSampleMarker(uint64 FrameIndex)
{
	uint64 Cycles64 = FPlatformTime::Cycles64(); 
		
	{
		UE::TUniqueLock ScopeLock(FramesMutex);

		FRHIFrameInfoMarker& FrameMarker = Frames.FindOrAdd(FrameIndex);
		FrameMarker.InputCycle = Cycles64;
		Trim();
	}	
}

void FRHIFrameInfo::SetPresentCounter(uint64 FrameIndex, uint64 PresentCounter)
{
	{
		UE::TUniqueLock ScopeLock(FramesMutex);

		FRHIFrameInfoMarker& FrameMarker = Frames.FindOrAdd(FrameIndex);
		FrameMarker.PresentCounter = PresentCounter;
		Trim();
	}
	
}


void FRHIFrameInfo::SetVsyncDetails(const FRHIFlipDetails& NewFlipFrame)
{ 
	uint64 LatestFrameCounter = 0;

	{
		UE::TUniqueLock ScopeLock(FramesMutex);

		for (TPair<uint64, FRHIFrameInfoMarker>& Elem : Frames)
		{
			if (Elem.Value.PresentCounter == NewFlipFrame.PresentIndex)
			{
				//if VBlankTimeInCycles is not set we compute it from VBlankTimeInSeconds
				uint64 VBlankTimeInCycle = (NewFlipFrame.VBlankTimeInCycles != 0) ? NewFlipFrame.VBlankTimeInCycles : ((uint64)(NewFlipFrame.VBlankTimeInSeconds / FPlatformTime::GetSecondsPerCycle64()));
				float Hz = NewFlipFrame.Hz;

				Elem.Value.VSyncCycle = VBlankTimeInCycle;
				//LatestFrameCounter = Elem.Value.FrameIndex;
				LatestFrameCounter = Elem.Key; 
			
				uint64 CurrentInputLatency = 0;

				if (VBlankTimeInCycle > Elem.Value.InputCycle)
				{
					CurrentInputLatency = VBlankTimeInCycle - Elem.Value.InputCycle;
				}

				if (CurrentInputLatency)
				{
					GInputLatencyTime = CurrentInputLatency;
				}

				Elem.Value.InputLatencyMS = FPlatformTime::ToMilliseconds64(CurrentInputLatency);

#if UE_INPUT_LATENCY_TRACKING
				if (GVsyncInformationInsights == 1)
				{
					TRACE_BOOKMARK_CYCLES(Elem.Value.InputCycle, TEXT("Input Frame: %" UINT64_FMT), LatestFrameCounter)

					if (Hz)
					{
						TRACE_BOOKMARK_CYCLES(VBlankTimeInCycle, TEXT("Vsync (%.2f Hz) Frame: %" UINT64_FMT), Hz, LatestFrameCounter)
					}
					else
					{
						TRACE_BOOKMARK_CYCLES(VBlankTimeInCycle, TEXT("Vsync Frame: %" UINT64_FMT), LatestFrameCounter)
					}
				}
				else if (GVsyncInformationInsights == 2)
				{
					TRACE_BOOKMARK_CYCLES(Elem.Value.InputCycle, TEXT("Input P:%" UINT64_FMT " F:%" UINT64_FMT), Elem.Value.PresentCounter, LatestFrameCounter)

					if (Hz)
					{
						TRACE_BOOKMARK_CYCLES(VBlankTimeInCycle, TEXT("Vsync (%.2f Hz) P:%" UINT64_FMT " F:%" UINT64_FMT " Lat:%.2f"), Hz, Elem.Value.PresentCounter, LatestFrameCounter, Elem.Value.InputLatencyMS)
					}
					else
					{
						TRACE_BOOKMARK_CYCLES(VBlankTimeInCycle, TEXT("Vsync P:%" UINT64_FMT " F:%" UINT64_FMT " Lat:%.2f"), Elem.Value.PresentCounter, LatestFrameCounter, Elem.Value.InputLatencyMS)
					}
				}
#endif
				break;
			}
		}
	}

	ComputeHistory(LatestFrameCounter);
}

void FRHIFrameInfo::ComputeHistory(uint64 FrameNumber)
{
	UE::TUniqueLock ScopeLock(FramesMutex);

	const uint64 FrameThreshold = FrameNumber < FrameInfoHistoryCount ? FrameInfoHistoryCount : FrameNumber - FrameInfoHistoryCount;

	uint32 ElementCount = 0;
	double AccumulatedInputLatency = 0.0;

	for (auto It = Frames.CreateIterator(); It; ++It)
	{
		if (It->Value.InputLatencyMS != 0.0)
		{
			AccumulatedInputLatency += It->Value.InputLatencyMS;
			ElementCount++;
		}

		if (It->Key <= FrameThreshold)
		{
			It.RemoveCurrent();
		}
	}

	if (ElementCount)
	{
		AverageInputLatencyMs = AccumulatedInputLatency / ((double)ElementCount);
	}

	Trim();
}

void FRHIFrameInfo::Trim()
{
	// Array should not grow indefinitely 
	if (Frames.Num() > (FrameInfoHistoryCount * 2))
	{
		Frames.CreateIterator().RemoveCurrent();
	}
}

#if UE_INPUT_LATENCY_TRACKING
extern RHI_API void RHIFrameInfoSetVsyncDetails(const FRHIFlipDetails& NewFlipFrame)
{
	GRHIFrameInfo.SetVsyncDetails(NewFlipFrame);
}

extern RHI_API void RHIFrameInfoSetInputSampleMarker(uint64 FrameNumber)
{
	GRHIFrameInfo.SetInputSampleMarker(FrameNumber);
}

extern RHI_API void RHIFrameInfoSetPresentCounter(uint64 FrameNumber, uint64 PresentCounter)
{
	GRHIFrameInfo.SetPresentCounter(FrameNumber, PresentCounter);
}
#endif
