// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHIValidation.cpp: Public RHI Validation layer definitions.
=============================================================================*/

#include "RHIValidation.h"
#include "RHIValidationContext.h"
#include "HAL/IConsoleManager.h"
#include "RHIValidationTransientResourceAllocator.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/CommandLine.h"
#include "Misc/OutputDeviceRedirector.h"
#include "RHIContext.h"
#include "RHIStrings.h"
#include "RHIUniformBufferUtilities.h"
#include "Algo/BinarySearch.h"
#include "Async/ParallelFor.h"
#include "RHIResources.h"

#if ENABLE_RHI_VALIDATION

bool GRHIValidationEnabled = false;

bool GRHIValidateBufferSourceCopy = true;

bool GRHIValidationPrintHumanReadableCallStack = false;

// Define the number of stack frames to capture
const int32 NumStackFrames = 30;
const uint32 IgnoreStackCount = 2; // Ignore the call to the function itself and the log

// When set to 1, callstack for each uniform buffer allocation will be tracked 
// (slow and leaks memory, but can be handy to find the location where an invalid
// allocation has been made)
#define CAPTURE_UNIFORMBUFFER_ALLOCATION_BACKTRACES 0

// When set to 1, logs resource transitions on all unnamed resources, useful for
// tracking down missing barriers when "-RHIValidationLog" cannot be used.
// Don't leave this enabled. Log backtraces are leaked.
#define LOG_UNNAMED_RESOURCES 0

namespace RHIValidation
{
	int32 GBreakOnTransitionError = 1;
	FAutoConsoleVariableRef CVarBreakOnTransitionError(
		TEXT("r.RHIValidation.DebugBreak.Transitions"),
		GBreakOnTransitionError,
		TEXT("Controls whether the debugger should break when a validation error is encountered.\n")
		TEXT(" 0: disabled;\n")
		TEXT(" 1: break in the debugger if a validation error is encountered."),
		ECVF_RenderThreadSafe);

	// Returns an array of resource names parsed from the "-RHIValidationLog" command line switch.
	// RHI validation logging is automatically enabled for resources whose debug names match those in this list.
	// Multiple values are comma separated, e.g. -RHIValidationLog="SceneDepthZ,GBufferA"
	// Use the additional -RHIValidationLogStack arg to enable printing the resolved symbols of the callstack in the log
	static TArray<FString> const& GetAutoLogResourceNames()
	{
		struct FInit
		{
			TArray<FString> Strings;

			FInit()
			{
				FString ResourceNames;
				if (FParse::Value(FCommandLine::Get(), TEXT("-RHIValidationLog="), ResourceNames, false))
				{
					FString Left, Right;
					while (ResourceNames.Split(TEXT(","), &Left, &Right))
					{
						Left.TrimStartAndEndInline();
						Strings.Add(Left);
						ResourceNames = Right;
					}

					ResourceNames.TrimStartAndEndInline();
					Strings.Add(ResourceNames);
				}
				
				GRHIValidationPrintHumanReadableCallStack = FParse::Param(FCommandLine::Get(), TEXT("RHIValidationLogStack"));
			}
		} static Init;

		return Init.Strings;
	}

	FBufferResource::FBufferResource(const FRHIBufferCreateDesc& CreateDesc)
	{
		InitBarrierTracking(CreateDesc);
	}

	void FBufferResource::InitBarrierTracking(const FRHIBufferCreateDesc& CreateDesc)
	{
		FResource::InitBarrierTracking(1, 1, 1, CreateDesc.InitialState, CreateDesc.DebugName);
	}

	FTextureResource::FTextureResource(FRHITextureCreateDesc const& CreateDesc)
		: FTextureResource()
	{
		InitBarrierTracking(CreateDesc);
	}

	void FTextureResource::InitBarrierTracking(FRHITextureCreateDesc const& CreateDesc)
	{
		InitBarrierTracking(
			CreateDesc.NumMips,
			CreateDesc.ArraySize * (CreateDesc.IsTextureCube() ? 6 : 1),
			CreateDesc.Format,
			CreateDesc.Flags,
			CreateDesc.InitialState,
			CreateDesc.DebugName);
	}

	int32 FTextureResource::GetNumPlanesFromFormat(EPixelFormat Format)
	{
		int32 NumPlanes = 1;

		// @todo: htile tracking
		if (IsStencilFormat(Format))
		{
			NumPlanes = 2; // Depth + Stencil
		}
		else
		{
			NumPlanes = 1; // Depth only
		}

		return NumPlanes;
	}

	void FTextureResource::InitBarrierTracking(int32 InNumMips, int32 InNumArraySlices, EPixelFormat PixelFormat, ETextureCreateFlags /*Flags*/, ERHIAccess InResourceState, const TCHAR* InDebugName)
	{
		FResource* Resource = GetTrackerResource();
		if (!Resource)
			return;

		Resource->InitBarrierTracking(InNumMips, InNumArraySlices, GetNumPlanesFromFormat(PixelFormat), InResourceState, InDebugName);
	}

	void FTextureResource::CheckValidationLayout(int32 InNumMips, int32 InNumArraySlices, EPixelFormat PixelFormat)
	{
		FResource* Resource = GetTrackerResource();
		check(Resource);

		check(Resource->NumMips == InNumMips);
		check(Resource->NumArraySlices == InNumArraySlices);
		check(Resource->NumPlanes == GetNumPlanesFromFormat(PixelFormat));
	}

	FResourceIdentity FTextureResource::GetViewIdentity(uint32 InMipIndex, uint32 InNumMips, uint32 InArraySlice, uint32 InNumArraySlices, uint32 InPlaneIndex, uint32 InNumPlanes)
	{
		FResource* Resource = GetTrackerResource();

		checkSlow((InMipIndex + InNumMips) <= Resource->NumMips);
		checkSlow((InArraySlice + InNumArraySlices) <= Resource->NumArraySlices);
		checkSlow((InPlaneIndex + InNumPlanes) <= Resource->NumPlanes);

		if (InNumMips == 0)
		{
			InNumMips = Resource->NumMips;
		}
		if (InNumArraySlices == 0)
		{
			InNumArraySlices = Resource->NumArraySlices;
		}
		if (InNumPlanes == 0)
		{
			InNumPlanes = Resource->NumPlanes;
		}

		FResourceIdentity Identity;
		Identity.Resource = Resource;
		Identity.SubresourceRange.MipIndex = InMipIndex;
		Identity.SubresourceRange.NumMips = InNumMips;
		Identity.SubresourceRange.ArraySlice = InArraySlice;
		Identity.SubresourceRange.NumArraySlices = InNumArraySlices;
		Identity.SubresourceRange.PlaneIndex = InPlaneIndex;
		Identity.SubresourceRange.NumPlanes = InNumPlanes;
		return Identity;
	}

	FResourceIdentity FTextureResource::GetTransitionIdentity(const FRHITransitionInfo& Info)
	{
		FResource* Resource = GetTrackerResource();

		FResourceIdentity Identity;
		Identity.Resource = Resource;

		if (Info.IsAllMips())
		{
			Identity.SubresourceRange.MipIndex = 0;
			Identity.SubresourceRange.NumMips = Resource->NumMips;
		}
		else
		{
			check(Info.MipIndex < uint32(Resource->NumMips));
			Identity.SubresourceRange.MipIndex = Info.MipIndex;
			Identity.SubresourceRange.NumMips = 1;
		}

		if (Info.IsAllArraySlices())
		{
			Identity.SubresourceRange.ArraySlice = 0;
			Identity.SubresourceRange.NumArraySlices = Resource->NumArraySlices;
		}
		else
		{
			check(Info.ArraySlice < uint32(Resource->NumArraySlices));
			Identity.SubresourceRange.ArraySlice = Info.ArraySlice;
			Identity.SubresourceRange.NumArraySlices = 1;
		}

		if (Info.IsAllPlaneSlices())
		{
			Identity.SubresourceRange.PlaneIndex = 0;
			Identity.SubresourceRange.NumPlanes = Resource->NumPlanes;
		}
		else
		{
			check(Info.PlaneSlice < uint32(Resource->NumPlanes));
			Identity.SubresourceRange.PlaneIndex = Info.PlaneSlice;
			Identity.SubresourceRange.NumPlanes = 1;
		}

		return Identity;
	}

	RHI_API FViewIdentity::FViewIdentity(FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc)
	{
		if (InViewDesc.IsBuffer())
		{
			FRHIBuffer* Buffer = static_cast<FRHIBuffer*>(InResource);
			Resource = Buffer;

			if (InViewDesc.IsUAV())
			{
				auto const Info = InViewDesc.Buffer.UAV.GetViewInfo(Buffer);
				if (ensureMsgf(!Info.bNullView, TEXT("Attempt to use a null buffer UAV.")))
				{
					SubresourceRange = Resource->GetWholeResourceRange();
					Stride = Info.StrideInBytes;
				}
			}
			else
			{
				auto const Info = InViewDesc.Buffer.SRV.GetViewInfo(Buffer);
				if (ensureMsgf(!Info.bNullView, TEXT("Attempt to use a null buffer SRV.")))
				{
					SubresourceRange = Resource->GetWholeResourceRange();
					Stride = Info.StrideInBytes;
				}
			}
		}
		else
		{
			FRHITexture* Texture = static_cast<FRHITexture*>(InResource);
			Resource = Texture->GetTrackerResource();
			
			auto GetPlaneIndex = [](ERHITexturePlane Plane)
			{
				switch (Plane)
				{
				default: checkNoEntry(); [[fallthrough]];
				case ERHITexturePlane::Primary:
				case ERHITexturePlane::PrimaryCompressed:
				case ERHITexturePlane::Depth:
					return EResourcePlane::Common;
				
				case ERHITexturePlane::Stencil:
					return EResourcePlane::Stencil;

				case ERHITexturePlane::HTile:
					return EResourcePlane::Htile;

				case ERHITexturePlane::FMask:
					return EResourcePlane::Cmask;

				case ERHITexturePlane::CMask:
					return EResourcePlane::Fmask;
				}
			};

			if (InViewDesc.IsUAV())
			{
				auto const Info = InViewDesc.Texture.UAV.GetViewInfo(Texture);

				SubresourceRange.MipIndex       = Info.MipLevel;
				SubresourceRange.NumMips        = 1;
				SubresourceRange.ArraySlice     = Info.ArrayRange.First;
				SubresourceRange.NumArraySlices = Info.ArrayRange.Num;
				SubresourceRange.PlaneIndex     = uint32(GetPlaneIndex(Info.Plane));
				SubresourceRange.NumPlanes      = 1;

				Stride = GPixelFormats[Info.Format].BlockBytes;
			}
			else
			{
				auto const Info = InViewDesc.Texture.SRV.GetViewInfo(Texture);

				SubresourceRange.MipIndex       = Info.MipRange.First;
				SubresourceRange.NumMips        = Info.MipRange.Num;
				SubresourceRange.ArraySlice     = Info.ArrayRange.First;
				SubresourceRange.NumArraySlices = Info.ArrayRange.Num;
				SubresourceRange.PlaneIndex     = uint32(GetPlaneIndex(Info.Plane));
				SubresourceRange.NumPlanes      = 1;

				Stride = GPixelFormats[Info.Format].BlockBytes;
			}
		}
	}

	void FTracker::FUAVTracker::DrawOrDispatch(FTracker* BarrierTracker, const FState& RequiredState)
	{
		// The barrier tracking expects us to call Assert() only once per unique resource.
		// However, multiple UAVs may be bound, all referencing the same resource.
		// Find the unique resources to ensure we only do the tracking once per resource.
		TArray<FResourceIdentity, TInlineAllocator<FRHIGlobals::MinGuaranteedSimultaneousUAVs>> UniqueIdentities;

		for (int32 UAVIndex = 0; UAVIndex < UAVs.Num(); ++UAVIndex)
		{
			if (UAVs[UAVIndex])
			{
				const FResourceIdentity& Identity = UAVs[UAVIndex]->GetViewIdentity();

				// Check if we've already seen this resource.
				bool bFound = false;
				for (int32 Index = 0; !bFound && Index < UniqueIdentities.Num(); ++Index)
				{
					bFound = UniqueIdentities[Index] == Identity;
				}

				if (!bFound)
				{
					RHI_VALIDATION_CONTEXT_CHECK(UniqueIdentities.Num() < GRHIGlobals.MaxSimultaneousUAVs, 
										BarrierTracker,
										*FString::Printf(TEXT("Using too many unique UAV's, Max UAV count is %d, current index %d"),
										GRHIGlobals.MaxSimultaneousUAVs,
										UniqueIdentities.Num() - 1));
					
					UniqueIdentities.Add(Identity);

					// Assert unique resources have the required state.
					BarrierTracker->AddOp(FOperation::Assert(Identity, RequiredState));
				}
			}
		}
	}
			
	FRayTracingPipelineState::FRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer)
	{		
		HitGroupShaders = Initializer.GetHitGroupTable();
		MissShaders = Initializer.GetMissTable();
		CallableShaders = Initializer.GetCallableTable();
	}

	FRHIRayTracingShader* FRayTracingPipelineState::GetShader(ERayTracingBindingType BindingType, uint32 Index) const
	{
		switch (BindingType)
		{
			case ERayTracingBindingType::HitGroup:
				return HitGroupShaders[Index];
			case ERayTracingBindingType::CallableShader:
				return CallableShaders[Index];
			case ERayTracingBindingType::MissShader:
				return MissShaders[Index];
		}

		return nullptr;
	}

	FShaderBindingTable::FShaderBindingTable(const FRayTracingShaderBindingTableInitializer& InInitializer)
	{
		LifeTime = InInitializer.Lifetime;
		ShaderBindingMode = InInitializer.ShaderBindingMode;
		HitGroupIndexingMode = InInitializer.HitGroupIndexingMode;
	}
	
	static void CollectShaderBindingTableResources(FRayTracingPipelineState* RayTracingPipelineState, FShaderBindingTable* ShaderTable, const FRayTracingLocalShaderBindings& LocalShaderBinding, ERayTracingBindingType BindingType, uint32 WorkerIndex)
	{
		FRHIRayTracingShader* Shader = RayTracingPipelineState->GetShader(BindingType, LocalShaderBinding.ShaderIndexInPipeline);
		ensure(Shader);

		struct FResourceBinder
		{
			FResourceBinder(FShaderBindingTable* InShaderTable, FRHIRayTracingShader* InShader, uint32 InWorkerIndex, uint32 InRecordIndex) :
			ShaderBindingTable(InShaderTable), RHIShader(InShader), WorkerIndex(InWorkerIndex), RecordIndex(InRecordIndex)
			{					
			}

			void SetUAV(FRHIUnorderedAccessView* UAV, uint8 Index)
			{
				if (GRHIValidationEnabled)
				{
					RHIValidation::ValidateUnorderedAccessView(RHIShader, Index, UAV);
				}
				ShaderBindingTable->AddUAV(UAV, Index, WorkerIndex);
			}

			void SetSRV(FRHIShaderResourceView* SRV, uint8 Index)
			{
				if (GRHIValidationEnabled)
				{
					RHIValidation::ValidateShaderResourceView(RHIShader, Index, SRV);
				}
				ShaderBindingTable->AddSRV(SRV->GetViewIdentity(), WorkerIndex);
			}

			void SetTexture(FRHITexture* Texture, uint8 Index)
			{
				if (GRHIValidationEnabled)
				{
					RHIValidation::ValidateShaderResourceView(RHIShader, Index, Texture);
				}				
				ShaderBindingTable->AddSRV(Texture->GetWholeResourceIdentitySRV(), WorkerIndex);
			}

			void SetResourceCollection(FRHIResourceCollection* ResourceCollection, uint8 Index)
			{
				for (const FRHIResourceCollectionMember& Member : ResourceCollection->GetMembers())
				{
					switch (Member.Type)
					{
						case FRHIResourceCollectionMember::EType::Texture:
							if (FRHITexture* Texture = static_cast<FRHITexture*>(Member.Resource))
							{
								ShaderBindingTable->AddSRV(Texture->GetWholeResourceIdentitySRV(), WorkerIndex);
							}
							break;
						case FRHIResourceCollectionMember::EType::TextureReference:
							if (FRHITextureReference* Texture = static_cast<FRHITextureReference*>(Member.Resource))
							{
								ShaderBindingTable->AddSRV(Texture->GetWholeResourceIdentitySRV(), WorkerIndex);
							}
							break;
						case FRHIResourceCollectionMember::EType::ShaderResourceView:
							if (FRHIShaderResourceView* SRV = static_cast<FRHIShaderResourceView*>(Member.Resource))
							{
								ShaderBindingTable->AddSRV(SRV->GetViewIdentity(), WorkerIndex);
							}
							break;
						default:
							break;
					}
				}
			}

			void SetSampler(FRHISamplerState* RHISampler, uint8 Index)
			{				
				// nothing to validate
			}

			FShaderBindingTable* ShaderBindingTable;
			FRHIRayTracingShader* RHIShader;
			uint32 WorkerIndex;
			uint32 RecordIndex;
		};
		FResourceBinder Binder(ShaderTable, Shader, WorkerIndex, LocalShaderBinding.RecordIndex);
		
		// Don't need to perform the state validation now because they can still change before the actual ray dispatch
		RHIValidation::FTracker* ValidationTracker = nullptr; 

		// Use rhi core function to find all the used RHI resources in the uniform buffers
		uint32 DirtyUniformBuffers = ~(0u);
		UE::RHI::Private::SetUniformBufferResourcesFromTables(Binder, *Shader, DirtyUniformBuffers, LocalShaderBinding.UniformBuffers, ValidationTracker);
	}		

	void FShaderBindingTable::SetBindingsOnShaderBindingTable(FRayTracingPipelineState* RayTracingPipelineState, uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings, ERayTracingBindingType BindingType)
	{		
		TRACE_CPUPROFILER_EVENT_SCOPE(RHIValidation-SetBindingsOnShaderBindingTable);

		// Disable tracking for persistent SBTs until per record tracking is implemented otherwise
		// it might end up with dangling SVR/UAV pointers 
		if (LifeTime != ERayTracingShaderBindingTableLifetime::Persistent)
		{
			FGraphEventArray TaskList;

			const uint32 NumWorkerThreads = FTaskGraphInterface::Get().GetNumWorkerThreads();
			const uint32 MaxTasks = FApp::ShouldUseThreadingForPerformance() ? FMath::Min<uint32>(NumWorkerThreads, MaxBindingWorkers) : 1;

			struct FTaskContext
			{
				uint32 WorkerIndex = 0;
			};

			TArray<FTaskContext, TInlineAllocator<MaxBindingWorkers>> TaskContexts;
			for (uint32 WorkerIndex = 0; WorkerIndex < MaxTasks; ++WorkerIndex)
			{
				TaskContexts.Add(FTaskContext{ WorkerIndex });
			}

			auto BindingTask = [this, RayTracingPipelineState, Bindings, BindingType](const FTaskContext& Context, int32 CurrentIndex)
				{
					const FRayTracingLocalShaderBindings& Binding = Bindings[CurrentIndex];

					// only collect shader binding data if RTPSO & hit group indexing mode
					bool bValidBinding = (Binding.BindingType == ERayTracingLocalShaderBindingType::Persistent || Binding.BindingType == ERayTracingLocalShaderBindingType::Validation);
					if (HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Allow && bValidBinding)
					{
						if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
						{
							CollectShaderBindingTableResources(RayTracingPipelineState, this, Binding, BindingType, Context.WorkerIndex);

							// Also add SRV view requirement for all index and vertex buffers used in the SBT
							const FRayTracingGeometryInitializer& BLASInitializer = Binding.Geometry->GetInitializer();
							if (BLASInitializer.IndexBuffer)
							{
								AddSRV(BLASInitializer.IndexBuffer->GetWholeResourceIdentity(), Context.WorkerIndex);
							}
							for (int32 SegmentIndex = 0; SegmentIndex < BLASInitializer.Segments.Num(); ++SegmentIndex)
							{
								AddSRV(BLASInitializer.Segments[SegmentIndex].VertexBuffer->GetWholeResourceIdentity(), Context.WorkerIndex);
							}
						}
					}
				};

			const int32 ItemsPerTask = 1024;
			ParallelForWithExistingTaskContext(TEXT("SetRayTracingBindings"), MakeArrayView(TaskContexts), NumBindings, ItemsPerTask, BindingTask);
		}
		
		// Mark dirty
		bIsDirty = true;
	}

	void FShaderBindingTable::Commit()
	{				
		// Merge all data from worker threads into the main set
		for (uint32 WorkerIndex = 1; WorkerIndex < MaxBindingWorkers; ++WorkerIndex)
		{
			for (const FResourceIdentity& ResourceIdentity : WorkerData[WorkerIndex].SRVs)
			{
				AddSRV(ResourceIdentity, 0);
			}

			for (const FUAVBinding& UAVBinding : WorkerData[WorkerIndex].UAVs)
			{
				AddUAV(UAVBinding.UAV, UAVBinding.Slot, 0);
			}

			WorkerData[WorkerIndex].SRVs.Empty();
			WorkerData[WorkerIndex].UAVs.Empty();
		}

		bIsDirty = false;
	}

	void FShaderBindingTable::ValidateStateForDispatch(RHIValidation::FTracker* Tracker) const
	{
		ensureMsgf(!bIsDirty, TEXT("RayTracing bindings have not been committed. You must call CommitRayTracingBindings first."));

		// Validate all used SRVs
		for (const FResourceIdentity& SRV : WorkerData[0].SRVs)
		{
			Tracker->Assert(SRV, ERHIAccess::SRVCompute);
		}

		// Validate all used UAVs
		for (const FUAVBinding& UAVBinding : WorkerData[0].UAVs)
		{
			Tracker->AssertUAV(UAVBinding.UAV, ERHIAccess::UAVCompute, UAVBinding.Slot);
		}
	}
}


TSet<uint32> FValidationRHI::SeenFailureHashes;
FCriticalSection FValidationRHI::SeenFailureHashesMutex;

FValidationRHI::FValidationRHI(FDynamicRHI* InRHI)
	: RHI(InRHI)
{
	check(RHI);
	UE_LOGF(LogRHI, Log, "FValidationRHI on, intercepting %ls RHI!", InRHI && InRHI->GetName() ? InRHI->GetName() : TEXT("<NULL>"));
	GRHIValidationEnabled = true;
	SeenFailureHashes.Reserve(256);
}

FValidationRHI::~FValidationRHI()
{
	GRHIValidationEnabled = false;
}

IRHITransientResourceAllocator* FValidationRHI::RHICreateTransientResourceAllocator()
{
	// Wrap around validation allocator
	if (IRHITransientResourceAllocator* RHIAllocator = RHI->RHICreateTransientResourceAllocator())
	{
		return new FValidationTransientResourceAllocator(RHIAllocator);
	}
	else
	{
		return nullptr;
	}
}

IRHICommandContext* FValidationRHI::RHIGetDefaultContext()
{
	IRHICommandContext* LowLevelContext = RHI->RHIGetDefaultContext();
	IRHICommandContext* HighLevelContext = static_cast<IRHICommandContext*>(&LowLevelContext->GetHighestLevelContext());

	if (LowLevelContext == HighLevelContext)
	{
		FValidationContext* ValidationContext = new FValidationContext(FValidationContext::EType::Default);
		ValidationContext->LinkToContext(LowLevelContext);
		HighLevelContext = ValidationContext;
	}

	return HighLevelContext;
}

struct FValidationCommandList : public IRHIPlatformCommandList
{
	const ERHIPipeline Pipeline = ERHIPipeline::None;
	TArray<RHIValidation::FOperation> CompletedOpList;
	IRHIPlatformCommandList* InnerCommandList = nullptr;

	// Used for upload command lists
	FValidationCommandList() = default;

	// Used for regular pipeline command lists
	FValidationCommandList(IRHIComputeContext* OuterContext)
		: Pipeline       (OuterContext->GetPipeline())
		, CompletedOpList(OuterContext->Tracker->Finalize())
	{}

	bool IsUpload() const
	{
		return Pipeline == ERHIPipeline::None;
	}
};

IRHIComputeContext* FValidationRHI::RHIGetCommandContext(ERHIPipeline Pipeline)
{
	IRHIComputeContext* InnerContext = RHI->RHIGetCommandContext(Pipeline);
	check(InnerContext);

	switch (Pipeline)
	{
	case ERHIPipeline::Graphics:
	{
		FValidationContext* OuterContext = new FValidationContext(FValidationContext::EType::Parallel);
		OuterContext->LinkToContext(static_cast<IRHICommandContext*>(InnerContext));
		return OuterContext;
	}

	case ERHIPipeline::AsyncCompute:
	{
		FValidationComputeContext* OuterContext = new FValidationComputeContext(FValidationComputeContext::EType::Parallel);
		OuterContext->LinkToContext(InnerContext);
		return OuterContext;
	}

	default:
		checkNoEntry();
		return nullptr;
	}
}

IRHICommandContext* FValidationRHI::RHIGetParallelCommandContext(FRHIParallelRenderPassInfo const& ParallelRenderPass)
{
	FValidationContext* OuterContext = new FValidationContext(FValidationContext::EType::Parallel);
	OuterContext->ValidateBeginRenderPass(ParallelRenderPass, ParallelRenderPass.PassName);

	IRHICommandContext* InnerContext = RHI->RHIGetParallelCommandContext(ParallelRenderPass);
	check(InnerContext);

	OuterContext->LinkToContext(InnerContext);
	return OuterContext;
}

FDynamicRHI::FRHIFinalizeContextsResult FValidationRHI::RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args)
{
	return RHI->RHIFinalizeContexts(MoveTemp(Args));
}

FDynamicRHI::FRHICloseTranslateChainResult FValidationRHI::RHICloseTranslateChain(FRHICloseTranslateChainArgs&& Args)
{
	TRHIPipelineArray<FValidationCommandList*> OuterCommandLists { InPlace, nullptr };

	// Re-combine the args so that the validation matches a normal call to RHIFinalizeContexts
	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		if (IRHIComputeContext* OuterContext = Args.Contexts[Pipeline])
		{
			// RHIFinalizeContexts makes the context available to other threads, so finalize the tracker beforehand.
			OuterCommandLists[OuterContext->GetPipeline()] = new FValidationCommandList(OuterContext);

			if (Args.bShouldFinalize)
			{
				// Extract the inner platform RHI context to forward to RHIFinalizeContexts below.
				Args.Contexts[Pipeline] = &OuterContext->GetLowestLevelContext();

				switch (OuterContext->GetPipeline())
				{
				case ERHIPipeline::Graphics:
					if (static_cast<FValidationContext*>(OuterContext)->Type == FValidationContext::EType::Parallel)
						delete OuterContext;
					break;

				case ERHIPipeline::AsyncCompute:
					if (static_cast<FValidationComputeContext*>(OuterContext)->Type == FValidationComputeContext::EType::Parallel)
						delete OuterContext;
					break;

				default:
					checkNoEntry();
					break;
				}
			}
		}
	}

	FDynamicRHI::FRHICloseTranslateChainResult Result;
	if (Args.bShouldFinalize)
	{
		Result = RHIFinalizeContexts(MoveTemp(Args));
	}

	// Wrap the returned platform command lists in the validation type.
	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		IRHIPlatformCommandList*& CommandList = Result.CommandLists[Pipeline];

		if (FValidationCommandList* OuterCommandList = OuterCommandLists[Pipeline])
		{
			check(OuterCommandList);
			OuterCommandList->InnerCommandList = CommandList;

			CommandList = OuterCommandList;
		}
		else
		{
			check(!CommandList);
		}
	}

	if (Result.UploadCommandList)
	{
		// There is currently no validation on upload contexts / command lists, but we
		// still need to wrap the type so that the casts in RHISubmitCommandLists are correct.
		FValidationCommandList* UploadCommandList = new FValidationCommandList();
		UploadCommandList->InnerCommandList = Result.UploadCommandList;
		Result.UploadCommandList = UploadCommandList;
	}

	return Result;
}

void FValidationRHI::RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args)
{
	SCOPED_NAMED_EVENT(RHIValidation_SubmitCommandLists, FColor::Purple);

	FDynamicRHI::FRHISubmitCommandListsArgs InnerArgs;
	InnerArgs.CommandLists.Reserve(Args.CommandLists.Num());

	for (IRHIPlatformCommandList* CmdList : Args.CommandLists)
	{
		FValidationCommandList* OuterCommandList = static_cast<FValidationCommandList*>(CmdList);

		// We don't currently do validation on upload command lists.
		if (!OuterCommandList->IsUpload())
		{
		#if WITH_RHI_BREADCRUMBS
			OuterCommandList->CompletedOpList.Insert(RHIValidation::FOperation::SetBreadcrumbRange(CmdList->BreadcrumbRange), 0);
		#endif

			// Replay or queue any barrier operations to validate resource barrier usage.
			RHIValidation::FTracker::SubmitValidationOps(OuterCommandList->Pipeline, MoveTemp(OuterCommandList->CompletedOpList));
		}

		if (IRHIPlatformCommandList* InnerCmdList = OuterCommandList->InnerCommandList)
		{
#if WITH_RHI_BREADCRUMBS
			// Forward the breadcrumb range and allocators
			InnerCmdList->BreadcrumbAllocators = MoveTemp(CmdList->BreadcrumbAllocators);
			InnerCmdList->BreadcrumbRange = CmdList->BreadcrumbRange;
#endif
			InnerArgs.CommandLists.Add(InnerCmdList);
		}

		delete OuterCommandList;
	}

	RHI->RHISubmitCommandLists(MoveTemp(InnerArgs));
}

void FValidationRHI::ValidatePipeline(const FGraphicsPipelineStateInitializer& PSOInitializer)
{
	{
		// Verify depth/stencil access/usage
		bool bHasDepth = IsDepthOrStencilFormat(PSOInitializer.DepthStencilTargetFormat);
		bool bHasStencil = IsStencilFormat(PSOInitializer.DepthStencilTargetFormat);
		const FDepthStencilStateInitializerRHI& Initializer = DepthStencilStates.FindChecked(PSOInitializer.DepthStencilState);
		if (bHasDepth)
		{
			if (!bHasStencil)
			{
				RHI_VALIDATION_CHECK(!Initializer.bEnableFrontFaceStencil
					&& Initializer.FrontFaceStencilTest == CF_Always
					&& Initializer.FrontFaceStencilFailStencilOp == SO_Keep
					&& Initializer.FrontFaceDepthFailStencilOp == SO_Keep
					&& Initializer.FrontFacePassStencilOp == SO_Keep
					&& !Initializer.bEnableBackFaceStencil
					&& Initializer.BackFaceStencilTest == CF_Always
					&& Initializer.BackFaceStencilFailStencilOp == SO_Keep
					&& Initializer.BackFaceDepthFailStencilOp == SO_Keep
					&& Initializer.BackFacePassStencilOp == SO_Keep, TEXT("No stencil render target set, yet PSO wants to use stencil operations!"));
/*
				RHI_VALIDATION_CHECK(PSOInitializer.StencilTargetLoadAction == ERenderTargetLoadAction::ENoAction,
					TEXT("No stencil target set, yet PSO wants to load from it!"));
				RHI_VALIDATION_CHECK(PSOInitializer.StencilTargetStoreAction == ERenderTargetStoreAction::ENoAction,
					TEXT("No stencil target set, yet PSO wants to store into it!"));
*/
			}
		}
		else
		{
			RHI_VALIDATION_CHECK(!Initializer.bEnableDepthWrite && Initializer.DepthTest == CF_Always, TEXT("No depth render target set, yet PSO wants to use depth operations!"));
			RHI_VALIDATION_CHECK(PSOInitializer.DepthTargetLoadAction == ERenderTargetLoadAction::ENoAction
				&& PSOInitializer.StencilTargetLoadAction == ERenderTargetLoadAction::ENoAction,
				TEXT("No depth/stencil target set, yet PSO wants to load from it!"));
			RHI_VALIDATION_CHECK(PSOInitializer.DepthTargetStoreAction == ERenderTargetStoreAction::ENoAction
				&& PSOInitializer.StencilTargetStoreAction == ERenderTargetStoreAction::ENoAction,
				TEXT("No depth/stencil target set, yet PSO wants to store into it!"));
		}
	}
}

void FValidationRHI::RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo)
{
	using namespace RHIValidation;

	const ERHIPipeline SrcPipelines = CreateInfo.SrcPipelines;
	const ERHIPipeline DstPipelines = CreateInfo.DstPipelines;

	TArray<FFence*> Fences;

	if (SrcPipelines != DstPipelines)
	{
		for (ERHIPipeline SrcPipe : MakeFlagsRange(SrcPipelines))
		{
			for (ERHIPipeline DstPipe : MakeFlagsRange(DstPipelines))
			{
				if (SrcPipe == DstPipe)
				{
					continue;
				}

				FFence* Fence = new FFence;
				Fence->SrcPipe = SrcPipe;
				Fence->DstPipe = DstPipe;
				Fences.Add(Fence);
			}
		}
	}

	TRHIPipelineArray<TArray<FOperation>> SignalOps, WaitOps;
	
	TArray<FOperation> AliasingOps, AliasingOverlapOps, BeginOps, EndOps;
	AliasingOverlapOps.Reserve(CreateInfo.AliasingInfos.Num());
	AliasingOps       .Reserve(CreateInfo.AliasingInfos.Num());
	BeginOps          .Reserve(CreateInfo.TransitionInfos.Num());
	EndOps            .Reserve(CreateInfo.TransitionInfos.Num());

	for (FFence* Fence : Fences)
	{
		WaitOps[Fence->DstPipe].Emplace(FOperation::Wait(Fence));
	}

	// Take a backtrace of this transition creation if any of the resources it contains have logging enabled.
	bool bDoTrace = false;

	for (const FRHITransientAliasingInfo& Info : CreateInfo.AliasingInfos)
	{
		if (!Info.Resource)
		{
			continue;
		}

		FResource* Resource = nullptr;

		if (Info.Type == FRHITransientAliasingInfo::EType::Texture)
		{
			Resource = Info.Texture->GetTrackerResource();
		}
		else
		{
			Resource = Info.Buffer;
		}

		bDoTrace |= (Resource->LoggingMode != RHIValidation::ELoggingMode::None);

		if (Info.IsAcquire())
		{
			checkf(Resource->TransientState.bTransient, TEXT("Acquiring resource %s which is not transient. Only transient resources can be acquired."), Resource->GetDebugName());

			AliasingOps.Emplace(FOperation::AcquireTransientResource(Resource, nullptr));

			for (const FRHITransientAliasingOverlap& Overlap : Info.Overlaps)
			{
				FResource* ResourceBefore = nullptr;

				if (Overlap.Type == FRHITransientAliasingOverlap::EType::Texture)
				{
					ResourceBefore = Overlap.Texture->GetTrackerResource();
				}
				else
				{
					ResourceBefore = Overlap.Buffer;
				}

				checkf(ResourceBefore, TEXT("Null resource provided as an aliasing overlap of %s"), Resource->GetDebugName());

				AliasingOverlapOps.Emplace(FOperation::AliasingOverlap(ResourceBefore, Resource, nullptr));
			}
		}
	}

	for (int32 Index = 0; Index < CreateInfo.TransitionInfos.Num(); ++Index)
	{
		const FRHITransitionInfo& Info = CreateInfo.TransitionInfos[Index];
		if (!Info.Resource)
			continue;

		RHI_VALIDATION_CHECK(Info.AccessAfter != ERHIAccess::Unknown || SrcPipelines == DstPipelines && DstPipelines != ERHIPipeline::All, TEXT("Cannot use Unknown after state when transitioning between pipelines."));

		checkf(Info.Type != FRHITransitionInfo::EType::Unknown, TEXT("FRHITransitionInfo::Type cannot be Unknown when creating a resource transition."));

		if (const FRHICommitResourceInfo* CommitInfo = Info.CommitInfo.GetPtrOrNull())
		{
			if (Info.Type == FRHITransitionInfo::EType::Buffer)
			{
				const FRHIBuffer* Buffer = Info.Buffer;
				const EBufferUsageFlags BufferUsage = Buffer->GetUsage();
				const uint32 BufferSize = Buffer->GetSize();
				RHI_VALIDATION_CHECK(EnumHasAllFlags(BufferUsage, BUF_ReservedResource), TEXT("Commit transitions can only be used with reserved resources."));
				RHI_VALIDATION_CHECK(CommitInfo->SizeInBytes <= BufferSize, TEXT("Buffer commit size request must not be larger than the size of the buffer itself, as virtual memory allocation cannot be resized."));
			}
			else
			{
				RHI_VALIDATION_CHECK(false, TEXT("Reserved resource commit is only supported for buffers"));
			}
		}

		FResourceIdentity Identity;

		switch (Info.Type)
		{
		default: checkNoEntry(); // fall through
		case FRHITransitionInfo::EType::Texture:
			Identity = Info.Texture->GetTransitionIdentity(Info);
			break;

		case FRHITransitionInfo::EType::Buffer:
			Identity = Info.Buffer->GetWholeResourceIdentity();
			break;

		case FRHITransitionInfo::EType::UAV:
			Identity = Info.UAV->GetViewIdentity();
			break;

		case FRHITransitionInfo::EType::BVH:
			Identity = Info.BVH->GetWholeResourceIdentity();
			break;
		}

		bDoTrace |= (Identity.Resource->LoggingMode != RHIValidation::ELoggingMode::None);

		FState PreviousState = FState(Info.AccessBefore, SrcPipelines);
		FState NextState = FState(Info.AccessAfter, DstPipelines);

		BeginOps.Emplace(FOperation::BeginTransitionResource(Identity, PreviousState, NextState, Info.Flags, CreateInfo.Flags, nullptr));
		EndOps.Emplace(FOperation::EndTransitionResource(Identity, PreviousState, NextState, Info.Flags, nullptr));
	}

	if (bDoTrace)
	{
		void* Backtrace = CaptureBacktrace();

		for (FOperation& Op : AliasingOps)
		{
			switch (Op.Type)
			{
			case EOpType::AcquireTransient:
				Op.Data_AcquireTransient.CreateBacktrace = Backtrace;
				break;
			}
		}

		for (FOperation& Op : AliasingOverlapOps) { Op.Data_AliasingOverlap.CreateBacktrace = Backtrace; }
		for (FOperation& Op : BeginOps          ) { Op.Data_BeginTransition.CreateBacktrace = Backtrace; }
		for (FOperation& Op : EndOps            ) { Op.Data_EndTransition  .CreateBacktrace = Backtrace; }
	}

	for (FFence* Fence : Fences)
	{
		SignalOps[Fence->SrcPipe].Emplace(FOperation::Signal(Fence));
	}

	Transition->PendingSignals          = MoveTemp(SignalOps);
	Transition->PendingWaits            = MoveTemp(WaitOps);
	Transition->PendingAliases          = MoveTemp(AliasingOps);
	Transition->PendingAliasingOverlaps = MoveTemp(AliasingOverlapOps);
	Transition->PendingOperationsBegin  = MoveTemp(BeginOps);
	Transition->PendingOperationsEnd    = MoveTemp(EndOps);

	return RHI->RHICreateTransition(Transition, CreateInfo);
}

namespace RHIValidation
{
	static inline FString GetReasonString_LockBufferInsideRenderPass(FResource* Buffer)
	{
		const TCHAR* DebugName = Buffer->GetDebugName();
		return FString::Printf(TEXT("Locking non-volatile buffers for writing inside a render pass is not allowed. Resource: \"%s\" (0x%p)."), DebugName ? DebugName : TEXT("Unnamed"), Buffer);
	}
}

void FValidationRHI::LockBufferValidate(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, EResourceLockMode LockMode)
{
	using namespace RHIValidation;

	check(GRHISupportsMultithreadedResources || RHICmdList.IsImmediate());
	check(LockMode != RLM_WriteOnly_NoOverwrite || GRHIGlobals.SupportsMapWriteNoOverwrite)

	if (RHICmdList.IsGraphics() && !EnumHasAnyFlags(Buffer->GetUsage(), BUF_Volatile) && LockMode == RLM_WriteOnly)
	{
		bool bIsInsideRenderPass;
		if (RHICmdList.IsTopOfPipe())
		{
			bIsInsideRenderPass = RHICmdList.IsInsideRenderPass();
		}
		else
		{
			FValidationContext& Ctx = static_cast<FValidationContext&>(RHICmdList.GetContext());
			bIsInsideRenderPass = Ctx.State.bInsideBeginRenderPass;
		}
		RHI_VALIDATION_CHECK(!bIsInsideRenderPass, *GetReasonString_LockBufferInsideRenderPass(Buffer));
	}
}

void* FValidationRHI::RHILockBuffer(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode)
{
	LockBufferValidate(RHICmdList, Buffer, LockMode);

	return RHI->RHILockBuffer(RHICmdList, Buffer, Offset, SizeRHI, LockMode);
}

void* FValidationRHI::RHILockBufferMGPU(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 GPUIndex, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode)
{
	LockBufferValidate(RHICmdList, Buffer, LockMode);

	return RHI->RHILockBufferMGPU(RHICmdList, Buffer, GPUIndex, Offset, SizeRHI, LockMode);
}

static void ValidateViewForBufferType(const FRHIViewDesc::FBuffer::FViewInfo& ViewInfo, FRHIBuffer* Buffer)
{
	if (ViewInfo.BufferType == FRHIViewDesc::EBufferType::Typed)
	{
		const uint64 MaxViewDimensionForTypedBuffer = GRHIGlobals.MaxViewDimensionForTypedBuffer;
		RHI_VALIDATION_CHECK(uint64(ViewInfo.NumElements) <= MaxViewDimensionForTypedBuffer, *FString::Printf(TEXT("Creating a View with Buffer Type = %s , BuferName(Pointer) = %s(%p) with Number of elements = %d which is greater than the Max Number of elements for this Type: %llu"), FRHIViewDesc::GetBufferTypeString(ViewInfo.BufferType), *Buffer->GetName().ToString(), Buffer, ViewInfo.NumElements, MaxViewDimensionForTypedBuffer));
	}
	else
	{
		const uint64 MaxViewSizeBytesForNonTypedBuffer = GRHIGlobals.MaxViewSizeBytesForNonTypedBuffer;
		RHI_VALIDATION_CHECK(uint64(ViewInfo.SizeInBytes) <= MaxViewSizeBytesForNonTypedBuffer, *FString::Printf(TEXT("Creating a View  with Buffer Type = %s , BuferName(Pointer) = %s(%p) with Size = %d which is greater than the Max Size for this Type: %llu"), FRHIViewDesc::GetBufferTypeString(ViewInfo.BufferType), *Buffer->GetName().ToString(), Buffer, ViewInfo.SizeInBytes, MaxViewSizeBytesForNonTypedBuffer));
	}
} 

FShaderResourceViewRHIRef FValidationRHI::RHICreateShaderResourceView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
	if (ViewDesc.IsBuffer())
	{
		FRHIBuffer* Buffer = static_cast<FRHIBuffer*>(Resource);
		const FRHIViewDesc::FBufferSRV::FViewInfo Info = ViewDesc.Buffer.SRV.GetViewInfo(Buffer);
		ValidateViewForBufferType(Info, Buffer);
	}
	return RHI->RHICreateShaderResourceView(RHICmdList, Resource, ViewDesc);
}

FUnorderedAccessViewRHIRef FValidationRHI::RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
	if (ViewDesc.IsBuffer())
	{
		FRHIBuffer* Buffer = static_cast<FRHIBuffer*>(Resource);
		const FRHIViewDesc::FBufferUAV::FViewInfo Info = ViewDesc.Buffer.UAV.GetViewInfo(Buffer);
		ValidateViewForBufferType(Info, Buffer);
	}
	return RHI->RHICreateUnorderedAccessView(RHICmdList, Resource, ViewDesc);
}

FRHILockTextureResult FValidationRHI::RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	FRHITexture* Texture = Arguments.Texture;
	const FRHITextureDesc& Desc = Arguments.Texture->GetDesc();

	RHI_VALIDATION_CHECK(Arguments.MipIndex < Desc.NumMips, TEXT("Out of bounds MipIndex"));

	switch (Desc.Dimension)
	{
	case ETextureDimension::Texture2D:
		RHI_VALIDATION_CHECK(Arguments.ArrayIndex == 0, TEXT("Texture2D locks do not support array indexing"));
		RHI_VALIDATION_CHECK(Arguments.FaceIndex == 0, TEXT("Texture2D locks do not support face indexing"));
		break;
	case ETextureDimension::Texture2DArray:
		RHI_VALIDATION_CHECK(Arguments.ArrayIndex < Desc.ArraySize, TEXT("Texture2DArray lock out of bounds ArrayIndex"));
		RHI_VALIDATION_CHECK(Arguments.FaceIndex == 0, TEXT("Texture2DArray locks do not support face indexing"));
		break;
	case ETextureDimension::Texture3D:
		RHI_VALIDATION_CHECK(false, TEXT("Texture3D locks have not been fully tested"));
		RHI_VALIDATION_CHECK(Arguments.FaceIndex == 0, TEXT("Texture3D locks do not support face indexing"));
		break;
	case ETextureDimension::TextureCube:
		RHI_VALIDATION_CHECK(Arguments.ArrayIndex == 0, TEXT("TextureCube locks do not support array indexing"));
		RHI_VALIDATION_CHECK(Arguments.FaceIndex < 6, TEXT("TextureCube lock out of bounds FaceIndex"));
		break;
	case ETextureDimension::TextureCubeArray:
		RHI_VALIDATION_CHECK(Arguments.ArrayIndex < Desc.ArraySize, TEXT("Out of bounds ArrayIndex"));
		RHI_VALIDATION_CHECK(Arguments.FaceIndex < 6, TEXT("TextureCubeArray lock out of bounds Face Index"));
		break;
	default:
		checkNoEntry();
		break;
	}

	return RHI->RHILockTexture(RHICmdList, Arguments);
}

void FValidationRHI::RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	return RHI->RHIUnlockTexture(RHICmdList, Arguments);
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
inline ERHIDescriptorType GetDescriptorTypeFromDescriptorResource(const FRHIDescriptorResource& InResource)
{
	if (InResource.Resource)
	{
		switch (InResource.Type)
		{
		case FRHIDescriptorResource::EType::None:
			return ERHIDescriptorType::Invalid;
		case FRHIDescriptorResource::EType::Texture:
			return static_cast<const FRHITexture*>(InResource.Resource)->GetDefaultBindlessHandle().GetType();
		case FRHIDescriptorResource::EType::ResourceView:
			return static_cast<const FRHIShaderResourceView*>(InResource.Resource)->GetBindlessHandle().GetType();
		case FRHIDescriptorResource::EType::UnorderedAccessView:
			return static_cast<const FRHIUnorderedAccessView*>(InResource.Resource)->GetBindlessHandle().GetType();
		}
	}

	return ERHIDescriptorType::Invalid;
}

inline bool ValidateDescriptorGroups(ERHIDescriptorTypeMask TypeMask)
{
	for (ERHIDescriptorTypeMask ValidMaskGroup : GRHIGlobals.DescriptorTypeGroups)
	{
		// If we only have a subset of the flags, then we have flags used by different groups, which is invalid.
		if (!EnumHasAllFlags(ValidMaskGroup, TypeMask) && EnumHasAnyFlags(ValidMaskGroup, TypeMask))
		{
			return false;
		}
	}

	return true;
}

inline FString GetAllDescriptorTypeGroupsAsString()
{
	TStringBuilder<128> Result;
	for (ERHIDescriptorTypeMask GroupMask : GRHIGlobals.DescriptorTypeGroups)
	{
		if (Result.Len())
		{
			Result.Append(TEXT(", "));
		}
		Result.Append(GetDescriptorMaskString(GroupMask));
	}
	return FString(Result);
}

void FValidationRHI::ValidateDescriptorRangeResources(const FRHIDescriptorRangeDesc& RangeDesc, TConstArrayView<FRHIDescriptorResource> Resources)
{
	RHI_VALIDATION_CHECK(ValidateDescriptorGroups(RangeDesc.TypeMask),
		*FString::Printf(TEXT("Invalid descriptor type mask \"%s\" overlaps type groups: %s"), *GetDescriptorMaskString(RangeDesc.TypeMask), *GetAllDescriptorTypeGroupsAsString()));

	RHI_VALIDATION_CHECK(Resources.Num() <= (int32)RangeDesc.Count,
		*FString::Printf(TEXT("Resources count (%d) exceeds descriptor range capacity (%d)."), Resources.Num(), (int32)RangeDesc.Count));

	for (const FRHIDescriptorResource& Resource : Resources)
	{
		RHI_VALIDATION_CHECK(Resource.Type == FRHIDescriptorResource::EType::None || Resource.Resource != nullptr,
			TEXT("Descriptor resource has a type set but a null resource pointer."));

		if (Resource.Resource && Resource.Type != FRHIDescriptorResource::EType::None)
		{
			const ERHIDescriptorType DescriptorType = GetDescriptorTypeFromDescriptorResource(Resource);

			RHI_VALIDATION_CHECK(DescriptorType != ERHIDescriptorType::Invalid, TEXT("Invalid descriptor type found."));
			RHI_VALIDATION_CHECK(EnumHasAllFlags(RangeDesc.TypeMask, RHIDescriptorTypeMaskFromType(DescriptorType)),
				*FString::Printf(TEXT("Unexpected descriptor type found: %s With expected type mask: %s"), GetDescriptorTypeString(DescriptorType), *GetDescriptorMaskString(RangeDesc.TypeMask)));
		}
	}
}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

class FRHIValidationQueueScope
{
	RHIValidation::FOpQueueState* Prev;

public:
	FRHIValidationQueueScope(RHIValidation::FOpQueueState& Queue)
		: Prev(ActiveQueue)
	{
		ActiveQueue = &Queue;
	}

	~FRHIValidationQueueScope()
	{
		ActiveQueue = Prev;
	}

	thread_local static RHIValidation::FOpQueueState* ActiveQueue;
};

thread_local RHIValidation::FOpQueueState* FRHIValidationQueueScope::ActiveQueue = nullptr;

static FString GetBreadcrumbPath()
{
#if WITH_RHI_BREADCRUMBS
	RHIValidation::FOpQueueState* Queue = FRHIValidationQueueScope::ActiveQueue;
	if (Queue && Queue->Breadcrumbs.Current)
	{
		return Queue->Breadcrumbs.Current->GetFullPath();
	}
	
	return {};

#else

	return TEXT("<breadcrumbs not enabled>");

#endif
	
}

// FlushType: Thread safe
void FValidationRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const TCHAR* Name)
{
	RHICmdList.EnqueueLambda([Texture, Name = FString(Name)](FRHICommandListBase& RHICmdList)
	{
		((FValidationContext&)RHICmdList.GetComputeContext()).Tracker->Rename(Texture->GetTrackerResource(), *Name);
	});

	RHI->RHIBindDebugLabelName(RHICmdList, Texture, Name);
}

void FValidationRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, const TCHAR* Name)
{
	RHICmdList.EnqueueLambda([Buffer, Name = FString(Name)](FRHICommandListBase& RHICmdList)
	{
		((FValidationContext&)RHICmdList.GetComputeContext()).Tracker->Rename(Buffer, *Name);
	});

	RHI->RHIBindDebugLabelName(RHICmdList, Buffer, Name);
}

void FValidationRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIUnorderedAccessView* UnorderedAccessViewRHI, const TCHAR* Name)
{
	RHIValidation::FResource* Resource = UnorderedAccessViewRHI->GetViewIdentity().Resource;
	RHICmdList.EnqueueLambda([Resource, Name = FString(Name)](FRHICommandListBase& RHICmdList)
	{
		((FValidationContext&)RHICmdList.GetComputeContext()).Tracker->Rename(Resource, *Name);
	});

	RHI->RHIBindDebugLabelName(RHICmdList, UnorderedAccessViewRHI, Name);
}

void FValidationRHI::ReportValidationFailure(const TCHAR* InMessage, RHIValidation::FTracker* Tracker)
{
	ReportValidationFailure(InMessage, Tracker->GetExecutingCmdList());
}

void FValidationRHI::ReportValidationFailure(const TCHAR* InMessage, FValidationComputeContext* Context)
{
	ReportValidationFailure(InMessage, &Context->RHIContext->GetExecutingCommandList());
}

void FValidationRHI::ReportValidationFailure(const TCHAR* InMessage, FValidationContext* Context)
{
	ReportValidationFailure(InMessage, &Context->RHIContext->GetExecutingCommandList());
}

void FValidationRHI::ReportValidationFailure(const TCHAR* InMessage, FRHICommandListBase* CmdList)
{
	// Report failures only once per session, since many of them will happen repeatedly. This is similar to what ensure() does, but
	// ensure() looks at the source location to determine if it's seen the error before. We want to look at the actual message, since
	// all failures of a given kind will come from the same place, but (hopefully) the error message contains the name of the resource
	// and a description of the state, so it should be unique for each failure.
	uint32 Hash = FCrc::StrCrc32<TCHAR>(InMessage);
	
	SeenFailureHashesMutex.Lock();
	bool bIsAlreadyInSet;
	SeenFailureHashes.Add(Hash, &bIsAlreadyInSet);
	SeenFailureHashesMutex.Unlock();

	if (bIsAlreadyInSet)
	{
		return;
	}

	FString Message;
	FString BreadcrumbPath;
	
	if (CmdList)
	{
		FRHIBreadcrumbNode* BreadcrumbNode = CmdList->GetCurrentBreadcrumbRef();
		BreadcrumbPath = BreadcrumbNode ? BreadcrumbNode->GetFullPath() : TEXT("NoBreadcrumb");
	}
	else 
	{
		BreadcrumbPath = GetBreadcrumbPath();
	}
	
	if(!BreadcrumbPath.IsEmpty())
	{
		Message = FString::Printf(
					  TEXT("%s\n")
					  TEXT("Breadcrumbs: %s\n")
					  TEXT("--------------------------------------------------------------------\n"),
					  InMessage, *BreadcrumbPath);
	}
	
	if(Message.IsEmpty())
	{
		Message = InMessage;
	}

	UE_LOGF(LogRHI, Error, "%ls", *Message);

	if (FPlatformMisc::IsDebuggerPresent() && RHIValidation::GBreakOnTransitionError)
	{
		// Print the message again using the debug output function, because UE_LOG doesn't always reach
		// the VS output window before the breakpoint is triggered, despite the log flush call below.
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s\n"), *Message);
		GLog->Flush();
		PLATFORM_BREAK();
	}
}

static void ValidateBoundUniformBuffers(FRHIShader* Shader, const RHIValidation::FStaticUniformBuffers& StaticUniformBuffers, const RHIValidation::FStageBoundUniformBuffers& BoundUniformBuffers, FRHICommandListBase* InCmdList)
{
	const TCHAR* FreqName = GetShaderFrequencyString(Shader->GetFrequency(), false);
	const TArray<uint32>& LayoutHashes = Shader->GetShaderResourceTable().ResourceTableLayoutHashes;

	const TArray<FUniformBufferStaticSlot>& StaticSlots = Shader->GetStaticSlots();
	if (LayoutHashes.Num() != StaticSlots.Num())
	{
		RHI_VALIDATION_CONTEXT_CHECK(false, InCmdList, *FString::Printf(TEXT("Shader %s(%s): The number of layout hashes (%d) is different from the number of static slots (%d)."), Shader->GetShaderName(), FreqName, LayoutHashes.Num(), StaticSlots.Num()));
		return;
	}

	for (int32 BindIndex = 0; BindIndex < LayoutHashes.Num(); ++BindIndex)
	{
		uint32 ExpectedLayoutHash = LayoutHashes[BindIndex];
		if (ExpectedLayoutHash == 0)
		{
			continue;
		}

		FRHIUniformBuffer* BoundBuffer = nullptr;
		bool bIsStatic = false;

		const FUniformBufferStaticSlot StaticSlot = StaticSlots[BindIndex];
		if (IsUniformBufferStaticSlotValid(StaticSlot) && StaticSlot < StaticUniformBuffers.Bindings.Num())
		{
			BoundBuffer = StaticUniformBuffers.Bindings[StaticSlot];
			if (BoundBuffer)
			{
				bIsStatic = true;
			}
		}

		if (BoundBuffer == nullptr && BindIndex < BoundUniformBuffers.Buffers.Num())
		{
			BoundBuffer = BoundUniformBuffers.Buffers[BindIndex];
		}

		if (BoundBuffer != nullptr)
		{
			const FRHIUniformBufferLayout& Layout = BoundBuffer->GetLayout();
			uint32 UniformBufferHash = Layout.GetHash();
			RHI_VALIDATION_CONTEXT_CHECK(UniformBufferHash == ExpectedLayoutHash, InCmdList, *FString::Printf(TEXT("Shader %s(%s): Invalid layout hash %u for uniform buffer \"%s\" at bind index %d (static: %s). Expecting a buffer called \"%s\", hash %u.)"),
				Shader->GetShaderName(), FreqName, UniformBufferHash, *Layout.GetDebugName(), BindIndex, bIsStatic ? TEXT("yes") : TEXT("no"), *Shader->GetUniformBufferName(BindIndex), ExpectedLayoutHash));
		}
		else
		{
			RHI_VALIDATION_CONTEXT_CHECK(false, InCmdList, *FString::Printf(TEXT("Shader %s(%s): missing uniform buffer \"%s\" at index %d."),
				Shader->GetShaderName(), FreqName , *Shader->GetUniformBufferName(BindIndex), BindIndex));
		}
	}
}

FValidationComputeContext::FValidationComputeContext(EType InType)
	: Type(InType)
{
	State.Reset();
	Tracker = &State.TrackerInstance;
}

void FValidationComputeContext::ValidateDispatch()
{
	if (State.BoundShader == nullptr)
	{
		RHI_VALIDATION_CONTEXT_CHECK(false, this, TEXT("A compute PSO has to be set before dispatching a compute shader."));
		return;
	}

	ValidateBoundUniformBuffers(State.BoundShader, State.StaticUniformBuffers, State.BoundUniformBuffers, &GetExecutingCommandList());
}

void FValidationComputeContext::FState::Reset()
{
	ComputePassName.Reset();
	BoundShader = nullptr;
	TrackerInstance.ResetAllUAVState();
	StaticUniformBuffers.Reset();
	BoundUniformBuffers.Reset();
}

FValidationContext::FValidationContext(EType InType)
	: Type(InType)
{
	State.Reset();
	Tracker = &State.TrackerInstance;
}

void FValidationRHI::RHIEndFrame_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	RenderThreadFrameID++;
	RHI->RHIEndFrame_RenderThread(RHICmdList);
}

void FValidationRHI::RHIEndFrame(const FRHIEndFrameArgs& Args)
{
	RHIThreadFrameID++;
	RHI->RHIEndFrame(Args);
}

namespace RHIValidation
{
	static inline FString GetReasonString_SourceCopyFlagMissing(FRHIBuffer* Buffer)
	{
		return FString::Printf(TEXT("Buffers used as copy source need to be created with BUF_SourceCopy! Resource: \"%s\" (0x%p)."), 
			(Buffer->GetName().GetStringLength() > 0) ? *Buffer->GetName().ToString() : TEXT("Unnamed"), Buffer);
	}
}

void FValidationContext::RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* DestinationStagingBufferRHI, uint32 InOffset, uint32 InNumBytes)
{
	using namespace RHIValidation;
	Tracker->Assert(SourceBufferRHI->GetWholeResourceIdentity(), ERHIAccess::CopySrc);
	if (GRHIValidateBufferSourceCopy)
	{
		RHI_VALIDATION_CONTEXT_CHECK(EnumHasAnyFlags(SourceBufferRHI->GetUsage(), BUF_SourceCopy), this, *GetReasonString_SourceCopyFlagMissing(SourceBufferRHI));
	}
	RHIContext->RHICopyToStagingBuffer(SourceBufferRHI, DestinationStagingBufferRHI, InOffset, InNumBytes);
}

void FValidationComputeContext::RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* DestinationStagingBufferRHI, uint32 InOffset, uint32 InNumBytes)
{
	using namespace RHIValidation;
	Tracker->Assert(SourceBufferRHI->GetWholeResourceIdentity(), ERHIAccess::CopySrc);
	if (GRHIValidateBufferSourceCopy)
	{
		RHI_VALIDATION_CONTEXT_CHECK(EnumHasAnyFlags(SourceBufferRHI->GetUsage(), BUF_SourceCopy), this, *GetReasonString_SourceCopyFlagMissing(SourceBufferRHI));
	}
	RHIContext->RHICopyToStagingBuffer(SourceBufferRHI, DestinationStagingBufferRHI, InOffset, InNumBytes);
}

void FValidationContext::ValidateDispatch()
{
	if (State.BoundShaders[SF_Compute] == nullptr)
	{
		RHI_VALIDATION_CONTEXT_CHECK(false, this, TEXT("A compute PSO has to be set before dispatching a compute shader."));
		return;
	}

	ValidateBoundUniformBuffers(State.BoundShaders[SF_Compute], State.StaticUniformBuffers, State.BoundUniformBuffers.Get(SF_Compute), &RHIContext->GetExecutingCommandList());
}

void FValidationContext::ValidateDrawing()
{
	if (!State.bGfxPSOSet)
	{
		RHI_VALIDATION_CONTEXT_CHECK(false, this, TEXT("A graphics PSO has to be set in order to be able to draw!"));
		return;
	}

	for (int32 FrequencyIndex = 0; FrequencyIndex < SF_NumFrequencies; ++FrequencyIndex)
	{
		EShaderFrequency Frequency = (EShaderFrequency)FrequencyIndex;
		if (IsValidGraphicsFrequency(Frequency) && State.BoundShaders[Frequency])
		{
			ValidateBoundUniformBuffers(State.BoundShaders[Frequency], State.StaticUniformBuffers, State.BoundUniformBuffers.Get(Frequency), & GetExecutingCommandList());
		}
	}
}

void FValidationContext::FState::Reset()
{
	bInsideBeginRenderPass = false;
	bGfxPSOSet = false;
	RenderPassName.Reset();
	PreviousRenderPassName.Reset();
	ComputePassName.Reset();
	FMemory::Memset(BoundShaders, 0);
	TrackerInstance.ResetAllUAVState();
	StaticUniformBuffers.Reset();
	BoundUniformBuffers.Reset();
}

namespace RHIValidation
{
	void FStaticUniformBuffers::Reset()
	{
		Bindings.Reset();
		check(!bInSetPipelineStateCall);
	}

	void FStaticUniformBuffers::ValidateSetShaderUniformBuffer(FRHIUniformBuffer* UniformBuffer)
	{
		check(UniformBuffer);
		UniformBuffer->ValidateLifeTime();

		// Skip validating global uniform buffers that are set internally by the RHI as part of the pipeline state.
		if (bInSetPipelineStateCall)
		{
			return;
		}

		const FRHIUniformBufferLayout& Layout = UniformBuffer->GetLayout();

		checkf(EnumHasAnyFlags(Layout.BindingFlags, EUniformBufferBindingFlags::Shader), TEXT("Uniform buffer '%s' does not have the 'Shader' binding flag."), *Layout.GetDebugName());

		if (Layout.StaticSlot < Bindings.Num())
		{
			check(Layout.BindingFlags == EUniformBufferBindingFlags::StaticAndShader);

			ensureMsgf(Bindings[Layout.StaticSlot] == nullptr,
				TEXT("Uniform buffer '%s' was bound statically and is now being bound on a specific RHI shader. Only one binding model should be used at a time."),
				*Layout.GetDebugName());
		}
	}

	FStageBoundUniformBuffers::FStageBoundUniformBuffers()
	{
		Buffers.Reserve(32);
	}

	void FStageBoundUniformBuffers::Reset()
	{
		Buffers.SetNum(0);
	}

	void FStageBoundUniformBuffers::Bind(uint32 Index, FRHIUniformBuffer* UniformBuffer)
	{
		if (Index >= (uint32)Buffers.Num())
		{
			Buffers.AddZeroed(Index + 1 - Buffers.Num());
		}

		Buffers[Index] = UniformBuffer;
	}

	void FBoundUniformBuffers::Reset()
	{
		for (FStageBoundUniformBuffers& Stage : StageBindings)
		{
			Stage.Reset();
		}
	}

	ERHIAccess DecayResourceAccess(ERHIAccess AccessMask, ERHIAccess RequiredAccess, bool bAllowUAVOverlap)
	{
		using T = __underlying_type(ERHIAccess);
		checkf(RequiredAccess == ERHIAccess::SRVGraphics || (T(RequiredAccess) & (T(RequiredAccess) - 1)) == 0, TEXT("Only one required access bit may be set at once."));
		
		if (EnumHasAnyFlags(RequiredAccess, ERHIAccess::UAVMask | ERHIAccess::BVHWrite))
		{
			// UAV writes decay to no allowed resource access when overlaps are disabled. A barrier is always required after the dispatch/draw.
			// Otherwise keep the same accessmask and don't touch or decay the state
			return !bAllowUAVOverlap ? ERHIAccess::None : AccessMask;
		}

		// Handle DSV modes
		if (EnumHasAnyFlags(RequiredAccess, ERHIAccess::DSVWrite))
		{
			constexpr ERHIAccess CompatibleStates =
				ERHIAccess::DSVRead |
				ERHIAccess::DSVWrite;

			return AccessMask & CompatibleStates;
		}
		if (EnumHasAnyFlags(RequiredAccess, ERHIAccess::DSVRead))
		{
			constexpr ERHIAccess CompatibleStates =
				ERHIAccess::DSVRead |
				ERHIAccess::DSVWrite |
				ERHIAccess::SRVGraphics |
				ERHIAccess::SRVCompute |
				ERHIAccess::CopySrc;

			return AccessMask & CompatibleStates;
		}

		if (EnumHasAnyFlags(RequiredAccess, ERHIAccess::WritableMask))
		{
			// Decay to only 1 allowed state for all other writable states.
			return RequiredAccess;
		}

		// Else, the state is readable. All readable states are compatible.
		return AccessMask;
	}

#define BARRIER_TRACKER_LOG_PREFIX_REASON(ReasonString) TEXT("RHI validation failed: " ReasonString ":\n\n")\
	TEXT("--------------------------------------------------------------------\n")\
	TEXT("              RHI Resource Transition Validation Error              \n")\
	TEXT("--------------------------------------------------------------------\n")\
	TEXT("\n\n")
	
// Warning: this prefix expects a string argument for the resource name, make sure you add it.
#define BARRIER_TRACKER_LOG_PREFIX_RESNAME TEXT("RHI validation failed for resource: %s:\n\n")\
	TEXT("--------------------------------------------------------------------\n")\
	TEXT("              RHI Resource Transition Validation Error              \n")\
	TEXT("--------------------------------------------------------------------\n")\
	TEXT("\n\n")

#define BARRIER_TRACKER_LOG_SUFFIX TEXT("\n\n")\
	TEXT("--------------------------------------------------------------------\n")\
	TEXT("\n\n")

#define BARRIER_TRACKER_LOG_ENABLE_TRANSITION_BACKTRACE \
	TEXT("    --- Enable barrier logging for this resource to see a callstack backtrace for the RHIBeginTransitions() call ") \
	TEXT("which has not been completed. Use -RHIValidationLog=X,Y,Z to enable backtrace logging for individual resources.\n\n")

	static inline FString GetResourceDebugName(FResource const* Resource, FSubresourceIndex const& SubresourceIndex)
	{
		const TCHAR* DebugName = Resource->GetDebugName();
		if (!DebugName)
		{
			DebugName = TEXT("Unnamed");
		}

		if (SubresourceIndex.IsWholeResource())
		{
			return FString::Printf(
				TEXT("\"%s\" (0x%p) (Whole Resource)"),
				DebugName,
				Resource);
		}
		else
		{
			return FString::Printf(
				TEXT("\"%s\" (0x%p) (Mip %d, Slice %d, Plane %d)"),
				DebugName,
				Resource, 
				SubresourceIndex.MipIndex,
				SubresourceIndex.ArraySlice,
				SubresourceIndex.PlaneIndex);
		}
	}

	static inline FString GetReasonString_MissingBarrier(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex, 
		const FState& CurrentState,
		const FState& RequiredState)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to access resource %s from a hardware unit it is not currently accessible from. A resource transition is required.\n\n")
			TEXT("    --- Allowed access states for this resource are: %s\n")
			TEXT("    --- Required access states are:                  %s\n")
			TEXT("    --- Allowed pipelines for this resource are:     %s\n")
			TEXT("    --- Required pipelines are:                      %s\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetRHIAccessName(CurrentState.Access),
			*GetRHIAccessName(RequiredState.Access),
			*GetRHIPipelineName(CurrentState.Pipelines),
			*GetRHIPipelineName(RequiredState.Pipelines));
	}

	static inline FString GetReasonString_IncorrectSetTrackedAccess(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		const FState& CurrentState,
		const FState& TrackedState)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to assign resource %s a tracked access that does not match its validation tracked access.\n\n")
			TEXT("    --- Actual access states:                    %s\n")
			TEXT("    --- Actual pipelines:                        %s\n")
			TEXT("    --- Assigned access states:                  %s\n")
			TEXT("    --- Assigned pipelines:                      %s\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetRHIAccessName(CurrentState.Access),
			*GetRHIPipelineName(CurrentState.Pipelines),
			*GetRHIAccessName(TrackedState.Access),
			*GetRHIPipelineName(TrackedState.Pipelines));
	}
	
	static inline FString GetReasonString_IncorrectGetTrackedAccess(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		const FState& CurrentState,
		const FState& TrackedState)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to resolve ERHIAccess::Unknown for resource %s but its tracked access that does match its validation tracked access.\n\n")
			TEXT("    --- Validation actual access states:        %s\n")
			TEXT("    --- Validation actual pipelines:            %s\n")
			TEXT("    --- Tracked access states:                  %s\n")
			TEXT("    --- Tracked pipelines:                      %s\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetRHIAccessName(CurrentState.Access),
			*GetRHIPipelineName(CurrentState.Pipelines),
			*GetRHIAccessName(TrackedState.Access),
			*GetRHIPipelineName(TrackedState.Pipelines));
	}

	static FString ResolveAndFormatCallstack (uint64* Trace)
	{
		FString Callstack;
		ANSICHAR Buffer[1024];

		for (uint32 Idx = IgnoreStackCount; Idx < NumStackFrames; Idx++)
		{
			Buffer[0] = '\0'; // Clear the buffer
			const ANSICHAR* TrimmedBuffer = Buffer;

			// Resolve the program counter to a human-readable string
			const bool bFoundSymbol = FPlatformStackWalk::ProgramCounterToHumanReadableString(
				Idx,
				Trace[Idx],
				Buffer,
				sizeof(Buffer)
			);

			if (bFoundSymbol)
			{
				// Find the "!" delimiter to trim the module and address
				const ANSICHAR* BufferAfterModuleAndAddress = FCStringAnsi::Strstr(Buffer, "!");
				if (BufferAfterModuleAndAddress)
				{
					TrimmedBuffer = BufferAfterModuleAndAddress + 1;
				}
			}

			if (FCStringAnsi::Strstr(Buffer, "UnknownFunction") == nullptr)
			{
				// Append the trimmed symbol information to the call stack string
				Callstack += FString(TrimmedBuffer) + TEXT("\r\n");
			}
		}

		return Callstack;
	}


	static inline FString GetReasonString_BeginBacktrace(void* CreateTrace, void* BeginTrace)
	{
		if (CreateTrace || BeginTrace)
		{
			if (GRHIValidationPrintHumanReadableCallStack)
			{
				return FString::Printf(
					TEXT("    --- Callstack backtraces for the transition which has not been completed:\n")
					TEXT("        RHICreateTransition: %s\n")
					TEXT("        RHIBeginTransitions: %s\n"),
					*ResolveAndFormatCallstack((uint64*)CreateTrace),
					*ResolveAndFormatCallstack((uint64*)BeginTrace));

			}
			else
			{
				return FString::Printf(
				TEXT("    --- Callstack backtraces for the transition which has not been completed (resolve in the Watch window):\n")
				TEXT("        RHICreateTransition: (void**)0x%p,32\n")
				TEXT("        RHIBeginTransitions: (void**)0x%p,32\n"),
				CreateTrace,
				BeginTrace);
			}
		}
		else
		{
			return BARRIER_TRACKER_LOG_ENABLE_TRANSITION_BACKTRACE;
		}
	}

	static inline FString GetReasonString_Backtrace(const TCHAR* OperationPrefix, const TCHAR* TracePrefix, void* Trace)
	{
		if (Trace)
		{
			if (GRHIValidationPrintHumanReadableCallStack)
			{
				return FString::Printf(
					TEXT("    --- Callstack backtrace for %s operation:\n")
					TEXT("        %s: %s\n"),
					OperationPrefix,
					TracePrefix,
					*ResolveAndFormatCallstack((uint64*)Trace));
			}
			else
			{
				return FString::Printf(
					TEXT("    --- Callstack backtrace for %s operation (resolve in the Watch window):\n")
					TEXT("        %s: (void**)0x%p,32\n"),
					OperationPrefix,
					TracePrefix,
					Trace);

			}
		}
		else
		{
			return FString(BARRIER_TRACKER_LOG_ENABLE_TRANSITION_BACKTRACE);
		}
	}

	static inline FString GetReasonString_DuplicateBackTrace(void* PreviousTrace, void* CurrentTrace)
	{
		if (PreviousTrace || CurrentTrace)
		{
			return
				GetReasonString_Backtrace(TEXT("previous"), TEXT("RHICreateTransition"), PreviousTrace) +
				GetReasonString_Backtrace(TEXT("current"), TEXT("RHICreateTransition"), CurrentTrace);
		}
		else
		{
			return BARRIER_TRACKER_LOG_ENABLE_TRANSITION_BACKTRACE;
		}
	}

	static inline FString GetReasonString_AccessDuringTransition(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex, 
		const FState& PendingState,
		const FState& AttemptedState,
		void* CreateTrace, void* BeginTrace)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to access resource %s whilst an asynchronous resource transition is in progress. A call to RHIEndTransitions() must be made before the resource can be accessed again.\n\n")
			TEXT("    --- Pending access states for this resource are: %s\n")
			TEXT("    --- Attempted access states are:                 %s\n")
			TEXT("    --- Pending pipelines for this resource are:     %s\n")
			TEXT("    --- Attempted pipelines are:                     %s\n")
			TEXT("%s")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetRHIAccessName(PendingState.Access),
			*GetRHIAccessName(AttemptedState.Access),
			*GetRHIPipelineName(PendingState.Pipelines),
			*GetRHIPipelineName(AttemptedState.Pipelines),
			*GetReasonString_BeginBacktrace(CreateTrace, BeginTrace));
	}

	static inline FString GetReasonString_TransitionWithoutAcquire(FResource* Resource)
	{
		FString DebugName = GetResourceDebugName(Resource, {});
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted a resource transition for transient resource %s without acquiring it. Transient resources must be acquired before any transitions are begun and discarded after all transitions are complete.\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName);
	}

	static inline FString GetReasonString_AcquireNonTransient(FResource* Resource)
	{
		FString DebugName = GetResourceDebugName(Resource, {});
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to acquire non-transient resource %s. Only transient resources may be acquired with the transient aliasing API.\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName);
	}

	static inline FString GetReasonString_DiscardNonTransient(FResource* Resource)
	{
		FString DebugName = GetResourceDebugName(Resource, {});
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to discard non-transient resource %s. Only transient resources may be discarded with the transient aliasing API.\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName);
	}

	static inline FString GetReasonString_AliasingOverlapNonDiscarded(FResource* ResourceBefore, FResource* ResourceAfter, void* CreateTrace)
	{
		FString DebugNameBefore = GetResourceDebugName(ResourceBefore, {});
		FString DebugNameAfter = GetResourceDebugName(ResourceAfter, {});
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to overlap resource %s (before) with resource %s (after), but %s (before) has not been discarded.\n")
			TEXT("%s")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugNameAfter,
			*DebugNameBefore,
			*DebugNameAfter,
			*DebugNameBefore,
			*GetReasonString_Backtrace(TEXT("acquire"), TEXT("RHICreateTransition"), CreateTrace));
	}

	static inline FString GetReasonString_AliasingOverlapNonTransient(FResource* ResourceBefore, FResource* ResourceAfter)
	{
		FString DebugNameBefore = GetResourceDebugName(ResourceBefore, {});
		FString DebugNameAfter = GetResourceDebugName(ResourceAfter, {});
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to overlap non-transient resource %s when acquiring resource %s. Only transient resources may be used in an aliasing overlap operation.\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugNameBefore,
			*DebugNameBefore,
			*DebugNameAfter);
	}

	static inline FString GetReasonString_DuplicateAcquireTransient(FResource* Resource, void* PreviousAcquireTrace, void* CurrentAcquireTrace)
	{
		FString DebugName = GetResourceDebugName(Resource, {});
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Mismatched acquire of transient resource %s. A transient resource may only be acquired once in its lifetime.\n")
			TEXT("%s")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetReasonString_DuplicateBackTrace(PreviousAcquireTrace, CurrentAcquireTrace));
	}

	static inline FString GetReasonString_DiscardWithoutAcquireTransient(FResource* Resource, void* DiscardTrace)
	{
		FString DebugName = GetResourceDebugName(Resource, {});
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to discard transient resource %s, but it was never acquired.\n")
			TEXT("%s")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetReasonString_Backtrace(TEXT("discard"), TEXT("RHICreateTransition"), DiscardTrace));
	}
	
	static inline FString GetReasonString_AlreadyDiscarded(FResource* Resource, void* DiscardTrace)
	{
		FString DebugName = GetResourceDebugName(Resource, {});
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to transition transient resource %s to ERHIAccess::Discard, but it has already been discarded.\n")
			TEXT("%s")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetReasonString_Backtrace(TEXT("discard"), TEXT("RHICreateTransition"), DiscardTrace));
	}

	static inline FString GetReasonString_DuplicateBeginTransition(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		const FState& PendingState,
		const FState& TargetState,
		void* CreateTrace, void* BeginTrace)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to begin a resource transition for resource %s whilst a previous asynchronous resource transition is already in progress. A call to RHIEndTransitions() must be made before the resource can be transitioned again.\n\n")
			TEXT("    --- Pending access states for this resource are:              %s\n")
			TEXT("    --- Attempted access states for the duplicate transition are: %s\n")
			TEXT("    --- Pending pipelines for this resource are:                  %s\n")
			TEXT("    --- Attempted pipelines for the duplicate transition are:     %s\n")
			TEXT("%s")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetRHIAccessName(PendingState.Access),
			*GetRHIAccessName(TargetState.Access),
			*GetRHIPipelineName(PendingState.Pipelines),
			*GetRHIPipelineName(TargetState.Pipelines),
			*GetReasonString_BeginBacktrace(CreateTrace, BeginTrace));
	}

	static inline FString GetReasonString_WrongPipeline(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		const FState& ActualCurrentState,
		const FState& CurrentStateFromRHI)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to begin a resource transition for resource %s on the wrong pipeline(s) (\"%s\"). The resource is currently accessible on the \"%s\" pipeline(s).\n\n")
			TEXT("    --- Current access states for this resource are: %s\n")
			TEXT("    --- Attempted access states are:                 %s\n\n")
			TEXT("    --- Ensure that resource transitions are issued on the correct pipeline.\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetRHIPipelineName(CurrentStateFromRHI.Pipelines),
			*GetRHIPipelineName(ActualCurrentState.Pipelines),
			*GetRHIAccessName(ActualCurrentState.Access),
			*GetRHIAccessName(CurrentStateFromRHI.Access));
	}
	
	static inline FString GetReasonString_IncorrectFencing(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		ERHIPipeline SrcPipelineSkipped,
		ERHIPipeline DstPipeline)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		FString SrcPipelineName = *GetRHIPipelineName(SrcPipelineSkipped);
		FString DstPipelineName = *GetRHIPipelineName(DstPipeline);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attemped to begin a resource transition for resource %s on the %s pipeline but skipping the transition on the %s pipeline (which is allowed with the NoFence flag), however no external\n")
			TEXT("fence was issued between these two pipelines between this begin transition and the last end transition call on the %s pipeline. You must insert a manual fence from '%s' to '%s'.\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*DstPipelineName,
			*SrcPipelineName,
			*SrcPipelineName,
			*SrcPipelineName,
			*DstPipelineName);
	}

	static inline FString GetReasonString_IncorrectPreviousExplicitState(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		const FState& CurrentState,
		const FState& CurrentStateFromRHI)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("The explicit previous state \"%s\" does not match the tracked current state \"%s\" for the resource %s.\n")
			TEXT("    --- Allowed pipelines for this resource are:                           %s\n")
			TEXT("    --- Previous pipelines passed as part of the resource transition were: %s\n\n")
			TEXT("    --- The best solution is to correct the explicit previous state passed for the resource in the call to RHICreateTransition().\n")
			TEXT("    --- Alternatively, use ERHIAccess::Unknown if the actual previous state cannot be determined. Unknown previous resource states have a performance impact so should be avoided if possible.\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*GetRHIAccessName(CurrentStateFromRHI.Access),
			*GetRHIAccessName(CurrentState.Access),
			*DebugName,
			*GetRHIPipelineName(CurrentState.Pipelines),
			*GetRHIPipelineName(CurrentStateFromRHI.Pipelines));
	}

	static inline FString GetReasonString_IncorrectPreviousTrackedState(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		const FState& CurrentState,
		ERHIPipeline PipelineFromRHI)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("The tracked previous state \"%s\" does not match the tracked current state \"%s\" for the resource %s.\n")
			TEXT("    --- Allowed pipelines for this resource are:                           %s\n")
			TEXT("    --- Previous pipelines passed as part of the resource transition were: %s\n\n")
			TEXT("    --- The previous state was pulled from the last call to RHICmdList.SetTrackedAccess due to the use of ERHIAccess::Unknown. If this doesn't match the expected state, be sure to update the \n")
			TEXT("    --- tracked state after using manual low - level transitions. It is highly recommended to coalesce all subresources into the same state before relying on tracked previous states with \n")
			TEXT("    --- ERHIAccess::Unknown. RHICmdList.SetTrackedAccess applies to whole resources.\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*GetRHIAccessName(Resource->GetTrackedState().Access),
			*GetRHIAccessName(CurrentState.Access),
			*DebugName,
			*GetRHIPipelineName(CurrentState.Pipelines),
			*GetRHIPipelineName(PipelineFromRHI));
	}

	static inline FString GetReasonString_MismatchedEndTransition(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		const FState& TargetState,
		const FState& TargetStateFromRHI)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("The expected target state \"%s\" on pipe \"%s\" in end transition does not match the tracked target state \"%s\" on pipe \"%s\" for the resource %s.\n")
			TEXT("    --- The call to EndTransition() is mismatched with the another BeginTransition() with different states.\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*GetRHIAccessName(TargetStateFromRHI.Access),
			*GetRHIPipelineName(TargetState.Pipelines),
			*GetRHIAccessName(TargetState.Access),
			*GetRHIPipelineName(TargetStateFromRHI.Pipelines),
			*DebugName);
	}

	static inline FString GetReasonString_UnnecessaryTransition(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		const FState& CurrentState)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to begin a resource transition for the resource %s to the \"%s\" state on the \"%s\" pipe, but the resource is already in this state. The resource transition is unnecessary.\n")
			TEXT("    --- This is not fatal, but does have an effect on CPU and GPU performance. Consider refactoring rendering code to avoid unnecessary resource transitions.\n")
			TEXT("    --- RenderGraph (RDG) is capable of handling resource transitions automatically.\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetRHIAccessName(CurrentState.Access),
			*GetRHIPipelineName(CurrentState.Pipelines));
	}

	static inline FString GetReasonString_MismatchedAllUAVsOverlapCall(bool bAllow)
	{
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_REASON("UAV overlap mismatch")
			TEXT("Mismatched call to %sUAVOverlap.\n\n")
			TEXT("    --- Ensure all calls to RHICmdList.BeginUAVOverlap() are paired with a call to RHICmdList.EndUAVOverlap().\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			bAllow ? TEXT("Begin") : TEXT("End")
		);
	}

	static inline FString GetReasonString_MismatchedExplicitUAVOverlapCall(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		bool bAllow)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_REASON("UAV overlap mismatch")
			TEXT("Mismatched call to %sUAVOverlap(FRHIUnorderedAccessView*) for the resource %s.\n\n")
			TEXT("    --- Ensure all calls to RHICmdList.BeginUAVOverlap() are paired with a call to RHICmdList.EndUAVOverlap().\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			bAllow ? TEXT("Begin") : TEXT("End"),
			*DebugName
		);
	}

	static inline FString GetReasonString_UAVOverlap(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		const FState& CurrentState, const FState& RequiredState)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to access resource %s which was previously used with overlapping UAV access, but has not been transitioned since UAV overlap was disabled. A resource transition is required.\n\n")
			TEXT("    --- Allowed access states for this resource are: %s\n")
			TEXT("    --- Required access states are:                  %s\n")
			TEXT("    --- Allowed pipelines for this resource are:     %s\n")
			TEXT("    --- Required pipelines are:                      %s\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetRHIAccessName(CurrentState.Access),
			*GetRHIAccessName(RequiredState.Access),
			*GetRHIPipelineName(CurrentState.Pipelines),
			*GetRHIPipelineName(RequiredState.Pipelines));
	}

	static inline FString GetReasonString_IgnoreAfterStateAllPipes(
		FResource* Resource, FSubresourceIndex const& SubresourceIndex,
		const FState& PendingState,
		const FState& TargetState)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to begin a resource transition for resource %s on All pipes. Transition with EResourceTransitionFlags::IgnoreAfterState on All pipes are not supported.\n\n")
			TEXT("    --- Pending access states for this resource are:              %s\n")
			TEXT("    --- Attempted access states for the current transition are: %s\n")
			TEXT("    --- Pending pipelines for this resource are:                  %s\n")
			TEXT("    --- Attempted pipelines for the current transition are:     %s\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetRHIAccessName(PendingState.Access),
			*GetRHIAccessName(TargetState.Access),
			*GetRHIPipelineName(PendingState.Pipelines),
			*GetRHIPipelineName(TargetState.Pipelines));
	}

	static inline FString GetReasonString_MismatchedIgnoreAfterState(
			FResource* Resource, FSubresourceIndex const& SubresourceIndex,
			const FState& PendingState,
			const FState& TargetState)
	{
		FString DebugName = GetResourceDebugName(Resource, SubresourceIndex);
		return FString::Printf(
			BARRIER_TRACKER_LOG_PREFIX_RESNAME
			TEXT("Attempted to begin a resource transition for resource %s whilst not having the flag EResourceTransitionFlags::IgnoreAfterState matching the previous transiton. Transition with EResourceTransitionFlags::IgnoreAfterState always needs to be done in pairs.\n\n")
			TEXT("    --- Pending access states for this resource are:              %s\n")
			TEXT("    --- Attempted access states for the current transition are: %s\n")
			TEXT("    --- Pending pipelines for this resource are:                  %s\n")
			TEXT("    --- Attempted pipelines for the current transition are:     %s\n")
			BARRIER_TRACKER_LOG_SUFFIX,
			*DebugName,
			*DebugName,
			*GetRHIAccessName(PendingState.Access),
			*GetRHIAccessName(TargetState.Access),
			*GetRHIPipelineName(PendingState.Pipelines),
			*GetRHIPipelineName(TargetState.Pipelines));
	}

	static inline void* Log(FResource* Resource, FSubresourceIndex const& SubresourceIndex, void* CreateTrace, const TCHAR* TracePrefix, const TCHAR* Type, const TCHAR* LogStr)
	{
		void* Trace = CaptureBacktrace();

		FString BreadcrumbMessage = GetBreadcrumbPath();

		if (CreateTrace)
		{
			if (GRHIValidationPrintHumanReadableCallStack)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("\n%s: Type: %s, %s, \nCreateTrace: %s\n, %sTrace: %s\n, %s\n"),
					*GetResourceDebugName(Resource, SubresourceIndex),
					Type,
					LogStr,
					*ResolveAndFormatCallstack((uint64*)CreateTrace),
					TracePrefix,
					*ResolveAndFormatCallstack((uint64*)Trace),
					*BreadcrumbMessage);
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("\n%s: Type: %s, %s, CreateTrace: 0x%p, %sTrace: 0x%p, %s\n"),
					*GetResourceDebugName(Resource, SubresourceIndex),
					Type,
					LogStr,
					CreateTrace,
					TracePrefix,
					Trace,
					*BreadcrumbMessage);
			}
		}
		else
		{
			if (GRHIValidationPrintHumanReadableCallStack)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("\n%s: Type: %s, %s, \nTrace: %s\n, %s\n"),
					*GetResourceDebugName(Resource, SubresourceIndex),
					Type,
					LogStr,
					*ResolveAndFormatCallstack((uint64*)Trace),
					*BreadcrumbMessage);
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("\n%s: Type: %s, %s, Trace: 0x%p, %s\n"),
					*GetResourceDebugName(Resource, SubresourceIndex),
					Type,
					LogStr,
					Trace,
					*BreadcrumbMessage);
			}
		}

		return Trace;
	}

	void FTransientState::Acquire(FResource* Resource, void* CreateTrace, ERHIPipeline ExecutingPipeline)
	{
		RHI_VALIDATION_CHECK(bTransient, *GetReasonString_AcquireNonTransient(Resource));
		RHI_VALIDATION_CHECK(Status == EStatus::None, *GetReasonString_DuplicateAcquireTransient(Resource, AcquireBacktrace, CreateTrace));
		Status = EStatus::Acquired;

		if (!AcquireBacktrace)
		{
			AcquireBacktrace = CreateTrace;
		}

		NumAcquiredSubresources = Resource->GetNumSubresources() * GetRHIPipelineCount();

		if (Resource->LoggingMode != ELoggingMode::None)
		{
			Log(Resource, {}, CreateTrace, TEXT("Acquire"), TEXT("Acquire"), *FString::Printf(TEXT("Transient Acquire, Executing Pipeline : %s"), *GetRHIPipelineName(ExecutingPipeline)));
		}
	}

	void FTransientState::Discard(FResource* Resource, void* CreateTrace, ERHIPipeline DiscardPipelines, ERHIPipeline ExecutingPipeline)
	{
		RHI_VALIDATION_CHECK(bTransient, *GetReasonString_DiscardNonTransient(Resource));
		RHI_VALIDATION_CHECK(Status != EStatus::None, *GetReasonString_DiscardWithoutAcquireTransient(Resource, CreateTrace));
		RHI_VALIDATION_CHECK(Status != EStatus::Discarded, *GetReasonString_AlreadyDiscarded(Resource, CreateTrace));

		// When discarding from all pipes, each pipe will call Discard separately. Otherwise it's just one call.
		const uint32 NumDerefs = DiscardPipelines == ERHIPipeline::All ? 1 : 2;

		NumAcquiredSubresources -= NumDerefs;

		if (NumAcquiredSubresources == 0)
		{
			Status = EStatus::Discarded;

			if (Resource->LoggingMode != ELoggingMode::None)
			{
				Log(Resource, {}, CreateTrace, TEXT("Discard"), TEXT("Discard"), *FString::Printf(TEXT("Transient Discard, Executing Pipeline : %s"), *GetRHIPipelineName(ExecutingPipeline)) );
			}
		}
	}

	void FTransientState::AliasingOverlap(FResource* ResourceBefore, FResource* ResourceAfter, void* CreateTrace)
	{
		FTransientState& TransientStateBefore = ResourceBefore->TransientState;
		FTransientState& TransientStateAfter = ResourceAfter->TransientState;

		// Acquire should validate whether ResourceAfter is transient. We assume it is here.
		RHI_VALIDATION_CHECK(TransientStateBefore.bTransient, *GetReasonString_AliasingOverlapNonTransient(ResourceBefore, ResourceAfter));
		RHI_VALIDATION_CHECK(TransientStateBefore.IsDiscarded(), *GetReasonString_AliasingOverlapNonDiscarded(ResourceBefore, ResourceAfter, CreateTrace));

		if (ResourceBefore->LoggingMode != ELoggingMode::None)
		{
			Log(ResourceBefore, {}, CreateTrace, TEXT("AliasingOverlap"), TEXT("AliasingOverlap"), TEXT("Aliasing Overlap (Before)"));
		}

		if (ResourceAfter->LoggingMode != ELoggingMode::None)
		{
			Log(ResourceAfter, {}, CreateTrace, TEXT("AliasingOverlap"), TEXT("AliasingOverlap"), TEXT("Aliasing Overlap (After)"));
		}
	}

	void FResource::SetDebugName(const TCHAR* Name, const TCHAR* Suffix)
	{
		DebugName = Suffix
			? FString::Printf(TEXT("%s%s"), Name, Suffix)
			: Name;

		if (LoggingMode != ELoggingMode::Manual)
		{
			// Automatically enable/disable barrier logging if the resource name
			// does/doesn't match one in the AutoLogResourceNames array.
			if (Name)
			{
				for (FString const& Str : GetAutoLogResourceNames())
				{
					if (FCString::Stricmp(Name, *Str) == 0)
					{
						LoggingMode = ELoggingMode::Automatic;
						return;
					}
				}
			}

			LoggingMode = ELoggingMode::None;
		}
	}

	void FResource::InitTransient(const TCHAR* InDebugName)
	{
		check(TransientState.bTransient && TransientState.Status != FTransientState::EStatus::Acquired);
		TransientState.Status = FTransientState::EStatus::None;
		DebugName = InDebugName;

		for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
		{
			auto& State = WholeResourceState.States[Pipeline];

			State.Current.Access = ERHIAccess::Discard;
			State.Current.Pipelines = Pipeline;
			State.Previous = State.Current;
		}
		SubresourceStates.Reset();
	}

	void FResource::InitBarrierTracking(int32 InNumMips, int32 InNumArraySlices, int32 InNumPlanes, ERHIAccess InResourceState, const TCHAR* InDebugName)
	{
		checkSlow(InNumMips > 0 && InNumArraySlices > 0 && InNumPlanes > 0);
		check(InResourceState != ERHIAccess::Unknown);

		NumMips = InNumMips;
		NumArraySlices = InNumArraySlices;
		NumPlanes = InNumPlanes;
		TransientState = FTransientState(InResourceState);
		TrackedState = FState(InResourceState, ERHIPipeline::None);

		for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
		{
			auto& State = WholeResourceState.States[Pipeline];

			State.Current.Access = InResourceState;
			State.Current.Pipelines = Pipeline;
			State.Previous = State.Current;
		}

		if (InDebugName != nullptr)
		{
			SetDebugName(InDebugName);
		}
	}

	void FSubresourceState::BeginTransition(FResource* Resource, FSubresourceIndex const& SubresourceIndex, const FState& CurrentStateFromRHI, const FState& InTargetState, EResourceTransitionFlags NewFlags, ERHITransitionCreateFlags CreateFlags, ERHIPipeline ExecutingPipeline, const TRHIPipelineArray<uint64>& PipelineMaxAwaitedFenceValues, void* CreateTrace)
	{
		FPipelineState& State = States[ExecutingPipeline];

		FState TargetState = InTargetState;

		if (TargetState.Access == ERHIAccess::Unknown)
		{
			TargetState.Access = Resource->GetTrackedState().Access;
		}

		void* BeginTrace = nullptr;
		if (Resource->LoggingMode != ELoggingMode::None 
#if LOG_UNNAMED_RESOURCES
			|| Resource->GetDebugName() == nullptr
#endif
			)
		{
			const TCHAR* PulledFromTracked = TEXT("");

			if (InTargetState.Access == ERHIAccess::Unknown)
			{
				PulledFromTracked = TEXT(" (Pulled From SetTrackedAccess)");
			}

			BeginTrace = Log(Resource, SubresourceIndex, CreateTrace, TEXT("Begin"), TEXT("BeginTransition"), *FString::Printf(TEXT("Current: (%s) -> Before(%s) New: (%s)%s, Flags: %s, Executing Pipeline: %s"),
				*State.Current.ToString(),
				*CurrentStateFromRHI.ToString(),
				*TargetState.ToString(),
				PulledFromTracked,
				*GetResourceTransitionFlagsName(NewFlags),
				*GetRHIPipelineName(ExecutingPipeline)
			));
		}

		if (CurrentStateFromRHI.Access == ERHIAccess::Unknown)
		{
			RHI_VALIDATION_CHECK(Resource->GetTrackedState().Access == State.Previous.Access, *GetReasonString_IncorrectGetTrackedAccess(Resource, SubresourceIndex, State.Previous, Resource->GetTrackedState()));
		}

		if (Resource->TransientState.bTransient)
		{
			RHI_VALIDATION_CHECK(Resource->TransientState.IsAcquired(), *GetReasonString_TransitionWithoutAcquire(Resource));

			if (EnumHasAnyFlags(TargetState.Access, ERHIAccess::Discard))
			{
				Resource->TransientState.Discard(Resource, CreateTrace, CurrentStateFromRHI.Pipelines, ExecutingPipeline);
			}
		}

		// If we are collapsing multiple pipes to one pipe (only allowed when not fencing), check that the other pipes were fenced prior to this call.
		if (EnumHasAnyFlags(CreateFlags, ERHITransitionCreateFlags::NoFence))
		{
			for (ERHIPipeline AlreadyFencedPipeline : MakeFlagsRange(State.Previous.Pipelines & ~CurrentStateFromRHI.Pipelines))
			{
				// The max awaited fence value should be higher than the last transitioned fence value, otherwise a fence was not issued.
				RHI_VALIDATION_CHECK(LastTransitionFences[AlreadyFencedPipeline] < PipelineMaxAwaitedFenceValues[AlreadyFencedPipeline], *GetReasonString_IncorrectFencing(Resource, SubresourceIndex, AlreadyFencedPipeline, ExecutingPipeline));
			}
		}

		// Check we're not already transitioning
		RHI_VALIDATION_CHECK(!State.bTransitioning, *GetReasonString_DuplicateBeginTransition(Resource, SubresourceIndex, State.Current, TargetState, State.CreateTransitionBacktrace, BeginTrace));

		// Validate the explicit previous state from the RHI matches what we expect...
		{
			// Check for the correct pipeline
			RHI_VALIDATION_CHECK(EnumHasAllFlags(CurrentStateFromRHI.Pipelines, ExecutingPipeline), *GetReasonString_WrongPipeline(Resource, SubresourceIndex, State.Current, TargetState));

			const auto HasMatchingPipelines = [CreateFlags, Resource] (ERHIPipeline PreviousFromState, ERHIPipeline PreviousFromRHI)
			{
				if (PreviousFromState == PreviousFromRHI)
				{
					return true;
				}

				// We allow collapsing pipes from All -> Single only if the flag is explicitly provided.
				if (EnumHasAnyFlags(CreateFlags, ERHITransitionCreateFlags::AllowDecayPipelines))
				{
					return EnumHasAnyFlags(PreviousFromState, PreviousFromRHI) && PreviousFromState == Resource->GetTrackedState().Pipelines;
				}

				return false;
			};


			bool bHasMatchingPipelines = true;

			// We do not check pipelines for IgnoreAfterState since we do not replicate those transition on all pipelines.
			if (EnumHasAnyFlags(NewFlags, EResourceTransitionFlags::IgnoreAfterState) == false)
			{
				bHasMatchingPipelines = HasMatchingPipelines(State.Previous.Pipelines, CurrentStateFromRHI.Pipelines);
			}

			if (CurrentStateFromRHI.Access == ERHIAccess::Unknown)
			{
				RHI_VALIDATION_CHECK(Resource->TrackedState.Access == State.Previous.Access && bHasMatchingPipelines,
					*GetReasonString_IncorrectPreviousTrackedState(Resource, SubresourceIndex, State.Previous, CurrentStateFromRHI.Pipelines));
			}
			else
			{
				// Check the current RHI state passed in matches the tracked state for the resource.
				RHI_VALIDATION_CHECK(CurrentStateFromRHI.Access == State.Previous.Access && bHasMatchingPipelines,
					*GetReasonString_IncorrectPreviousExplicitState(Resource, SubresourceIndex, State.Previous, CurrentStateFromRHI));
			}
		}

		const bool bIgnoreAfterStateAllPipes = ((EnumHasAnyFlags(NewFlags, EResourceTransitionFlags::IgnoreAfterState)) && (CurrentStateFromRHI.Pipelines == ERHIPipeline::All));
		RHI_VALIDATION_CHECK(bIgnoreAfterStateAllPipes == false,
			*GetReasonString_IgnoreAfterStateAllPipes(Resource, SubresourceIndex, State.Current, TargetState));

		const bool bRegularTransitionWhileIgnoring = (State.bIgnoringAfterState && (EnumHasAnyFlags(NewFlags, EResourceTransitionFlags::IgnoreAfterState) == false));

		RHI_VALIDATION_CHECK(bRegularTransitionWhileIgnoring == false,
				*GetReasonString_MismatchedIgnoreAfterState(Resource, SubresourceIndex, State.Current, TargetState));

		State.bTransitioning = true;
		State.Flags = NewFlags;
		State.BeginTransitionBacktrace = BeginTrace;
		// Update the tracked state once all pipes have begun.
		State.Previous = TargetState;
		State.Current = TargetState;
		State.CreateTransitionBacktrace = CreateTrace;
		State.bUsedWithAllUAVsOverlap = false;
		State.bUsedWithExplicitUAVsOverlap = false;

		// Do not replicate the state for EResourceTransitionFlags::IgnoreAfterState transition
		if (EnumHasAnyFlags(NewFlags, EResourceTransitionFlags::IgnoreAfterState) == false)
		{
			// Replicate the state to other pipes that are not part of the begin pipe mask.
			for (ERHIPipeline OtherPipeline : MakeFlagsRange(ERHIPipeline::All & ~CurrentStateFromRHI.Pipelines))
			{
				States[OtherPipeline] = State;
			}
		}
	}

	void FSubresourceState::EndTransition(FResource* Resource, FSubresourceIndex const& SubresourceIndex, const FState& CurrentStateFromRHI, const FState& InTargetState, EResourceTransitionFlags NewFlags, ERHIPipeline ExecutingPipeline, uint64 ExecutingPipelineFenceValue, void* CreateTrace)
	{
		FPipelineState& State = States[ExecutingPipeline];

		FState TargetState = InTargetState;

		if (TargetState.Access == ERHIAccess::Unknown)
		{
			TargetState.Access = Resource->GetTrackedState().Access;
		}
		
		if (Resource->LoggingMode != ELoggingMode::None
#if LOG_UNNAMED_RESOURCES
			|| Resource->GetDebugName() == nullptr
#endif
			)
		{
			const TCHAR* PulledFromTracked = TEXT("");

			if (InTargetState.Access == ERHIAccess::Unknown)
			{
				PulledFromTracked = TEXT(" (Pulled From SetTrackedAccess)");
			}

			Log(Resource, SubresourceIndex, CreateTrace, TEXT("End"), TEXT("EndTransition"), *FString::Printf(TEXT("Access: %s%s, Pipeline: %s, Executing Pipeline: %s"),
				*GetRHIAccessName(TargetState.Access),
				PulledFromTracked,
				*GetRHIPipelineName(TargetState.Pipelines),
				*GetRHIPipelineName(ExecutingPipeline)
			));
		}

		// Set bIgnoringAfterState to true in case of EResourceTransitionFlags::IgnoreAfterState so on the next transition we can check that EResourceTransitionFlags::IgnoreAfterState is used as well
		if (EnumHasAnyFlags(NewFlags, EResourceTransitionFlags::IgnoreAfterState))
		{
			State.bIgnoringAfterState = !State.bIgnoringAfterState;
		}

		// Check that we aren't ending a transition that never began.
		RHI_VALIDATION_CHECK(State.bTransitioning, TEXT("Unsolicited resource end transition call."));
		State.bTransitioning = false;
		State.BeginTransitionBacktrace = nullptr;

		// Check that the end matches the begin.
		RHI_VALIDATION_CHECK(TargetState == State.Current, *GetReasonString_MismatchedEndTransition(Resource, SubresourceIndex, State.Current, TargetState));

		// Do not replicate the state for EResourceTransitionFlags::IgnoreAfterState transitions
		if (EnumHasAnyFlags(NewFlags, EResourceTransitionFlags::IgnoreAfterState) == false)
		{
			// Replicate the state to other pipes that are not part of the end pipe mask.
			for (ERHIPipeline OtherPipeline : MakeFlagsRange(ERHIPipeline::All))
			{
				if (!EnumHasAnyFlags(TargetState.Pipelines, OtherPipeline))
				{
					States[OtherPipeline] = State;
				}
			}
		}

		LastTransitionFences[ExecutingPipeline] = ExecutingPipelineFenceValue;
	}

	void FSubresourceState::Assert(FResource* Resource, FSubresourceIndex const& SubresourceIndex, const FState& RequiredState, bool bAllowAllUAVsOverlap)
	{
		if (Resource->LoggingMode != ELoggingMode::None
#if LOG_UNNAMED_RESOURCES
			|| Resource->GetDebugName() == nullptr
#endif
			)
		{
			Log(Resource, SubresourceIndex, nullptr, nullptr, TEXT("Assert"), *FString::Printf(TEXT("Access: %s, Pipeline: %s"), 
				*GetRHIAccessName(RequiredState.Access),
				*GetRHIPipelineName(RequiredState.Pipelines)));
		}

		FPipelineState& State = States[RequiredState.Pipelines];

		// Check we're not trying to access the resource whilst a pending resource transition is in progress.
		RHI_VALIDATION_CHECK(!State.bTransitioning, *GetReasonString_AccessDuringTransition(Resource, SubresourceIndex, State.Current, RequiredState, State.CreateTransitionBacktrace, State.BeginTransitionBacktrace));

		// If UAV overlaps are now disabled, ensure the resource has been transitioned if it was previously used in UAV overlap state.
		RHI_VALIDATION_CHECK((bAllowAllUAVsOverlap || !State.bUsedWithAllUAVsOverlap) && (State.bExplicitAllowUAVOverlap || !State.bUsedWithExplicitUAVsOverlap), *GetReasonString_UAVOverlap(Resource, SubresourceIndex, State.Current, RequiredState));

		// Ensure the resource is in the required state for this operation
		RHI_VALIDATION_CHECK(EnumHasAllFlags(State.Current.Access, RequiredState.Access) && EnumHasAllFlags(State.Current.Pipelines, RequiredState.Pipelines), *GetReasonString_MissingBarrier(Resource, SubresourceIndex, State.Current, RequiredState));

		State.Previous = State.Current;

		if (EnumHasAnyFlags(RequiredState.Access, ERHIAccess::UAVMask | ERHIAccess::BVHWrite))
		{
			if (bAllowAllUAVsOverlap) { State.bUsedWithAllUAVsOverlap = true; }
			if (State.bExplicitAllowUAVOverlap) { State.bUsedWithExplicitUAVsOverlap = true; }
		}

		// Disable all non-compatible access types
		State.Current.Access = DecayResourceAccess(State.Current.Access, RequiredState.Access, bAllowAllUAVsOverlap || State.bExplicitAllowUAVOverlap);
	}

	void FSubresourceState::AssertTracked(FResource* Resource, FSubresourceIndex const& SubresourceIndex, const FState& RequiredState, ERHIPipeline ExecutingPipeline)
	{
		if (Resource->LoggingMode != ELoggingMode::None
#if LOG_UNNAMED_RESOURCES
			|| Resource->GetDebugName() == nullptr
#endif
			)
		{
			Log(Resource, SubresourceIndex, nullptr, nullptr, TEXT("AssertTracked"), *FString::Printf(TEXT("Access: %s, Pipelines %s"), *GetRHIAccessName(RequiredState.Access), *GetRHIPipelineName(RequiredState.Pipelines)));
		}

		for (ERHIPipeline Pipeline : MakeFlagsRange(RequiredState.Pipelines))
		{
			FPipelineState& State = States[Pipeline];

			// Check we're not trying to access the resource whilst a pending resource transition is in progress (can only do this on the executing pipeline).
			if (State.Current.Pipelines == ExecutingPipeline)
			{
				RHI_VALIDATION_CHECK(!State.bTransitioning, *GetReasonString_AccessDuringTransition(Resource, SubresourceIndex, State.Current, RequiredState, State.CreateTransitionBacktrace, State.BeginTransitionBacktrace));
			}

			// Ensure the resource is in the required state for this operation (ignore the Discard state which always resets).
			RHI_VALIDATION_CHECK(State.Current.Access == ERHIAccess::Discard || State.Current == RequiredState, *GetReasonString_IncorrectSetTrackedAccess(Resource, SubresourceIndex, State.Current, RequiredState));
		}
	}

	void FSubresourceState::SpecificUAVOverlap(FResource* Resource, FSubresourceIndex const& SubresourceIndex, ERHIPipeline Pipeline, bool bAllow)
	{
		if (Resource->LoggingMode != ELoggingMode::None
#if LOG_UNNAMED_RESOURCES
			|| Resource->GetDebugName() == nullptr
#endif
			)
		{
			Log(Resource, SubresourceIndex, nullptr, nullptr, TEXT("UAVOverlap"), *FString::Printf(TEXT("Allow: %s"), bAllow ? TEXT("True") : TEXT("False")));
		}

		FPipelineState& State = States[Pipeline];
		RHI_VALIDATION_CHECK(State.bExplicitAllowUAVOverlap != bAllow, *GetReasonString_MismatchedExplicitUAVOverlapCall(Resource, SubresourceIndex, bAllow));
		State.bExplicitAllowUAVOverlap = bAllow;
	}

	inline void FResource::EnumerateSubresources(FSubresourceRange const& SubresourceRange, TFunctionRef<void(FSubresourceState&, FSubresourceIndex const&)> Callback, bool bBeginTransition)
	{
		bool bWholeResource = SubresourceRange.IsWholeResource(*this);
		if (bWholeResource && SubresourceStates.Num() == 0)
		{
			Callback(WholeResourceState, FSubresourceIndex());
		}
		else
		{
			if (SubresourceStates.Num() == 0)
			{
				const uint32 NumSubresources = NumMips * NumArraySlices * NumPlanes;
				SubresourceStates.Reserve(NumSubresources);

				// Copy the whole resource state into all the subresource slots
				for (uint32 Index = 0; Index < NumSubresources; ++Index)
				{
					SubresourceStates.Add(WholeResourceState);
				}
			}

			if (SubresourceStates.Num() != 0)
			{
				uint32 LastMip = SubresourceRange.MipIndex + SubresourceRange.NumMips;
				uint32 LastArraySlice = SubresourceRange.ArraySlice + SubresourceRange.NumArraySlices;
				uint32 LastPlaneIndex = SubresourceRange.PlaneIndex + SubresourceRange.NumPlanes;

				for (uint32 PlaneIndex = SubresourceRange.PlaneIndex; PlaneIndex < LastPlaneIndex; ++PlaneIndex)
				{
					for (uint32 MipIndex = SubresourceRange.MipIndex; MipIndex < LastMip; ++MipIndex)
					{
						for (uint32 ArraySlice = SubresourceRange.ArraySlice; ArraySlice < LastArraySlice; ++ArraySlice)
						{
							uint32 SubresourceIndex = PlaneIndex + (MipIndex + ArraySlice * NumMips) * NumPlanes;
							Callback(SubresourceStates[SubresourceIndex], FSubresourceIndex(MipIndex, ArraySlice, PlaneIndex));
						}
					}
				}
			}
		}

		if (bWholeResource && bBeginTransition && SubresourceStates.Num() != 0)
		{
			// Switch back to whole resource state tracking on begin transitions
			WholeResourceState = SubresourceStates[0];
			SubresourceStates.Reset();
		}
	}

#if WITH_RHI_BREADCRUMBS
	bool IsInRange(FRHIBreadcrumbRange const& Range, FRHIBreadcrumbNode* const Target, ERHIPipeline Pipeline)
	{
		for (FRHIBreadcrumbNode* Current : Range.Enumerate(Pipeline))
		{
			if (Current == Target)
				return true;
		}

		// Include all parent nodes above Last
		for (FRHIBreadcrumbNode* Current = Range.Last; Current; Current = Current->GetParent())
		{
			if (Current == Target)
				return true;
		}

		// Include all parent nodes above First
		for (FRHIBreadcrumbNode* Current = Range.First; Current; Current = Current->GetParent())
		{
			if (Current == Target)
				return true;
		}

		return false;
	}

	int32 CountLevels(FRHIBreadcrumbNode* Node)
	{
		auto Recurse = [](auto const& Recurse, FRHIBreadcrumbNode* Current) -> int32
		{
			check(Current != FRHIBreadcrumbNode::Sentinel);
			return Current
				? Recurse(Recurse, Current->GetParent()) + 1
				: 0;
		};
		return Recurse(Recurse, Node) - 1;
	}

	void LogNode(FRHIBreadcrumbNode* Node, bool bBegin, ERHIPipeline Pipeline)
	{
		static bool bOutputBreadcrumbLog = FParse::Param(FCommandLine::Get(), TEXT("RHIValidationBreadcrumbLog"));
		if (bOutputBreadcrumbLog)
		{
			int32 Levels = CountLevels(Node);
			FString Output;
			for (int32 Index = 0; Index < Levels; ++Index)
			{
				Output += TEXT("\t");
			}
			FRHIBreadcrumb::FBuffer Buffer;
			const TCHAR* Str = Node->GetTCHAR(Buffer);
			Output += Str;
			UE_LOGF(LogRHI, Display, " ## BC (0x%016p, 0x%08x) [%12ls] [%ls]: %ls"
				, Node
				, Node->ID
				, *GetRHIPipelineName(Pipeline)
				, bBegin ? TEXT("BEGIN") : TEXT(" END ")
				, *Output
			);
		}
	}
#endif // WITH_RHI_BREADCRUMBS

	bool FOperation::Replay(FOpQueueState& Queue) const
	{
		switch (Type)
		{
		default:
			checkNoEntry();
			break;

#if WITH_RHI_BREADCRUMBS
		case EOpType::BeginBreadcrumbGPU:
			{
				FRHIBreadcrumbNode* Node = Data_Breadcrumb.Breadcrumb;

				check(Node && Node != FRHIBreadcrumbNode::Sentinel);
				check(Node->GetParent() != FRHIBreadcrumbNode::Sentinel);
				check(Node->GetParent() == Queue.Breadcrumbs.Current);
				check(GRHICommandList.Bypass() || IsInRange(Queue.Breadcrumbs.Range, Node, Queue.Pipeline));
				check(EnumHasAllFlags(static_cast<ERHIPipeline>(Node->BeginPipes.load()), Queue.Pipeline));

				LogNode(Node, true, Queue.Pipeline);

				Queue.Breadcrumbs.Current = Node;
			}
			break;

		case EOpType::EndBreadcrumbGPU:
			{
				FRHIBreadcrumbNode* Node = Data_Breadcrumb.Breadcrumb;

				check(Node && Node != FRHIBreadcrumbNode::Sentinel);
				check(Node->GetParent() != FRHIBreadcrumbNode::Sentinel);
				check(Node == Queue.Breadcrumbs.Current);
				check(GRHICommandList.Bypass() || IsInRange(Queue.Breadcrumbs.Range, Node, Queue.Pipeline));
				check(EnumHasAllFlags(static_cast<ERHIPipeline>(Node->EndPipes.load()), Queue.Pipeline));

				LogNode(Node, false, Queue.Pipeline);

				Queue.Breadcrumbs.Current = Node->GetParent();
			}
			break;

		case EOpType::SetBreadcrumbRange:
			{
				Queue.Breadcrumbs.Range = Data_BreadcrumbRange.Range;
				check(!Queue.Breadcrumbs.Range.First == !Queue.Breadcrumbs.Range.Last);

				TSet<FRHIBreadcrumbAllocator*> AllAllocators;
				for (FRHIBreadcrumbNode* Node : Queue.Breadcrumbs.Range.Enumerate(Queue.Pipeline))
				{
					AllAllocators.Add(Node->Allocator);

					// Check current node and all parents are valid
					for (FRHIBreadcrumbNode* Other = Node; Other; Other = Other->GetParent())
					{
						check(Other != FRHIBreadcrumbNode::Sentinel);
						check(Other->GetParent() != FRHIBreadcrumbNode::Sentinel);
					}
				}

				// Check for circular references in the allocator parent pointers
				for (FRHIBreadcrumbAllocator* Allocator : AllAllocators)
				{
					auto Recurse = [](FRHIBreadcrumbAllocator* Current, auto& Recurse) -> void
					{
						checkf(!Current->bVisited, TEXT("Circular reference detected in breadcrumb allocators."));
						Current->bVisited = true;
						
						for (auto const& Parent : Current->GetParents())
						{
							Recurse(&Parent.Get(), Recurse);
						}

						Current->bVisited = false;
					};
					Recurse(Allocator, Recurse);
				}
			}
			break;
#endif // WITH_RHI_BREADCRUMBS

		case EOpType::Rename:
			Data_Rename.Resource->SetDebugName(Data_Rename.DebugName, Data_Rename.Suffix);
			delete[] Data_Rename.DebugName;
			Data_Rename.Resource->ReleaseOpRef();
			break;

		case EOpType::BeginTransition:
			Data_BeginTransition.Identity.Resource->EnumerateSubresources(Data_BeginTransition.Identity.SubresourceRange, [this, &Queue](FSubresourceState& State, FSubresourceIndex const& SubresourceIndex)
			{
				State.BeginTransition(
					Data_BeginTransition.Identity.Resource,
					SubresourceIndex,
					Data_BeginTransition.PreviousState,
					Data_BeginTransition.NextState,
					Data_BeginTransition.Flags,
					Data_BeginTransition.CreateFlags,
					Queue.Pipeline,
					Queue.MaxAwaitedFenceValues,
					Data_BeginTransition.CreateBacktrace);

			}, true);
			Data_BeginTransition.Identity.Resource->ReleaseOpRef();
			break;

		case EOpType::EndTransition:
			Data_EndTransition.Identity.Resource->EnumerateSubresources(Data_EndTransition.Identity.SubresourceRange, [this, &Queue](FSubresourceState& State, FSubresourceIndex const& SubresourceIndex)
			{
				State.EndTransition(
					Data_EndTransition.Identity.Resource,
					SubresourceIndex,
					Data_EndTransition.PreviousState,
					Data_EndTransition.NextState,
					Data_EndTransition.Flags,
					Queue.Pipeline,
					Queue.FenceValue,
					Data_EndTransition.CreateBacktrace);
			});
			Data_EndTransition.Identity.Resource->ReleaseOpRef();
			break;

		case EOpType::AliasingOverlap:
			FTransientState::AliasingOverlap(Data_AliasingOverlap.ResourceBefore, Data_AliasingOverlap.ResourceAfter, Data_AliasingOverlap.CreateBacktrace);
			Data_AliasingOverlap.ResourceBefore->ReleaseOpRef();
			Data_AliasingOverlap.ResourceAfter->ReleaseOpRef();
			break;

		case EOpType::SetTrackedAccess:
		{
			Data_Assert.Identity.Resource->EnumerateSubresources(Data_SetTrackedAccess.Resource->GetWholeResourceRange(), [this, &Queue](FSubresourceState& State, FSubresourceIndex const& SubresourceIndex)
			{
				State.AssertTracked(
					Data_SetTrackedAccess.Resource,
					SubresourceIndex,
					Data_SetTrackedAccess.State,
					Queue.Pipeline);
			});
			FResource* Resource = Data_SetTrackedAccess.Resource;
			if (Resource->LoggingMode != ELoggingMode::None)
			{
				Log(Resource, {}, nullptr, TEXT("SetTrackedAccess"), TEXT("SetTrackedAccess"), *FString::Printf(TEXT("Access: (%s), Pipelines: (%s), Executing Pipeline: (%s)"), *GetRHIAccessName(Data_SetTrackedAccess.State.Access), *GetRHIPipelineName(Data_SetTrackedAccess.State.Pipelines), *GetRHIPipelineName(Queue.Pipeline)) );
			}
			Resource->TrackedState = Data_SetTrackedAccess.State;
			Resource->ReleaseOpRef();
			break;
		}
		case EOpType::AcquireTransient:
			Data_AcquireTransient.Resource->TransientState.Acquire(Data_AcquireTransient.Resource, Data_AcquireTransient.CreateBacktrace, Queue.Pipeline);
			Data_AcquireTransient.Resource->ReleaseOpRef();
			break;

		case EOpType::InitTransient:
			Data_InitTransient.Resource->InitTransient(Data_InitTransient.DebugName);
			delete[] Data_InitTransient.DebugName;
			Data_InitTransient.Resource->ReleaseOpRef();
			break;

		case EOpType::Assert:
			Data_Assert.Identity.Resource->EnumerateSubresources(Data_Assert.Identity.SubresourceRange, [this, &Queue](FSubresourceState& State, FSubresourceIndex const& SubresourceIndex)
			{
				State.Assert(
					Data_Assert.Identity.Resource,
					SubresourceIndex,
					Data_Assert.RequiredState,
					Queue.bAllowAllUAVsOverlap);
			});
			Data_Assert.Identity.Resource->ReleaseOpRef();
			break;

		case EOpType::Signal:
			check(Data_Signal.Fence->SrcPipe == Queue.Pipeline);
			Data_Signal.Fence->bSignaled = true;
			Data_Signal.Fence->FenceValue = ++Queue.FenceValue;
			break;

		case EOpType::Wait:
			{
				FFence* Fence = Data_Wait.Fence;
				check(Fence->DstPipe == Queue.Pipeline);
				if (!Fence->bSignaled)
				{
					return false;
				}

				Queue.MaxAwaitedFenceValues[Fence->SrcPipe] = FMath::Max(Fence->FenceValue, Queue.MaxAwaitedFenceValues[Fence->SrcPipe]);

				// The fence has been completed. Free it now.
				delete Fence;
			}
			break;

		case EOpType::AllUAVsOverlap:
			RHI_VALIDATION_CHECK(Queue.bAllowAllUAVsOverlap != Data_AllUAVsOverlap.bAllow, *GetReasonString_MismatchedAllUAVsOverlapCall(Data_AllUAVsOverlap.bAllow));
			Queue.bAllowAllUAVsOverlap = Data_AllUAVsOverlap.bAllow;
			break;

		case EOpType::SpecificUAVOverlap:
			Data_SpecificUAVOverlap.Identity.Resource->EnumerateSubresources(Data_SpecificUAVOverlap.Identity.SubresourceRange, [this, &Queue](FSubresourceState& State, FSubresourceIndex const& SubresourceIndex)
			{
				State.SpecificUAVOverlap(
					Data_SpecificUAVOverlap.Identity.Resource,
					SubresourceIndex,
					Queue.Pipeline,
					Data_SpecificUAVOverlap.bAllow);
			});
			Data_SpecificUAVOverlap.Identity.Resource->ReleaseOpRef();
			break;
		}

		return true;
	}

	void FTracker::AddOp(const RHIValidation::FOperation& Op)
	{
		if (GRHICommandList.Bypass() && CurrentList.IsEmpty())
		{
			if (Op.Replay(GetQueue(Pipeline)))
			{
				return;
			}
		}

		CurrentList.Add(Op);
	}

	void FOpQueueState::AppendOps(FValidationCommandList* CommandList)
	{
		Ops.Emplace(MoveTemp(CommandList->CompletedOpList));
	}

	bool FOpQueueState::Execute()
	{
		if (!Ops.Num())
			return false;

		bool bProgressMade = false;
		FRHIValidationQueueScope Scope(*this);

		while (Ops.Num())
		{
			for (FOpsList& List = Ops[0]; List.ReplayPos < List.Num(); ++List.ReplayPos)
			{
				if (!List[List.ReplayPos].Replay(*this))
				{
					// Queue is blocked
					return bProgressMade;
				}

				bProgressMade = true;
			}

			Ops.RemoveAt(0);
		}

		return bProgressMade;
	}

	void FTracker::SubmitValidationOps(ERHIPipeline Pipeline, TArray<RHIValidation::FOperation>&& Ops)
	{
		GetQueue(Pipeline).Ops.Emplace(MoveTemp(Ops));

		// Keep executing until no more progress is made,
		// (i.e. until queues are empty or blocked on fences).
		bool bProgressMade;
		do
		{
			bProgressMade = false;
			for (FOpQueueState& CurrentQueue : OpQueues)
			{
				bProgressMade |= CurrentQueue.Execute();
			}

		} while (bProgressMade);
	}


	void FUniformBufferResource::InitLifetimeTracking(uint64 FrameID, const void* Contents, EUniformBufferUsage Usage)
	{
		AllocatedFrameID = FrameID;
		UniformBufferUsage = Usage;
		bContainsNullContents = Contents == nullptr;

#if CAPTURE_UNIFORMBUFFER_ALLOCATION_BACKTRACES
		AllocatedCallstack = (UniformBufferUsage != UniformBuffer_MultiFrame) ? RHIValidation::CaptureBacktrace() : nullptr;
#else
		AllocatedCallstack = nullptr;
#endif
	}

	void FUniformBufferResource::UpdateAllocation(uint64 FrameID)
	{
		AllocatedFrameID = FrameID;
		bContainsNullContents = false;

#if CAPTURE_UNIFORMBUFFER_ALLOCATION_BACKTRACES
		AllocatedCallstack = (UniformBufferUsage != UniformBuffer_MultiFrame) ? RHIValidation::CaptureBacktrace() : nullptr;
#else
		AllocatedCallstack = nullptr;
#endif
	}

	void FUniformBufferResource::ValidateLifeTime()
	{
		FValidationRHI* ValidateRHI = (FValidationRHI*)GDynamicRHI;

		RHI_VALIDATION_CHECK(bContainsNullContents == false, TEXT("Uniform buffer created with null contents is now being bound for rendering on an RHI context. The contents must first be updated."));

		if (UniformBufferUsage != UniformBuffer_MultiFrame && AllocatedFrameID < ValidateRHI->RHIThreadFrameID)
		{
			FString ErrorMessage = "Non MultiFrame Uniform buffer has been allocated in a previous frame. The data could have been deleted already!";
			if (AllocatedCallstack != nullptr)
			{
				ErrorMessage += FString::Printf(TEXT("\nAllocation callstack: (void**)0x%p,32"), AllocatedCallstack);
			}
			RHI_VALIDATION_CHECK(false, *ErrorMessage);
		}
	}

	FOpQueueState FTracker::OpQueues[int32(ERHIPipeline::Num)]
	{
		ERHIPipeline::Graphics,
		ERHIPipeline::AsyncCompute
	};

	FOpQueueState& FTracker::GetQueue(ERHIPipeline Pipeline)
	{
		uint32 Index;
		switch (Pipeline)
		{
		default: checkNoEntry(); [[fallthrough]];
		case ERHIPipeline::Graphics:
			Index = 0;
			break;

		case ERHIPipeline::AsyncCompute:
			Index = 1;
			break;
		}

		return OpQueues[Index];
	}

	void* CaptureBacktrace()
	{
		// Back traces will leak. Don't leave this turned on.
		uint64* Backtrace = new uint64[NumStackFrames];
		FPlatformStackWalk::CaptureStackBackTrace(Backtrace, NumStackFrames);

		return Backtrace;
	}

	bool ValidateDimension(EShaderCodeResourceBindingType Type, FRHIViewDesc::EDimension Dimension, ERHITexturePlane TexturePlane, bool SRV)
	{
		// Ignore invalid types
		if (Type == EShaderCodeResourceBindingType::Invalid)
		{
			return true;
		}

		if (IsResourceBindingTypeSRV(Type) != SRV)
		{
			return false;
		}

		if (Type == EShaderCodeResourceBindingType::RWStructuredBuffer || Type == EShaderCodeResourceBindingType::StructuredBuffer)
		{
			return TexturePlane == ERHITexturePlane::HTile;
		}

		if (Type == EShaderCodeResourceBindingType::RWByteAddressBuffer || Type == EShaderCodeResourceBindingType::ByteAddressBuffer)
		{
			return TexturePlane == ERHITexturePlane::CMask;
		}
		
		if (Type == EShaderCodeResourceBindingType::RWBuffer || Type == EShaderCodeResourceBindingType::Buffer)
		{
			return TexturePlane == ERHITexturePlane::PrimaryCompressed || TexturePlane == ERHITexturePlane::CMask;
		}

		if (Type == EShaderCodeResourceBindingType::Texture2D || Type == EShaderCodeResourceBindingType::RWTexture2D || Type == EShaderCodeResourceBindingType::Texture2DMS)
		{
			return Dimension == FRHIViewDesc::EDimension::Texture2D;
		}

		if (Type == EShaderCodeResourceBindingType::Texture2DArray || Type == EShaderCodeResourceBindingType::RWTexture2DArray)
		{
			return Dimension == FRHIViewDesc::EDimension::Texture2DArray || Dimension == FRHIViewDesc::EDimension::TextureCube;
		}

		if (Type == EShaderCodeResourceBindingType::Texture3D || Type == EShaderCodeResourceBindingType::RWTexture3D)
		{
			return Dimension == FRHIViewDesc::EDimension::Texture3D;
		}

		if (Type == EShaderCodeResourceBindingType::TextureCube || Type == EShaderCodeResourceBindingType::RWTextureCube)
		{
			return Dimension == FRHIViewDesc::EDimension::TextureCube;
		}

		if (Type == EShaderCodeResourceBindingType::TextureCubeArray)
		{
			return Dimension == FRHIViewDesc::EDimension::TextureCubeArray;
		}

		if (Type == EShaderCodeResourceBindingType::Texture2DMultiView || Type == EShaderCodeResourceBindingType::RWTexture2DMultiView)
		{
			checkf(false, TEXT("Texture2DMultiView resource binding types should be parsed into Texture2D or Texture2DArray types during a shader's compilation, and so should not be added to the shader's debug type validation data."));
		}

		return false;
	}

	bool ValidateDimension(EShaderCodeResourceBindingType Type, ETextureDimension Dimension, bool SRV)
	{
		// Ignore invalid types
		if (Type == EShaderCodeResourceBindingType::Invalid)
		{
			return true;
		}

		if (Type == EShaderCodeResourceBindingType::Texture2D || Type == EShaderCodeResourceBindingType::RWTexture2D || Type == EShaderCodeResourceBindingType::Texture2DMS)
		{
			return Dimension == ETextureDimension::Texture2D;
		}

		if (Type == EShaderCodeResourceBindingType::Texture2DArray || Type == EShaderCodeResourceBindingType::RWTexture2DArray)
		{
			return Dimension == ETextureDimension::Texture2DArray || Dimension == ETextureDimension::TextureCube;
		}

		if (Type == EShaderCodeResourceBindingType::Texture3D || Type == EShaderCodeResourceBindingType::RWTexture3D)
		{
			return Dimension == ETextureDimension::Texture3D;
		}

		if (Type == EShaderCodeResourceBindingType::TextureCube || Type == EShaderCodeResourceBindingType::RWTextureCube)
		{
			return Dimension == ETextureDimension::TextureCube;
		}

		if (Type == EShaderCodeResourceBindingType::TextureCubeArray)
		{
			return Dimension == ETextureDimension::TextureCubeArray;
		}

		if (Type == EShaderCodeResourceBindingType::Texture2DMultiView || Type == EShaderCodeResourceBindingType::RWTexture2DMultiView)
		{
			checkf(false, TEXT("Texture2DMultiView resource binding types should be parsed into Texture2D or Texture2DArray types during a shader's compilation, and so should not be added to the shader's debug type validation data."));
		}

		return false;
	}

	bool ValidateBuffer(EShaderCodeResourceBindingType Type, FRHIViewDesc::EBufferType BufferType, bool SRV)
	{
		// Ignore invalid types
		if (Type == EShaderCodeResourceBindingType::Invalid)
		{
			return true;
		}

		if (IsResourceBindingTypeSRV(Type) != SRV)
		{
			return false;
		}

		if (Type == EShaderCodeResourceBindingType::ByteAddressBuffer || Type == EShaderCodeResourceBindingType::RWByteAddressBuffer)
		{
			return BufferType == FRHIViewDesc::EBufferType::Raw;
		}
		else if (Type == EShaderCodeResourceBindingType::StructuredBuffer || Type == EShaderCodeResourceBindingType::RWStructuredBuffer)
		{
			return BufferType == FRHIViewDesc::EBufferType::Structured || BufferType == FRHIViewDesc::EBufferType::AccelerationStructure;
		}
		else if (Type == EShaderCodeResourceBindingType::Buffer || Type == EShaderCodeResourceBindingType::RWBuffer)
		{
			return BufferType == FRHIViewDesc::EBufferType::Typed;
		}
		else if (Type == EShaderCodeResourceBindingType::RaytracingAccelerationStructure)
		{
			return BufferType == FRHIViewDesc::EBufferType::AccelerationStructure;
		}
		
		return false;
	}

	/** Validates that the SRV is conform to what the shader expects */
	void ValidateShaderResourceView(const FRHIShader* RHIShaderBase, uint32 BindIndex, const FRHIShaderResourceView* SRV)
	{
#if RHI_INCLUDE_SHADER_DEBUG_DATA
		if (SRV)
		{
			auto const ViewIdentity = SRV->GetViewIdentity();

			static const auto GetSRVName = [](const FRHIShaderResourceView* SRV, auto& ViewIdentity) -> FString
			{
				FString SRVName;
				if (ViewIdentity.Resource)
				{
					SRVName = ViewIdentity.Resource->GetDebugName();
				}
				if (SRVName.IsEmpty())
				{
					SRVName = SRV->GetOwnerName().ToString();
				}

				return SRVName;
			};

			// DebugStrideValidationData is supposed to be already sorted
			static const auto ShaderCodeValidationStridePredicate = [](const FShaderCodeValidationStride& lhs, const FShaderCodeValidationStride& rhs) -> bool { return lhs.BindPoint < rhs.BindPoint; };
			FShaderCodeValidationStride SRVValidationStride = { BindIndex , ViewIdentity.Stride };

			int32 FoundIndex =  Algo::BinarySearch(RHIShaderBase->DebugStrideValidationData, SRVValidationStride, ShaderCodeValidationStridePredicate);
			if (FoundIndex != INDEX_NONE)
			{
				FString SRVName = GetSRVName(SRV, ViewIdentity);
				uint16 ExpectedStride = RHIShaderBase->DebugStrideValidationData[FoundIndex].Stride;
				if (ExpectedStride != SRVValidationStride.Stride && SRV->GetDesc().Buffer.SRV.BufferType != FRHIViewDesc::EBufferType::AccelerationStructure)
				{
					
					FString ErrorMessage = FString::Printf(TEXT("Shader %s: Buffer stride for \"%s\" must match structure size declared in the shader"), RHIShaderBase->GetShaderName(), *SRVName);
					ErrorMessage += FString::Printf(TEXT("\nBind point: %d, HLSL size: %d, Buffer Size: %d"), BindIndex, ExpectedStride, SRVValidationStride.Stride);
					RHI_VALIDATION_CHECK(false, *ErrorMessage);
				}
			}

			// Validate Type
			if (!RHIShaderBase->DebugSRVTypeValidationData.Num())
				return;

			static const auto ShaderCodeValidationTypePredicate = [](const FShaderCodeValidationType& lhs, const FShaderCodeValidationType& rhs) -> bool { return lhs.BindPoint < rhs.BindPoint; };
			
			FShaderCodeValidationType SRVValidationType = { BindIndex , EShaderCodeResourceBindingType::Invalid };
			FoundIndex = Algo::BinarySearch(RHIShaderBase->DebugSRVTypeValidationData, SRVValidationType, ShaderCodeValidationTypePredicate);

			if (FoundIndex != INDEX_NONE)
			{
				EShaderCodeResourceBindingType ExpectedType = RHIShaderBase->DebugSRVTypeValidationData[FoundIndex].Type;

				if (SRV->IsTexture())
				{
					if (!ValidateDimension(ExpectedType, SRV->GetDesc().Texture.SRV.Dimension, SRV->GetDesc().Texture.SRV.Plane, true))
					{
						FString SRVName = GetSRVName(SRV, ViewIdentity);
						FString ErrorMessage = FString::Printf(TEXT("Shader %s: Dimension for SRV \"%s\" must match type declared in the shader"), RHIShaderBase->GetShaderName(), *SRVName);
						ErrorMessage += FString::Printf(TEXT("\nBind point: %d, HLSL Type: %s, Actual Dimension: %s"),
							BindIndex, 
							GetShaderCodeResourceBindingTypeName(ExpectedType),
							FRHIViewDesc::GetTextureDimensionString(SRV->GetDesc().Texture.SRV.Dimension));
						RHI_VALIDATION_CHECK(false, *ErrorMessage);
					}
				}
				else if (SRV->IsBuffer())
				{
					if (!ValidateBuffer(ExpectedType, SRV->GetDesc().Buffer.SRV.BufferType, true))
					{
						FString SRVName = GetSRVName(SRV, ViewIdentity);
						FString ErrorMessage = FString::Printf(TEXT("Shader %s: Buffer type for SRV \"%s\" must match buffer type declared in the shader"), RHIShaderBase->GetShaderName(), *SRVName);
						ErrorMessage += FString::Printf(TEXT("\nBind point: %d, HLSL Type: %s, Actual Type: %s"),
							BindIndex,
							GetShaderCodeResourceBindingTypeName(ExpectedType),
							FRHIViewDesc::GetBufferTypeString(SRV->GetDesc().Buffer.SRV.BufferType));
						RHI_VALIDATION_CHECK(false, *ErrorMessage);
					}
				}
			}
			else
			{ 
				FString SRVName = GetSRVName(SRV, ViewIdentity);
				FString ErrorMessage = FString::Printf(TEXT("Shader %s: No bind point found for SRV \"%s\" possible UAV/SRV mismatch"), RHIShaderBase->GetShaderName(), *SRVName);
				if (SRV->IsTexture())
				{
					ErrorMessage += FString::Printf(TEXT("\nBind point: %d, Type: %s"),
						BindIndex,
						FRHIViewDesc::GetTextureDimensionString(SRV->GetDesc().Texture.SRV.Dimension));
				}
				else
				{
					ErrorMessage += FString::Printf(TEXT("\nBind point: %d, Type: %s"),
						BindIndex,
						FRHIViewDesc::GetBufferTypeString(SRV->GetDesc().Buffer.SRV.BufferType));
				}
				RHI_VALIDATION_CHECK(false, *ErrorMessage);
			}
		}
#endif
	}

	/** Validates that the SRV is conform to what the shader expects */
	void ValidateShaderResourceView(const FRHIShader* RHIShaderBase, uint32 BindIndex, const FRHITexture* Texture)
	{
#if RHI_INCLUDE_SHADER_DEBUG_DATA
		if (Texture)
		{
			// Validate Type
			if (!RHIShaderBase->DebugSRVTypeValidationData.Num())
				return;

			static const auto ShaderCodeValidationTypePredicate = [](const FShaderCodeValidationType& lhs, const FShaderCodeValidationType& rhs) -> bool { return lhs.BindPoint < rhs.BindPoint; };

			FShaderCodeValidationType SRVValidationType = { BindIndex , EShaderCodeResourceBindingType::Invalid };
			int32 FoundIndex = Algo::BinarySearch(RHIShaderBase->DebugSRVTypeValidationData, SRVValidationType, ShaderCodeValidationTypePredicate);

			if (FoundIndex != INDEX_NONE)
			{
				EShaderCodeResourceBindingType ExpectedType = RHIShaderBase->DebugSRVTypeValidationData[FoundIndex].Type;

				if (!ValidateDimension(ExpectedType, Texture->GetDesc().Dimension, true))
				{
					FString ErrorMessage = FString::Printf(TEXT("Shader %s: Dimension for Texture %s at BindIndex \"%d\" must match type declared in the shader"), 
						RHIShaderBase->GetShaderName(),
						*Texture->GetName().ToString(),
						BindIndex);
					ErrorMessage += FString::Printf(TEXT("\nBind point: %d, HLSL Type: %s, Actual Dimension: %s"),
						BindIndex,
						GetShaderCodeResourceBindingTypeName(ExpectedType),
						GetTextureDimensionString(Texture->GetDesc().Dimension));
					RHI_VALIDATION_CHECK(false, *ErrorMessage);
				}
			}
			else
			{
				FString ErrorMessage = FString::Printf(TEXT("Shader %s: No bind point found at BindIndex \"%d\" possible UAV/SRV mismatch"), RHIShaderBase->GetShaderName(), BindIndex);
					ErrorMessage += FString::Printf(TEXT("\nBind point: %d, Type: %s"),
						BindIndex,
						GetTextureDimensionString(Texture->GetDesc().Dimension));
				RHI_VALIDATION_CHECK(false, *ErrorMessage);
			}
		}
#endif
	}

	/** Validates that the UAV is conform to what the shader expects */
	void ValidateUnorderedAccessView(const FRHIShader* RHIShaderBase, uint32 BindIndex, const FRHIUnorderedAccessView* UAV)
	{
#if RHI_INCLUDE_SHADER_DEBUG_DATA
		if (UAV)
		{
			auto const ViewIdentity = UAV->GetViewIdentity();

			static const auto GetUAVName = [](const FRHIUnorderedAccessView* UAV, auto& ViewIdentity) -> FString
			{
				FString UAVName;
				if (ViewIdentity.Resource)
				{
					UAVName = ViewIdentity.Resource->GetDebugName();
				}
				if (UAVName.IsEmpty())
				{
					UAVName = UAV->GetOwnerName().ToString();
				}

				return UAVName;
			};

			// Validate Type
			if (!RHIShaderBase->DebugUAVTypeValidationData.Num())
				return;

			static const auto ShaderCodeValidationTypePredicate = [](const FShaderCodeValidationType& lhs, const FShaderCodeValidationType& rhs) -> bool { return lhs.BindPoint < rhs.BindPoint; };

			FShaderCodeValidationType SRVValidationType = { BindIndex , EShaderCodeResourceBindingType::Invalid };
			int32 FoundIndex = Algo::BinarySearch(RHIShaderBase->DebugUAVTypeValidationData, SRVValidationType, ShaderCodeValidationTypePredicate);

			if (FoundIndex != INDEX_NONE)
			{
				EShaderCodeResourceBindingType ExpectedType = RHIShaderBase->DebugUAVTypeValidationData[FoundIndex].Type;

				if (UAV->IsTexture())
				{
					if (!ValidateDimension(ExpectedType, UAV->GetDesc().Texture.UAV.Dimension, UAV->GetDesc().Texture.UAV.Plane, false))
					{
						FString UAVName = GetUAVName(UAV, ViewIdentity);
						FString ErrorMessage = FString::Printf(TEXT("Shader %s: Dimension for UAV \"%s\" must match type declared in the shader"), RHIShaderBase->GetShaderName(), *UAVName);
						ErrorMessage += FString::Printf(TEXT("\nBind point: %d, HLSL Type: %s, Actual Dimension: %s"), 
							BindIndex,
							GetShaderCodeResourceBindingTypeName(ExpectedType),
							FRHIViewDesc::GetTextureDimensionString(UAV->GetDesc().Texture.SRV.Dimension));
						RHI_VALIDATION_CHECK(false, *ErrorMessage);
					}
				}
				else if (UAV->IsBuffer())
				{
					if (!ValidateBuffer(ExpectedType, UAV->GetDesc().Buffer.UAV.BufferType, false))
					{
						FString UAVName = GetUAVName(UAV, ViewIdentity);
						FString ErrorMessage = FString::Printf(TEXT("Shader %s: Buffer type for UAV \"%s\" must match buffer type declared in the shader"), RHIShaderBase->GetShaderName(), *UAVName);
						ErrorMessage += FString::Printf(TEXT("\nBind point: %d, HLSL Type: %s, Actual Type: %s"),
							BindIndex,
							GetShaderCodeResourceBindingTypeName(ExpectedType),
							FRHIViewDesc::GetBufferTypeString(UAV->GetDesc().Buffer.UAV.BufferType));
						RHI_VALIDATION_CHECK(false, *ErrorMessage);
					}
				}
			}
			else
			{
				FString UAVName = GetUAVName(UAV, ViewIdentity);
				FString ErrorMessage = FString::Printf(TEXT("Shader %s: No bind point found for UAV \"%s\" possible UAV/SRV mismatch"), RHIShaderBase->GetShaderName(), *UAVName);

				if (UAV->IsTexture())
				{
					ErrorMessage += FString::Printf(TEXT("\nBind point: %d, Type: %s"),
						BindIndex,
						FRHIViewDesc::GetTextureDimensionString(UAV->GetDesc().Texture.SRV.Dimension));
				}
				else
				{
					ErrorMessage += FString::Printf(TEXT("\nBind point: %d, Type: %s"),
						BindIndex,
						FRHIViewDesc::GetBufferTypeString(UAV->GetDesc().Buffer.SRV.BufferType));
				}
				RHI_VALIDATION_CHECK(false, *ErrorMessage);
			}
		}
#endif
	}

	/** Validates that the Uniform conforms to what the shader expects */
	void ValidateUniformBuffer(const FRHIShader* RHIShaderBase, uint32 BindIndex, FRHIUniformBuffer* UB)
	{
		if (!UB)
		{
			return;
		}

		const FRHIUniformBufferLayout& Layout = UB->GetLayout();

		const TArray<uint32>& LayoutHashes = RHIShaderBase->GetShaderResourceTable().ResourceTableLayoutHashes;
		if (BindIndex >= (uint32)LayoutHashes.Num())
		{
			FString ErrorMessage = FString::Printf(TEXT("Shader %s: Invalid bind index %u for uniform buffer \"%s\" (UB table size: %d)"), RHIShaderBase->GetShaderName(), BindIndex, *Layout.GetDebugName(), LayoutHashes.Num());
			RHI_VALIDATION_CHECK(false, *ErrorMessage);
			return;
		}

		uint32 ShaderTableHash = LayoutHashes[BindIndex];
		uint32 UniformBufferHash = Layout.GetHash();
		if (ShaderTableHash != 0 && UniformBufferHash != ShaderTableHash)
		{
			FString ErrorMessage = FString::Printf(TEXT("Shader %s: Invalid layout hash %u for uniform buffer \"%s\" at bind index %u, expecting %u"), RHIShaderBase->GetShaderName(), UniformBufferHash, *Layout.GetDebugName(), BindIndex, ShaderTableHash);
			RHI_VALIDATION_CHECK(false, *ErrorMessage);
		}

#if RHI_INCLUDE_SHADER_DEBUG_DATA
		{
			// Validate Type
			static const auto ShaderCodeValidationUBSizePredicate = [](const FShaderCodeValidationUBSize& lhs, const FShaderCodeValidationUBSize& rhs) -> bool { return lhs.BindPoint < rhs.BindPoint; };

			FShaderCodeValidationUBSize SRVValidationSize = { BindIndex , 0 };
			int32 FoundIndex = Algo::BinarySearch(RHIShaderBase->DebugUBSizeValidationData, SRVValidationSize, ShaderCodeValidationUBSizePredicate);
			if (FoundIndex != INDEX_NONE)
			{
				uint32_t Size = RHIShaderBase->DebugUBSizeValidationData[FoundIndex].Size;

				if(Size > 0 && Size > UB->GetSize())
				{
					FString ErrorMessage = FString::Printf(TEXT("Shader %s: Uniform buffer \"%s\" has unexpected size"), RHIShaderBase->GetShaderName(), *Layout.GetDebugName());
					ErrorMessage += FString::Printf(TEXT("\nBind point: %d, HLSL size: %d, Actual size: %d"), BindIndex, Size, UB->GetSize());
					RHI_VALIDATION_CHECK(false, *ErrorMessage);
				}
			}
		}
#endif
	}
}


//-----------------------------------------------------------------------------
//	Validation Transient Resource Allocator
//-----------------------------------------------------------------------------

#define TRANSIENT_RESOURCE_LOG_PREFIX_REASON(ReasonString) TEXT("RHI validation failed: " ReasonString "\n\n")\
	TEXT("--------------------------------------------------------------------\n")\
	TEXT("         RHI Transient Resource Allocation Validation Error		  \n")\
	TEXT("--------------------------------------------------------------------\n")\
	TEXT("\n")

#define TRANSIENT_RESOURCE_LOG_SUFFIX TEXT("\n")\
	TEXT("--------------------------------------------------------------------\n")\
	TEXT("\n")

FValidationTransientResourceAllocator::~FValidationTransientResourceAllocator()
{
	checkf(!RHIAllocator, TEXT("Release was not called on FRHITransientResourceAllocator."));
}

void FValidationTransientResourceAllocator::SetCreateMode(ERHITransientResourceCreateMode InCreateMode)
{
	// Validation intentionally doesn't pass through the create mode. It's always inline.
}

FRHITransientTexture* FValidationTransientResourceAllocator::CreateTexture(const FRHITextureCreateInfo& InCreateInfo, const TCHAR* InDebugName, const FRHITransientAllocationFences& Fences)
{
	check(FRHITextureCreateInfo::CheckValidity(InCreateInfo, InDebugName));

	FRHITransientTexture* TransientTexture = RHIAllocator->CreateTexture(InCreateInfo, InDebugName, Fences);

	if (!TransientTexture)
	{
		return nullptr;
	}

	FRHITexture* RHITexture = TransientTexture->GetRHI();

	checkf(!AllocatedResourceMap.Contains(RHITexture), TEXT("Platform RHI returned an FRHITexture (0x%p) which was already in use by another transient texture resource on this allocator (0x%p)."), RHITexture, this);
	AllocatedResourceMap.Add(RHITexture, { InDebugName, FAllocatedResourceData::EType::Texture });

	RHIValidation::FResource* Resource = RHITexture->GetTrackerResource();
	check(Resource);

	if (!Resource->IsBarrierTrackingInitialized())
	{
		RHITexture->InitBarrierTracking(InCreateInfo.NumMips, InCreateInfo.ArraySize * (InCreateInfo.IsTextureCube() ? 6 : 1), InCreateInfo.Format, InCreateInfo.Flags, ERHIAccess::Discard, InDebugName);
	}
	else
	{
		// The existing resource returned by the platform RHI should have the layout we expect.
		RHITexture->CheckValidationLayout(InCreateInfo.NumMips, InCreateInfo.ArraySize * (InCreateInfo.IsTextureCube() ? 6 : 1), InCreateInfo.Format);

		// @todo dev-pr debug names are global properties of resources. It seems wrong to require the graphics pipe here. Decouple this.
		// @todo we should validate the resource was in the Discard state rather than forcing it
		PendingPipelineOps[ERHIPipeline::Graphics].Emplace(RHIValidation::FOperation::InitTransient(Resource, InDebugName));
	}

	return TransientTexture;
}

FRHITransientBuffer* FValidationTransientResourceAllocator::CreateBuffer(const FRHIBufferCreateInfo& InCreateInfo, const TCHAR* InDebugName, const FRHITransientAllocationFences& Fences)
{
	FRHITransientBuffer* TransientBuffer = RHIAllocator->CreateBuffer(InCreateInfo, InDebugName, Fences);

	if (!TransientBuffer)
	{
		return nullptr;
	}

	FRHIBuffer* RHIBuffer = TransientBuffer->GetRHI();

	checkf(!AllocatedResourceMap.Contains(RHIBuffer), TEXT("Platform RHI returned an FRHIBuffer (0x%p) which was already in use by another transient buffer resource on this allocator (0x%p)."), RHIBuffer, this);
	AllocatedResourceMap.Add(RHIBuffer, { InDebugName, FAllocatedResourceData::EType::Buffer });

	if (!RHIBuffer->IsBarrierTrackingInitialized())
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(InDebugName, InCreateInfo)
			.SetInitialState(ERHIAccess::Discard);

		RHIBuffer->InitBarrierTracking(CreateDesc);
	}
	else
	{
		// @todo dev-pr debug names are global properties of resources. It seems wrong to require the graphics pipe here. Decouple this.
		// @todo we should validate the resource was in the Discard state rather than forcing it
		PendingPipelineOps[ERHIPipeline::Graphics].Emplace(RHIValidation::FOperation::InitTransient(RHIBuffer, InDebugName));
	}

	return TransientBuffer;
}

void FValidationTransientResourceAllocator::DeallocateMemory(FRHITransientTexture* InTransientTexture, const FRHITransientAllocationFences& Fences)
{
	check(InTransientTexture);

	RHIAllocator->DeallocateMemory(InTransientTexture, Fences);

	checkf(AllocatedResourceMap.Contains(InTransientTexture->GetRHI()), TEXT("DeallocateMemory called on texture %s, but it is not marked as allocated."), InTransientTexture->GetName());
	AllocatedResourceMap.Remove(InTransientTexture->GetRHI());
}

void FValidationTransientResourceAllocator::DeallocateMemory(FRHITransientBuffer* InTransientBuffer, const FRHITransientAllocationFences& Fences)
{
	check(InTransientBuffer);

	RHIAllocator->DeallocateMemory(InTransientBuffer, Fences);

	checkf(AllocatedResourceMap.Contains(InTransientBuffer->GetRHI()), TEXT("DeallocateMemory called on buffer %s, but it is not marked as allocated."), InTransientBuffer->GetName());
	AllocatedResourceMap.Remove(InTransientBuffer->GetRHI());
}

void FValidationTransientResourceAllocator::Flush(FRHICommandListImmediate& RHICmdList, FRHITransientAllocationStats* OutHeapStats)
{
	// Insert pending ops into context trackers
	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		if (PendingPipelineOps[Pipeline].Num())
		{
			FRHICommandListScopedPipeline Scope(RHICmdList, Pipeline);
			RHICmdList.EnqueueLambda([Pipeline, PendingOps = MoveTemp(PendingPipelineOps[Pipeline])](FRHICommandListImmediate& InRHICmdList)
			{
				IRHIComputeContext& Context = InRHICmdList.GetComputeContext().GetLowestLevelContext();
				Context.Tracker->AddOps(PendingOps);
			});
		}
	}

	RHIAllocator->Flush(RHICmdList, OutHeapStats);
}

void FValidationTransientResourceAllocator::Release(FRHICommandListImmediate& RHICmdList)
{
	RHIAllocator->Release(RHICmdList);
	RHIAllocator = nullptr;
	delete this;
}

void ValidateShaderParameters(FRHIShader* RHIShader, RHIValidation::FTracker* Tracker, RHIValidation::FStaticUniformBuffers& StaticUniformBuffers, RHIValidation::FStageBoundUniformBuffers& BoundUniformBuffers, TConstArrayView<FRHIShaderParameterResource> InParameters, ERHIAccess InRequiredAccess, RHIValidation::EUAVMode InRequiredUAVMode)
{
	for (const FRHIShaderParameterResource& Parameter : InParameters)
	{
		switch (Parameter.Type)
		{
		case FRHIShaderParameterResource::EType::Texture:
			if (FRHITexture* Texture = static_cast<FRHITexture*>(Parameter.Resource))
			{
				if (GRHIValidationEnabled)
				{
					RHIValidation::ValidateShaderResourceView(RHIShader, Parameter.Index, Texture);
				}
				Tracker->Assert(Texture->GetWholeResourceIdentitySRV(), InRequiredAccess);
			}
			break;
		case FRHIShaderParameterResource::EType::ResourceView:
			if (FRHIShaderResourceView* SRV = static_cast<FRHIShaderResourceView*>(Parameter.Resource))
			{
				if (GRHIValidationEnabled)
				{
					RHIValidation::ValidateShaderResourceView(RHIShader, Parameter.Index, SRV);
				}
				Tracker->Assert(SRV->GetViewIdentity(), InRequiredAccess);
			}
			break;
		case FRHIShaderParameterResource::EType::UnorderedAccessView:
			if (FRHIUnorderedAccessView* UAV = static_cast<FRHIUnorderedAccessView*>(Parameter.Resource))
			{
				if (GRHIValidationEnabled)
				{
					RHIValidation::ValidateUnorderedAccessView(RHIShader, Parameter.Index, UAV);
				}
				Tracker->AssertUAV(static_cast<FRHIUnorderedAccessView*>(Parameter.Resource), InRequiredUAVMode, Parameter.Index);
			}
			break;
		case FRHIShaderParameterResource::EType::Sampler:
			// No validation
			break;
		case FRHIShaderParameterResource::EType::UniformBuffer:
			if (FRHIUniformBuffer* UniformBuffer = static_cast<FRHIUniformBuffer*>(Parameter.Resource))
			{
				if (GRHIValidationEnabled)
				{
					RHIValidation::ValidateUniformBuffer(RHIShader, Parameter.Index, UniformBuffer);
				}

				BoundUniformBuffers.Bind(Parameter.Index, UniformBuffer);
				StaticUniformBuffers.ValidateSetShaderUniformBuffer(UniformBuffer);
			}
			break;
		case FRHIShaderParameterResource::EType::ResourceCollection:
			if (const FRHIResourceCollection* ResourceCollection = static_cast<const FRHIResourceCollection*>(Parameter.Resource))
			{
				for (const FRHIResourceCollectionMember& Member : ResourceCollection->GetMembers())
				{
					switch (Member.Type)
					{
					case FRHIResourceCollectionMember::EType::Texture:
						if (FRHITexture* Texture = static_cast<FRHITexture*>(Member.Resource))
						{
							Tracker->Assert(Texture->GetWholeResourceIdentitySRV(), InRequiredAccess);
						}
						break;
					case FRHIResourceCollectionMember::EType::TextureReference:
						if (FRHITextureReference* Texture = static_cast<FRHITextureReference*>(Member.Resource))
						{
							Tracker->Assert(Texture->GetWholeResourceIdentitySRV(), InRequiredAccess);
						}
						break;
					case FRHIResourceCollectionMember::EType::ShaderResourceView:
						if (FRHIShaderResourceView* SRV = static_cast<FRHIShaderResourceView*>(Member.Resource))
						{
							Tracker->Assert(SRV->GetViewIdentity(), InRequiredAccess);
						}
						break;
					default:
						break;
					}
				}
			}
			break;
		case FRHIShaderParameterResource::EType::DescriptorRange:
			if (const FRHIDescriptorRange* DescriptorRange = static_cast<const FRHIDescriptorRange*>(Parameter.Resource))
			{
				if (GRHIValidationEnabled)
				{
					RHI_VALIDATION_CHECK(DescriptorRange->IsValid(),
						TEXT("Descriptor range bound to shader is not valid. Allocation may have failed."));
				}

				for (int32 ResourceIndex = 0; ResourceIndex < DescriptorRange->GetNumResources(); ResourceIndex++)
				{
					const FRHIDescriptorResource Resource = DescriptorRange->GetResource(ResourceIndex);
					switch (Resource.Type)
					{
					case FRHIDescriptorResource::EType::None:
						break;
					case FRHIDescriptorResource::EType::Texture:
						if (FRHITexture* Texture = static_cast<FRHITexture*>(Resource.Resource))
						{
							Tracker->Assert(Texture->GetWholeResourceIdentitySRV(), InRequiredAccess);
						}
						break;
					case FRHIDescriptorResource::EType::ResourceView:
						if (FRHIShaderResourceView* SRV = static_cast<FRHIShaderResourceView*>(Resource.Resource))
						{
							Tracker->Assert(SRV->GetViewIdentity(), InRequiredAccess);
						}
						break;
					case FRHIDescriptorResource::EType::UnorderedAccessView:
						if (FRHIUnorderedAccessView* UAV = static_cast<FRHIUnorderedAccessView*>(Resource.Resource))
						{
							Tracker->AssertUAV(UAV, InRequiredUAVMode, Parameter.Index);
						}
						break;
					default:
						checkf(false, TEXT("Unhandled DescriptorRange resource type: %d"), static_cast<int32>(Resource.Type));
						break;
					}
				}
			}
			break;
		default:
			checkf(false, TEXT("Unhandled resource type?"));
			break;
		}
	}
}

#endif	// ENABLE_RHI_VALIDATION
