// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTracing.h"
#include "Trace/Trace.inl"
#include "HAL/PlatformTime.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "String/ParseTokens.h"
#include "RHIDefinitions.h"

UE_TRACE_CHANNEL_DEFINE(RenderTraceChannel,
	"RHI command lists, render-graph (RDG) passes, translate jobs, GPU payload submissions and sync points/fences; "
	"consumed by the RenderTraceInsights plugin.");

#if UE_RENDER_TRACING_ENABLED

namespace RenderTracing
{

// Start with everything enabled to make sure we don't lose any events before initialization runs.
static ERenderTracingChannels GRenderTracingChannels = ERenderTracingChannels::All;

static void SetChannels(IConsoleVariable* CVar)
{
	FString ChannelString;
	if (!FParse::Value(FCommandLine::Get(), TEXT("-rendertracechannels="), ChannelString, false))
	{
		ChannelString = CVar->GetString();
	}

	GRenderTracingChannels = ERenderTracingChannels::None;

	FStringView Trimmed = FStringView(ChannelString).TrimStartAndEnd();
	if (Trimmed.IsEmpty() || Trimmed.Equals(TEXT("none"), ESearchCase::IgnoreCase))
	{
		UE_LOGF(LogRHI, Log, "Render tracing: all channels disabled.");
		return;
	}

	UE::String::ParseTokens(Trimmed, TEXT(','), [](FStringView Token)
	{
		FStringView Name = Token.TrimStartAndEnd();
		if      (Name.Equals(TEXT("RDGPasses"),			ESearchCase::IgnoreCase)) { GRenderTracingChannels |= ERenderTracingChannels::RDGPasses; }
		else if (Name.Equals(TEXT("RHICmdLists"),		ESearchCase::IgnoreCase)) { GRenderTracingChannels |= ERenderTracingChannels::RHICmdLists; }
		else if (Name.Equals(TEXT("RHITranslation"),	ESearchCase::IgnoreCase)) { GRenderTracingChannels |= ERenderTracingChannels::RHITranslation; }
		else if (Name.Equals(TEXT("RHISubmission"),		ESearchCase::IgnoreCase)) { GRenderTracingChannels |= ERenderTracingChannels::RHISubmission; }
		else if (Name.Equals(TEXT("Submission"),		ESearchCase::IgnoreCase)) { GRenderTracingChannels |= ERenderTracingChannels::Submission; }
		else if (Name.Equals(TEXT("All"),				ESearchCase::IgnoreCase)) { GRenderTracingChannels |= ERenderTracingChannels::All; }
		else
		{
			UE_LOGF(LogRHI, Warning, "Unknown render tracing channel '%.*ls', ignoring.", Name.Len(), Name.GetData());
		}
	});

	UE_LOGF(LogRHI, Log, "Render tracing channels set: '%ls' -> 0x%llx", *ChannelString, static_cast<uint64>(GRenderTracingChannels));
}

// Read-only cvar which controls which channels are enabled. We add a change callback because we want to allow the value to be overridden by
// the hotfix system, which runs after the Initialize() call below. The command line flag (-rendertracechannels) always takes precedence,
// regardless of having a hotfix value or not.
static TAutoConsoleVariable<FString> CVarRenderTraceChannels(
	TEXT("r.Trace.Channels"),
	TEXT("Submission"),
	TEXT("Comma-separated list of render tracing channels to enable. Valid values are the members of the ERenderTracingChannels enum.\n")
	TEXT("Use 'none' or an empty string to disable all channels. Override via -rendertracechannels=<channels> on the command line."),
	FConsoleVariableDelegate::CreateStatic(&SetChannels),
	ECVF_ReadOnly
);

void Initialize()
{
	SetChannels(CVarRenderTraceChannels.AsVariable());
}

bool IsEnabled(ERenderTracingChannels ChannelMask)
{
	return EnumHasAnyFlags(GRenderTracingChannels, ChannelMask) && UE_TRACE_CHANNELEXPR_IS_ENABLED(RenderTraceChannel);
}

}
#endif
