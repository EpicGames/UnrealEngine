// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHI.h"
#include "Modules/ModuleManager.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Windows/WindowsD3D.h"
#include "Windows/WindowsPlatformApplicationMisc.h"
#include "Windows/WindowsConfigRules.h"

#define LOCTEXT_NAMESPACE "WindowsDynamicRHI"

bool GDynamicRHIFailedToInitializeAdvancedPlatform = false;

static const TCHAR* GLoadedRHIModuleName;

enum class EWindowsRHI
{
	D3D11,
	D3D12,
	Vulkan,
	OpenGL,
	Imported,
	count
};
static constexpr int32 EWindowsRHICount = static_cast<int32>(EWindowsRHI::count);

struct FOverrideRHI
{
	FString ModuleName;
	FString RHIName;
	ERHIFeatureLevel::Type FeatureLevel;
};

static FOverrideRHI* GetOverrideRHI()
{
	static bool bHasChecked = false;
	static FOverrideRHI* Override = nullptr;
	if (bHasChecked)
	{
		return Override;
	}
	bHasChecked = true;

#if ALLOW_OTHER_PLATFORM_CONFIG
	FString OverrideRHIPlatformName;
	if (FParse::Value(FCommandLine::Get(), TEXT("PlatformRHI="), OverrideRHIPlatformName))
	{
		Override = new FOverrideRHI;
		FConfigCacheIni* Config = FConfigCacheIni::ForPlatform(*OverrideRHIPlatformName);
		Config->GetString(TEXT("ImportedRHI"), TEXT("ModuleName"), Override->ModuleName, GEngineIni);
		Config->GetString(TEXT("ImportedRHI"), TEXT("ModuleName"), Override->RHIName, GEngineIni);
		Config->GetString(TEXT("ImportedRHI"), TEXT("RHIName"), Override->RHIName, GEngineIni);
		FString FeatureLevel;
		Config->GetString(TEXT("ImportedRHI"), TEXT("FeatureLevel"), FeatureLevel, GEngineIni);
		GetFeatureLevelFromName(FeatureLevel, Override->FeatureLevel);
	}
#endif
	return Override;
}


inline const TCHAR* GetLogName(EWindowsRHI InWindowsRHI)
{
	switch (InWindowsRHI)
	{
	case EWindowsRHI::D3D11:  return TEXT("D3D11");
	case EWindowsRHI::D3D12:  return TEXT("D3D12");
	case EWindowsRHI::Vulkan: return TEXT("Vulkan");
	case EWindowsRHI::OpenGL: return TEXT("OpenGL");
	case EWindowsRHI::Imported: return *GetOverrideRHI()->RHIName;
	default:                  return TEXT("<unknown>");
	}
}

inline FText GetTextName(EWindowsRHI InWindowsRHI)
{
	switch (InWindowsRHI)
	{
	case EWindowsRHI::D3D11:  return LOCTEXT("D3D11",      "DirectX 11");
	case EWindowsRHI::D3D12:  return LOCTEXT("D3D12",      "DirectX 12");
	case EWindowsRHI::Vulkan: return LOCTEXT("Vulkan",     "Vulkan");
	case EWindowsRHI::OpenGL: return LOCTEXT("OpenGL",     "OpenGL");
	case EWindowsRHI::Imported:  return FText::FromString(GetOverrideRHI()->RHIName);
	default:                  return LOCTEXT("UnknownRHI", "<Unknown>");
	}
}

inline const TCHAR* GetLogName(ERHIFeatureLevel::Type InFeatureLevel)
{
	switch (InFeatureLevel)
	{
	case ERHIFeatureLevel::ES3_1: return TEXT("ES3_1");
	case ERHIFeatureLevel::SM5:   return TEXT("SM5");
	case ERHIFeatureLevel::SM6:   return TEXT("SM6");
	default:                      return TEXT("<unknown>");
	}
}

inline FText GetTextName(ERHIFeatureLevel::Type InFeatureLevel)
{
	switch (InFeatureLevel)
	{
	case ERHIFeatureLevel::ES3_1: return LOCTEXT("FeatureLevelES31",    "ES3_1");
	case ERHIFeatureLevel::SM5:   return LOCTEXT("FeatureLevelSM4",     "SM5");
	case ERHIFeatureLevel::SM6:   return LOCTEXT("FeatureLevelSM6",     "SM6");
	default:                      return LOCTEXT("FeatureLevelUnknown", "<unknown>");
	}
}

const EWindowsRHI GRHISearchOrder[] =
{
	EWindowsRHI::D3D12,
	EWindowsRHI::D3D11,
	EWindowsRHI::Vulkan,
	EWindowsRHI::OpenGL,
};

static TAutoConsoleVariable<bool> CVarIgnorePerformanceModeCheck(
	TEXT("r.IgnorePerformanceModeCheck"),
	false,
	TEXT("Ignore performance mode check"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarForcePreferredRHISupport(
	TEXT("r.ForcePreferredRHISupport"),
	false,
	TEXT("Whether to force PreferredRHI support even if disabled in config"),
	ECVF_ReadOnly
);

static bool IsPreferredRHIAllowed()
{
	bool bAllowPreferredRHI = GIsEditor || CVarForcePreferredRHISupport.GetValueOnGameThread();
	if (!bAllowPreferredRHI)
	{
		bAllowPreferredRHI = true;
		GConfig->GetBool(TEXT("RHI"), TEXT("bUsePreferredRHI"), bAllowPreferredRHI, GEngineIni);
	}

	return bAllowPreferredRHI;
}

static bool IsSM5Allowed()
{
	bool bAllowSM5 = GIsEditor;
	if (!bAllowSM5)
	{
		bool bBlockSM5 = false;
		GConfig->GetBool(TEXT("RHI"), TEXT("bBlockSM5"), bBlockSM5, GEngineIni);
		bAllowSM5 = !bBlockSM5;
	}

	return bAllowSM5;
}

static const TCHAR* ModuleNameFromWindowsRHI(EWindowsRHI InWindowsRHI)
{
	switch (InWindowsRHI)
	{
	default: check(false);
	case EWindowsRHI::D3D11:  return TEXT("D3D11RHI");
	case EWindowsRHI::D3D12:  return TEXT("D3D12RHI");
	case EWindowsRHI::Vulkan: return TEXT("VulkanRHI");
	case EWindowsRHI::OpenGL: return TEXT("OpenGLDrv");
	case EWindowsRHI::Imported: return *GetOverrideRHI()->ModuleName;
	}
}

static const TCHAR* GetRHINameFromWindowsRHI(EWindowsRHI InWindowsRHI)
{
	switch (InWindowsRHI)
	{
	default: check(false);
	case EWindowsRHI::D3D11:  return TEXT("DirectX 11");
	case EWindowsRHI::D3D12:  return TEXT("DirectX 12");
	case EWindowsRHI::Vulkan: return TEXT("Vulkan");
	case EWindowsRHI::OpenGL: return TEXT("OpenGL");
	case EWindowsRHI::Imported: return *GetOverrideRHI()->RHIName;
	}
}

static FString GetRHINameFromWindowsRHI(EWindowsRHI InWindowsRHI, ERHIFeatureLevel::Type InFeatureLevel)
{
	switch (InWindowsRHI)
	{
	case EWindowsRHI::D3D12:
	default:
	{
		FString FeatureLevelName;
		GetFeatureLevelName(InFeatureLevel, FeatureLevelName);
		return FString::Printf(TEXT("%s (%s)"), GetLogName(InWindowsRHI), *FeatureLevelName);
	}
	case EWindowsRHI::D3D11:  return TEXT("DirectX 11");
	case EWindowsRHI::Vulkan:
	{
		FString FeatureLevelName;
		GetFeatureLevelName(InFeatureLevel, FeatureLevelName);
		return FString::Printf(TEXT("Vulkan (%s)"), *FeatureLevelName);
	}
	case EWindowsRHI::OpenGL: return TEXT("OpenGL");
	}
}

static const TCHAR* GetES31ModuleName(ERHIInterfaceType InterfaceType)
{
	switch (InterfaceType)
	{
	default: check(false);
	case ERHIInterfaceType::D3D11:  return TEXT("ES31_D3D11RHI");
	case ERHIInterfaceType::D3D12:  return TEXT("ES31_D3D12RHI");
	case ERHIInterfaceType::Vulkan: return TEXT("ES31_VulkanRHI");
	case ERHIInterfaceType::OpenGL: return TEXT("ES31_OpenGLDrv");
	}
}

static const TCHAR* GetES31ModuleNameByString(const TCHAR* RHIModuleName)
{
	if (FCString::Strcmp(RHIModuleName, ModuleNameFromWindowsRHI(EWindowsRHI::D3D11)) == 0)
	{
		return GetES31ModuleName(ERHIInterfaceType::D3D11);
	}

	if (FCString::Strcmp(RHIModuleName, ModuleNameFromWindowsRHI(EWindowsRHI::D3D12)) == 0)
	{
		return GetES31ModuleName(ERHIInterfaceType::D3D12);
	}

	if (FCString::Strcmp(RHIModuleName, ModuleNameFromWindowsRHI(EWindowsRHI::Vulkan)) == 0)
	{
		return GetES31ModuleName(ERHIInterfaceType::Vulkan);
	}

	if (FCString::Strcmp(RHIModuleName, ModuleNameFromWindowsRHI(EWindowsRHI::OpenGL)) == 0)
	{
		return GetES31ModuleName(ERHIInterfaceType::OpenGL);
	}

	check(false);
	return RHIModuleName;
}

static ERHIFeatureLevel::Type GetDefaultFeatureLevelForRHI(EWindowsRHI InWindowsRHI)
{
	switch (InWindowsRHI)
	{
	case EWindowsRHI::D3D11:  return ERHIFeatureLevel::SM5;
	case EWindowsRHI::D3D12:  return ERHIFeatureLevel::SM5;
	case EWindowsRHI::Vulkan: return ERHIFeatureLevel::SM5;
	case EWindowsRHI::OpenGL: return ERHIFeatureLevel::ES3_1;
	case EWindowsRHI::Imported: return GetOverrideRHI()->FeatureLevel;
	default: check(false);    return ERHIFeatureLevel::SM5;
	}
}

struct FWindowsRHIConfig
{
	TArray<EShaderPlatform>        ShaderPlatforms;
	TArray<ERHIFeatureLevel::Type> FeatureLevels;
};

struct FParsedWindowsDynamicRHIConfig
{
	TOptional<EWindowsRHI> DefaultRHI{};

	TStaticArray<FWindowsRHIConfig, EWindowsRHICount> RHIConfigs;

	bool IsEmpty() const;

	bool IsRHISupported(EWindowsRHI InWindowsRHI) const;
	TOptional<ERHIFeatureLevel::Type> GetHighestSupportedFeatureLevel(EWindowsRHI InWindowsRHI) const;
	TOptional<ERHIFeatureLevel::Type> GetNextHighestTargetedFeatureLevel(EWindowsRHI InWindowsRHI, ERHIFeatureLevel::Type InFeatureLevel) const;

	bool IsFeatureLevelTargeted(EWindowsRHI InWindowsRHI, ERHIFeatureLevel::Type InFeatureLevel) const;
	TOptional<EWindowsRHI> GetFirstRHIWithFeatureLevelSupport(ERHIFeatureLevel::Type InFeatureLevel) const;

	void MergeConfig(EWindowsRHI InWindowsRHI, const FWindowsRHIConfig& Other);
};

bool FParsedWindowsDynamicRHIConfig::IsEmpty() const
{
	for (const FWindowsRHIConfig& Config : RHIConfigs)
	{
		if (!Config.ShaderPlatforms.IsEmpty())
		{
			return false;
		}
	}

	return true;
}

bool FParsedWindowsDynamicRHIConfig::IsRHISupported(EWindowsRHI InWindowsRHI) const
{
	// If we don't require cooked data, then we should be able to support this RHI at any feature level.
	if (!FPlatformProperties::RequiresCookedData())
	{
		return true;
	}

	return !RHIConfigs[(int32)InWindowsRHI].ShaderPlatforms.IsEmpty();
}

TOptional<ERHIFeatureLevel::Type> FParsedWindowsDynamicRHIConfig::GetHighestSupportedFeatureLevel(EWindowsRHI InWindowsRHI) const
{
	const TArray<ERHIFeatureLevel::Type>& FeatureLevels = RHIConfigs[(int32)InWindowsRHI].FeatureLevels;
	if (FeatureLevels.Num() == 0)
	{
		return TOptional<ERHIFeatureLevel::Type>();
	}

	ERHIFeatureLevel::Type MaxFeatureLevel = (ERHIFeatureLevel::Type)0;
	for (ERHIFeatureLevel::Type SupportedFeatureLevel : FeatureLevels)
	{
		MaxFeatureLevel = std::max(MaxFeatureLevel, SupportedFeatureLevel);
	}
	return MaxFeatureLevel;
}

TOptional<ERHIFeatureLevel::Type> FParsedWindowsDynamicRHIConfig::GetNextHighestTargetedFeatureLevel(EWindowsRHI InWindowsRHI, ERHIFeatureLevel::Type InFeatureLevel) const
{
	TArray<ERHIFeatureLevel::Type> LowerFeatureLevels(RHIConfigs[(int32)InWindowsRHI].FeatureLevels);
	LowerFeatureLevels.RemoveAll([InFeatureLevel](ERHIFeatureLevel::Type OtherFeatureLevel) { return OtherFeatureLevel >= InFeatureLevel; });

	if (LowerFeatureLevels.Num())
	{
		ERHIFeatureLevel::Type MaxFeatureLevel = (ERHIFeatureLevel::Type)0;
		for (ERHIFeatureLevel::Type SupportedFeatureLevel : LowerFeatureLevels)
		{
			MaxFeatureLevel = std::max(MaxFeatureLevel, SupportedFeatureLevel);
		}
		return MaxFeatureLevel;
	}

	return TOptional<ERHIFeatureLevel::Type>();
}

bool FParsedWindowsDynamicRHIConfig::IsFeatureLevelTargeted(EWindowsRHI InWindowsRHI, ERHIFeatureLevel::Type InFeatureLevel) const
{
	for (ERHIFeatureLevel::Type SupportedFeatureLevel : RHIConfigs[(int32)InWindowsRHI].FeatureLevels)
	{
		if (SupportedFeatureLevel == InFeatureLevel)
		{
			return true;
		}
	}
	return false;
}

TOptional<EWindowsRHI> FParsedWindowsDynamicRHIConfig::GetFirstRHIWithFeatureLevelSupport(ERHIFeatureLevel::Type InFeatureLevel) const
{
	for (EWindowsRHI WindowsRHI : GRHISearchOrder)
	{
		if (IsFeatureLevelTargeted(WindowsRHI, InFeatureLevel))
		{
			return WindowsRHI;
		}
	}
	return {};
}

void FParsedWindowsDynamicRHIConfig::MergeConfig(EWindowsRHI InWindowsRHI, const FWindowsRHIConfig& Other)
{
	FWindowsRHIConfig& ExistingConfig = RHIConfigs[(int32)InWindowsRHI];

	for (EShaderPlatform ShaderPlatform : Other.ShaderPlatforms)
	{
		ExistingConfig.ShaderPlatforms.AddUnique(ShaderPlatform);
	}

	for (ERHIFeatureLevel::Type FeatureLevel : Other.FeatureLevels)
	{
		ExistingConfig.FeatureLevels.AddUnique(FeatureLevel);
	}
}

TOptional<EWindowsRHI> ParseDefaultWindowsRHI()
{
	TOptional<EWindowsRHI> DefaultRHI{};

	FString DefaultGraphicsRHI;
	if (GConfig->GetString(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("DefaultGraphicsRHI"), DefaultGraphicsRHI, GEngineIni))
	{
		const FString NAME_DX11("DefaultGraphicsRHI_DX11");
		const FString NAME_DX12("DefaultGraphicsRHI_DX12");
		const FString NAME_VULKAN("DefaultGraphicsRHI_Vulkan");

		if (DefaultGraphicsRHI == NAME_DX11)
		{
			DefaultRHI = EWindowsRHI::D3D11;
		}
		else if (DefaultGraphicsRHI == NAME_DX12)
		{
			DefaultRHI = EWindowsRHI::D3D12;
		}
		else if (DefaultGraphicsRHI == NAME_VULKAN)
		{
			DefaultRHI = EWindowsRHI::Vulkan;
		}
		else if (DefaultGraphicsRHI != TEXT("DefaultGraphicsRHI_Default"))
		{
			UE_LOGF(LogRHI, Error, "Unrecognized setting '%ls' for DefaultGraphicsRHI", *DefaultGraphicsRHI);
		}
	}

	return DefaultRHI;
}

static FWindowsGPUInfo GetGPUInfo()
{
	FWindowsGPUInfo Info = FWindowsD3D::GetExpectedGPUInfo();

	if (Info.AdapterName.IsEmpty())
	{
		Info.AdapterName = FPlatformMisc::GetPrimaryGPUBrand();
	}

	return Info;
}

static bool IsGPUInHardwareListForRHI(const FWindowsGPUInfo& BestGPUInfo, const TCHAR* const ConfigKey, const TCHAR* const MatchingRHIName)
{
	TArray<FString> DeviceRHIList;
	GConfig->GetArray(TEXT("Devices"), ConfigKey, DeviceRHIList, GHardwareIni);

	const FString& GPUBrand = BestGPUInfo.AdapterName;
	for (const FString& DeviceRHIString : DeviceRHIList)
	{
		const TCHAR* Line = *DeviceRHIString;

		ensure(Line[0] == TCHAR('('));

		FString RHIName;
		FParse::Value(Line + 1, TEXT("RHI="), RHIName);

		FString DeviceName;
		FParse::Value(Line + 1, TEXT("DeviceName="), DeviceName);

		if (RHIName.Compare(MatchingRHIName, ESearchCase::IgnoreCase) == 0 && GPUBrand.Compare(DeviceName, ESearchCase::IgnoreCase) == 0)
		{
			UE_LOGF(LogRHI, Log, "%ls: Found RHIName %ls with DeviceName %ls", ConfigKey, *RHIName, *DeviceName);
			return true;
		}

		FString VendorId;
		FParse::Value(Line + 1, TEXT("VendorId="), VendorId);
		uint32 VendorIdInt = FParse::HexNumber(*VendorId);

		FString DeviceId;
		const bool bSpecificDevice = FParse::Value(Line + 1, TEXT("DeviceId="), DeviceId);
		uint32 DeviceIdInt = FParse::HexNumber(*DeviceId);

		if (BestGPUInfo.VendorId && (BestGPUInfo.VendorId == VendorIdInt))
		{
			if (!bSpecificDevice || (BestGPUInfo.DeviceId && (BestGPUInfo.DeviceId == DeviceIdInt)))
			{
				if (RHIName.Compare(MatchingRHIName, ESearchCase::IgnoreCase) == 0)
				{
					UE_LOGF(LogRHI, Log, "%ls: Found RHIName %ls with VendorId %ls, and DeviceId %ls", ConfigKey, *RHIName, *VendorId, *DeviceId);
					return true;
				}
			}
		}
	}

	return false;
}

static bool ShouldAvoidD3D11_SM5()
{
	bool bAvoidD3D11SM5 = false;

	static TOptional<bool> AvoidD3D11SM5;
	if (AvoidD3D11SM5.IsSet())
	{
		bAvoidD3D11SM5 = AvoidD3D11SM5.GetValue();
	}
	else if (!FParse::Param(FCommandLine::Get(), TEXT("d3d11")) &&
		!FParse::Param(FCommandLine::Get(), TEXT("dx11")) &&
		!FParse::Param(FCommandLine::Get(), TEXT("sm5")))
	{
		const FWindowsGPUInfo BestGPUInfo = GetGPUInfo();

		bAvoidD3D11SM5 = IsGPUInHardwareListForRHI(BestGPUInfo, TEXT("DeviceAvoidRHIList"), TEXT("D3D11_SM5"));
		if (bAvoidD3D11SM5)
		{
			UE_LOGF(LogRHI, Log, "Avoiding feature level D3D11 %ls", GetLogName(ERHIFeatureLevel::SM5));
		}

		AvoidD3D11SM5 = bAvoidD3D11SM5;
	}

	return bAvoidD3D11SM5;
}

static bool ShouldAvoidD3D11_ES31()
{
	bool bAvoidD3D11ES31 = false;

	static TOptional<bool> AvoidD3D11ES31;
	if (AvoidD3D11ES31.IsSet())
	{
		bAvoidD3D11ES31 = AvoidD3D11ES31.GetValue();
	}
	else if (!FParse::Param(FCommandLine::Get(), TEXT("d3d11")) &&
		!FParse::Param(FCommandLine::Get(), TEXT("dx11")) &&
		!FParse::Param(FCommandLine::Get(), TEXT("es31")) &&
		!FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES31")) &&
		!FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES3_1")))
	{
		const FWindowsGPUInfo BestGPUInfo = GetGPUInfo();

		bAvoidD3D11ES31 = IsGPUInHardwareListForRHI(BestGPUInfo, TEXT("DeviceAvoidRHIList"), TEXT("D3D11_ES31"));
		if (bAvoidD3D11ES31)
		{
			UE_LOGF(LogRHI, Log, "Avoiding feature level D3D11 %ls", GetLogName(ERHIFeatureLevel::ES3_1));
		}

		AvoidD3D11ES31 = bAvoidD3D11ES31;
	}

	return bAvoidD3D11ES31;
}

static TArray<EShaderPlatform> ParseShaderPlatformsConfig(const TCHAR* InSettingName)
{
	TArray<FString> TargetedShaderFormats;
	GConfig->GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), InSettingName, TargetedShaderFormats, GEngineIni);

	TArray<EShaderPlatform> ShaderPlatforms;
	ShaderPlatforms.Reserve(TargetedShaderFormats.Num());

	for (const FString& ShaderFormat : TargetedShaderFormats)
	{
		ShaderPlatforms.AddUnique(ShaderFormatToLegacyShaderPlatform(FName(*ShaderFormat)));
	}

	if (GIsEditor)
	{
		ShaderPlatforms.RemoveAll(
			[](EShaderPlatform ShaderPlatform)
			{
				return FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(ShaderPlatform) < ERHIFeatureLevel::SM5;
			});
	}
	else
	{
		if (!IsSM5Allowed())
		{
			ShaderPlatforms.RemoveAll(
				[](EShaderPlatform ShaderPlatform)
				{
					return FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(ShaderPlatform) == ERHIFeatureLevel::SM5;
				});
		}

		if (FCString::Strcmp(InSettingName, TEXT("D3D11TargetedShaderFormats")) == 0)
		{
			bool bUseSM5AvoidGPUList = false;
			if (GConfig->GetBool(TEXT("D3D11_SM5"), TEXT("bUseSM5AvoidGPUList"), bUseSM5AvoidGPUList, GEngineIni))
			{
				if (bUseSM5AvoidGPUList && ShouldAvoidD3D11_SM5())
				{
					ShaderPlatforms.RemoveAll(
						[](EShaderPlatform ShaderPlatform)
						{
							return FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(ShaderPlatform) == ERHIFeatureLevel::SM5;
						});
				}
			}

			bool bUseES31AvoidGPUList = false;
			if (GConfig->GetBool(TEXT("D3D11_ES31"), TEXT("bUseES31AvoidGPUList"), bUseES31AvoidGPUList, GEngineIni))
			{
				if (bUseES31AvoidGPUList && ShouldAvoidD3D11_ES31())
				{
					ShaderPlatforms.RemoveAll(
						[](EShaderPlatform ShaderPlatform)
						{
							return FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(ShaderPlatform) == ERHIFeatureLevel::ES3_1;
						});
				}
			}
		}
	}

	return ShaderPlatforms;
}

static TArray<ERHIFeatureLevel::Type> FeatureLevelsFromShaderPlatforms(const TArray<EShaderPlatform>& InShaderPlatforms)
{
	TArray<ERHIFeatureLevel::Type> FeatureLevels;
	FeatureLevels.Reserve(InShaderPlatforms.Num());

	for (EShaderPlatform ShaderPlatform : InShaderPlatforms)
	{
		FeatureLevels.AddUnique(FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(ShaderPlatform));
	}

	return FeatureLevels;
}

static FWindowsRHIConfig ParseWindowsRHIConfig(const TCHAR* ShaderFormatsSettingName)
{
	FWindowsRHIConfig Config;

	Config.ShaderPlatforms = ParseShaderPlatformsConfig(ShaderFormatsSettingName);
	Config.FeatureLevels = FeatureLevelsFromShaderPlatforms(Config.ShaderPlatforms);

	return Config;
}

FParsedWindowsDynamicRHIConfig ParseWindowsDynamicRHIConfig()
{
	FParsedWindowsDynamicRHIConfig Config;

	Config.DefaultRHI = ParseDefaultWindowsRHI();

	Config.RHIConfigs[(int32)EWindowsRHI::D3D11]  = ParseWindowsRHIConfig(TEXT("D3D11TargetedShaderFormats"));
	Config.RHIConfigs[(int32)EWindowsRHI::D3D12]  = ParseWindowsRHIConfig(TEXT("D3D12TargetedShaderFormats"));
	Config.RHIConfigs[(int32)EWindowsRHI::Vulkan] = ParseWindowsRHIConfig(TEXT("VulkanTargetedShaderFormats"));

	// Only add OpenGL support to non-client programs.
	if (!FPlatformProperties::RequiresCookedData())
	{
		Config.RHIConfigs[(int32)EWindowsRHI::OpenGL].ShaderPlatforms.Add(SP_OPENGL_PCES3_1);
		Config.RHIConfigs[(int32)EWindowsRHI::OpenGL].FeatureLevels.Add(ERHIFeatureLevel::ES3_1);
	}

	if (FWindowsRHIConfig DeprecatedConfig = ParseWindowsRHIConfig(TEXT("TargetedRHIs")); !DeprecatedConfig.ShaderPlatforms.IsEmpty())
	{
		// Since we don't have context here, we have to add this old config setting to every potential RHI.
		Config.MergeConfig(EWindowsRHI::D3D11,  DeprecatedConfig);
		Config.MergeConfig(EWindowsRHI::D3D12,  DeprecatedConfig);
		Config.MergeConfig(EWindowsRHI::Vulkan, DeprecatedConfig);
		Config.MergeConfig(EWindowsRHI::OpenGL, DeprecatedConfig);
	}

	return Config;
}

static bool IsForcedToD3D11_ES31()
{
	bool bRequireD3D11ES31 = false;

	static TOptional<bool> ForceD3D11_ES31;
	if (ForceD3D11_ES31.IsSet())
	{
		bRequireD3D11ES31 = ForceD3D11_ES31.GetValue();
	}
	else
	{
		const FWindowsGPUInfo BestGPUInfo = GetGPUInfo();

		bRequireD3D11ES31 = IsGPUInHardwareListForRHI(BestGPUInfo, TEXT("DeviceForceRHIList"), TEXT("D3D11_ES31"));
		if (bRequireD3D11ES31)
		{
			UE_LOGF(LogRHI, Log, "Forced feature level to D3D11 %ls", GetLogName(ERHIFeatureLevel::ES3_1));
		}

		ForceD3D11_ES31 = bRequireD3D11ES31;
	}

	return bRequireD3D11ES31;
}

// Whether to default to Performance Mode on low-end machines (ES31 feature level).
// Looks at a few configuration options that control thresholds to enable this as the default.
static bool ShouldDefaultToPerformanceMode(bool& bOutForceD3D11)
{
	static TOptional<bool> ForceES31;
	static bool bForceD3D11 = false;
	if (ForceES31.IsSet())
	{
		bOutForceD3D11 = bForceD3D11;
		return ForceES31.GetValue();
	}

	const FWindowsGPUInfo BestGPUInfo = GetGPUInfo();

	if (IsGPUInHardwareListForRHI(BestGPUInfo, TEXT("DeviceDefaultRHIList"), TEXT("D3D11_ES31")))
	{
		UE_LOGF(LogRHI, Log, "Default feature level to %ls", GetLogName(ERHIFeatureLevel::ES3_1));

		ForceES31 = true;
		bForceD3D11 = true;

		bOutForceD3D11 = true;
		return true;
	}

	// Force Performance mode for laptops. We don't have an explicit way to check for laptops
	// so we use whether we're running on battery as a proxy.
	bool bIsLaptop = FPlatformMisc::IsRunningOnBattery();
	if (GConfig->GetBoolOrDefault(TEXT("PerformanceMode"), TEXT("bDefaultOnLaptops"), false, GEngineIni) && bIsLaptop)
	{
		UE_LOGF(LogRHI, Log, "Running on laptop: default feature level to %ls", GetLogName(ERHIFeatureLevel::ES3_1));
		ForceES31 = true;
		return true;
	}

	// Force Performance mode for machines with too few cores including hyperthreads
	int MinCoreCount = 0;
	if (GConfig->GetInt(TEXT("PerformanceMode"), TEXT("MinCoreCount"), MinCoreCount, GEngineIni) && FPlatformMisc::NumberOfCoresIncludingHyperthreads() < MinCoreCount)
	{
		UE_LOGF(LogRHI, Log, "MinCoreCount found (%d) and user core count (%d): default feature level to %ls", MinCoreCount, FPlatformMisc::NumberOfCoresIncludingHyperthreads(), GetLogName(ERHIFeatureLevel::ES3_1));
		ForceES31 = true;
		return true;
	}

	FString MinMemorySizeBucketString;
	FString MinIntegratedMemorySizeBucketString;
	if (GConfig->GetString(TEXT("PerformanceMode"), TEXT("MinMemorySizeBucket"), MinMemorySizeBucketString, GEngineIni) && GConfig->GetString(TEXT("PerformanceMode"), TEXT("MinIntegratedMemorySizeBucket"), MinIntegratedMemorySizeBucketString, GEngineIni))
	{
		for (int EnumIndex = int(EPlatformMemorySizeBucket::Largest); EnumIndex <= int(EPlatformMemorySizeBucket::Tiniest); EnumIndex++)
		{
			const TCHAR* BucketString = LexToString(EPlatformMemorySizeBucket(EnumIndex));
			// Force Performance mode for machines with too little memory
			if (MinMemorySizeBucketString == BucketString)
			{
				if (FPlatformMemory::GetMemorySizeBucket() >= EPlatformMemorySizeBucket(EnumIndex))
				{
					UE_LOGF(LogRHI, Log, "MinMemorySizeBucket found (%ls) and user memory (%d): default feature level to %ls", *MinMemorySizeBucketString, int32(FPlatformMemory::GetMemorySizeBucket()), GetLogName(ERHIFeatureLevel::ES3_1));

					ForceES31 = true;
					return true;
				}
			}

			// Force Performance mode for machines with too little memory when shared with the GPU
			if (MinIntegratedMemorySizeBucketString == BucketString)
			{
				const int MIN_GPU_MEMORY = 512 * 1024 * 1024;
				if (FPlatformMemory::GetMemorySizeBucket() >= EPlatformMemorySizeBucket(EnumIndex) && BestGPUInfo.DedicatedVideoMemory < MIN_GPU_MEMORY)
				{
					UE_LOGF(LogRHI, Log, "MinMemorySizeBucket found (%ls) and user memory (%d): default feature level to %ls", *MinMemorySizeBucketString, int32(FPlatformMemory::GetMemorySizeBucket()), GetLogName(ERHIFeatureLevel::ES3_1));

					ForceES31 = true;
					return true;
				}
			}
		}
	}

	ForceES31 = false;
	return false;
}

static TOptional<ERHIFeatureLevel::Type> GetPreferredFeatureLevel(EWindowsRHI ChosenRHI, const FParsedWindowsDynamicRHIConfig& Config, bool& bOutForceD3D11)
{
	bOutForceD3D11 = false;

	TOptional<ERHIFeatureLevel::Type> PreferredFeatureLevel;
	if (!GIsEditor)
	{
		bool bSetD3D11_ES31 = IsForcedToD3D11_ES31();
		if (bSetD3D11_ES31)
		{
			bOutForceD3D11 = true;
			PreferredFeatureLevel = ERHIFeatureLevel::ES3_1;
		}
		else
		{
			FString PreferredFeatureLevelName;
			if (GConfig->GetString(TEXT("D3DRHIPreference"), TEXT("PreferredFeatureLevel"), PreferredFeatureLevelName, GGameUserSettingsIni))
			{
				UE_LOGF(LogRHI, Log, "Found D3DRHIPreference PreferredFeatureLevel: %ls", *PreferredFeatureLevelName);

				if (PreferredFeatureLevelName == TEXT("sm5"))
				{
					if (IsSM5Allowed())
					{
						PreferredFeatureLevel = ERHIFeatureLevel::SM5;
					}
					else
					{
						UE_LOGF(LogRHI, Log, "Ignoring PreferredFeatureLevel for SM5, which is not allowed");
						GDynamicRHIIgnoredSM5Preference = true;
					}
				}
				else if (PreferredFeatureLevelName == TEXT("sm6"))
				{
					PreferredFeatureLevel = ERHIFeatureLevel::SM6;
				}
				else if (PreferredFeatureLevelName == TEXT("es31"))
				{
					PreferredFeatureLevel = ERHIFeatureLevel::ES3_1;
				}
				else
				{
					UE_LOGF(LogRHI, Error, "unknown feature level name \"%ls\" in game user settings, using default", *PreferredFeatureLevelName);
				}
			}
			else if (Config.IsFeatureLevelTargeted(ChosenRHI, ERHIFeatureLevel::ES3_1))
			{
				bool bPreferFeatureLevelES31 = false;
				bool bFoundPreference = GConfig->GetBool(TEXT("D3DRHIPreference"), TEXT("bPreferFeatureLevelES31"), bPreferFeatureLevelES31, GGameUserSettingsIni);
				if (bFoundPreference)
				{
					UE_LOGF(LogRHI, Log, "Found D3DRHIPreference bPreferFeatureLevelES31: %s", bPreferFeatureLevelES31 ? "true" : "false");
				}

				// Force low-spec users into performance mode but respect their choice once they have set a preference
				bool bDefaultES31 = false;
				if (!bFoundPreference && !CVarIgnorePerformanceModeCheck.GetValueOnGameThread())
				{
					bDefaultES31 = ShouldDefaultToPerformanceMode(bOutForceD3D11);
				}

				if (bPreferFeatureLevelES31 || bDefaultES31)
				{
					bSetD3D11_ES31 = !bFoundPreference;
					PreferredFeatureLevel = ERHIFeatureLevel::ES3_1;
				}
			}
		}

		if (bSetD3D11_ES31)
		{
			GConfig->SetString(TEXT("D3DRHIPreference"), TEXT("PreferredFeatureLevel"), TEXT("es31"), GGameUserSettingsIni);
			if (bOutForceD3D11 && IsPreferredRHIAllowed())
			{
				GConfig->SetString(TEXT("D3DRHIPreference"), TEXT("PreferredRHI"), TEXT("dx11"), GGameUserSettingsIni);
			}
		}
	}
	return PreferredFeatureLevel;
}

static bool ShouldDevicePreferSM5(uint32 DeviceId)
{
	uint32 SM5PreferedDeviceIds[] =
	{
		0x1B80, // "NVIDIA GeForce GTX 1080"
		0x1B81, // "NVIDIA GeForce GTX 1070"
		0x1B82, // "NVIDIA GeForce GTX 1070 Ti"
		0x1B83, // "NVIDIA GeForce GTX 1060 6GB"
		0x1B84, // "NVIDIA GeForce GTX 1060 3GB"
		0x1C01, // "NVIDIA GeForce GTX 1050 Ti"
		0x1C02, // "NVIDIA GeForce GTX 1060 3GB"
		0x1C03, // "NVIDIA GeForce GTX 1060 6GB"
		0x1C04, // "NVIDIA GeForce GTX 1060 5GB"
		0x1C06, // "NVIDIA GeForce GTX 1060 6GB"
		0x1C08, // "NVIDIA GeForce GTX 1050"
		0x1C81, // "NVIDIA GeForce GTX 1050"
		0x1C82, // "NVIDIA GeForce GTX 1050 Ti"
		0x1C83, // "NVIDIA GeForce GTX 1050"
		0x1B06, // "NVIDIA GeForce GTX 1080 Ti"
	};

	for (int Index = 0; Index < UE_ARRAY_COUNT(SM5PreferedDeviceIds); ++Index)
	{
		if (DeviceId == SM5PreferedDeviceIds[Index])
		{
			return true;
		}
	}

	return false;
}

// Whether a SM6 capable device should always default to SM6
// regardless of other heuristics suggesting otherwise.
static bool ShouldDevicePreferSM6(uint32 SM6CapableDeviceId)
{
	TArray<FString> SM6PreferredDeviceIds;
	GConfig->GetArray(TEXT("D3D12_SM6"), TEXT("SM6PreferredGPUDeviceIDs"), SM6PreferredDeviceIds, GEngineIni);

	for (const FString& PreferredDeviceID : SM6PreferredDeviceIds)
	{
		if (SM6CapableDeviceId == FCString::Strtoi(*PreferredDeviceID, nullptr, 10) ||
			SM6CapableDeviceId == FCString::Strtoi(*PreferredDeviceID, nullptr, 16))
		{
			return true;
		}
	}

	return false;
}

static bool IsRHIAllowedAsDefault(EWindowsRHI InRHI, ERHIFeatureLevel::Type InFeatureLevel)
{
	static const FWindowsGPUInfo BestGPUInfo = GetGPUInfo();

	if (InRHI == EWindowsRHI::D3D12 && InFeatureLevel == ERHIFeatureLevel::SM6)
	{
		if (ShouldDevicePreferSM6(BestGPUInfo.DeviceId))
		{
			return true;
		}
	}

	bool bAllowed = true;
	if (InRHI == EWindowsRHI::D3D12 && InFeatureLevel >= ERHIFeatureLevel::SM6)
	{
		int32 MinDedicatedMemoryMB = 0;
		if (GConfig->GetInt(TEXT("D3D12_SM6"), TEXT("MinDedicatedMemory"), MinDedicatedMemoryMB, GEngineIni))
		{
			const uint64 MinDedicatedMemory = static_cast<uint64>(MinDedicatedMemoryMB) << 20;
			bAllowed &= BestGPUInfo.DedicatedVideoMemory >= MinDedicatedMemory;
		}

		bool bAllowSM5 = IsSM5Allowed();
		if (bAllowSM5)
		{
			bool bUseSM5PreferredGPUList = true;
			GConfig->GetBool(TEXT("D3D12_SM6"), TEXT("bUseSM5PreferredGPUList"), bUseSM5PreferredGPUList, GEngineIni);
			if (bUseSM5PreferredGPUList)
			{
				bAllowed &= !ShouldDevicePreferSM5(BestGPUInfo.DeviceId);
			}
		}
	}
	return bAllowed;
}

static TOptional<ERHIFeatureLevel::Type> ChooseDefaultFeatureLevel(EWindowsRHI InRHI, const FParsedWindowsDynamicRHIConfig& Config)
{
	TOptional<ERHIFeatureLevel::Type> HighestFL = Config.GetHighestSupportedFeatureLevel(InRHI);
	while (HighestFL)
	{
		if (IsRHIAllowedAsDefault(InRHI, HighestFL.GetValue()))
		{
			return HighestFL;
		}
		HighestFL = Config.GetNextHighestTargetedFeatureLevel(InRHI, HighestFL.GetValue());
	}

	return TOptional<ERHIFeatureLevel::Type>();
}

// Choose the default RHI using this search order:
//  * DefaultGraphicsRHI if specified;
//  * The first supported RHI from TargetedRHIs, enumerated according to GRHISearchOrder;
//  * D3D11, if all else fails.
static EWindowsRHI ChooseDefaultRHI(const FParsedWindowsDynamicRHIConfig& Config)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ChooseDefaultRHI);
	// Default graphics RHI is the main project setting that governs the choice, so it takes the priority
	if (TOptional<EWindowsRHI> ConfigDefault = Config.DefaultRHI)
	{
		return ConfigDefault.GetValue();
	}

	// Find the first RHI with configured support based on the order above
	for (EWindowsRHI DefaultRHI : GRHISearchOrder)
	{
		if (TOptional<ERHIFeatureLevel::Type> HighestFL = ChooseDefaultFeatureLevel(DefaultRHI, Config))
		{
			return DefaultRHI;
		}
	}

	return EWindowsRHI::D3D11;
}

static TOptional<EWindowsRHI> ChoosePreferredRHI(EWindowsRHI InDefaultRHI)
{
	TOptional<EWindowsRHI> RHIPreference{};

	// If we are in game, there is a separate setting that can make it prefer D3D12 over D3D11 (but not over other RHIs).
	if (!GIsEditor && (InDefaultRHI == EWindowsRHI::D3D11 || InDefaultRHI == EWindowsRHI::D3D12))
	{
		bool bUseD3D12InGame = false;
		bool bPreferFeatureLevelES31 = false;
		FString PreferredRHIName;
		if (GConfig->GetString(TEXT("D3DRHIPreference"), TEXT("PreferredRHI"), PreferredRHIName, GGameUserSettingsIni))
		{
			if (IsPreferredRHIAllowed())
			{
				bool bBlockSM5 = !IsSM5Allowed();
				if (bBlockSM5)
				{
					FString PreferredFeatureLevelName;
					bBlockSM5 =
						GConfig->GetString(TEXT("D3DRHIPreference"), TEXT("PreferredFeatureLevel"), PreferredFeatureLevelName, GGameUserSettingsIni) &&
						(PreferredFeatureLevelName == TEXT("sm5"));
				}

				if (bBlockSM5)
				{
					UE_LOGF(LogRHI, Log, "Ignoring PreferredRHI \"%ls\" because PreferredFeatureLevel specifies SM5, which is not allowed", *PreferredRHIName);
					GDynamicRHIIgnoredSM5Preference = true;
				}
				else
				{
					UE_LOGF(LogRHI, Log, "Found D3DRHIPreference PreferredRHI: %ls", *PreferredRHIName);

					if (PreferredRHIName == TEXT("dx12"))
					{
						RHIPreference = EWindowsRHI::D3D12;
					}
					else if (PreferredRHIName == TEXT("dx11"))
					{
						RHIPreference = EWindowsRHI::D3D11;
					}
					else if (PreferredRHIName == TEXT("vulkan"))
					{
						RHIPreference = EWindowsRHI::Vulkan;
					}
					else if (PreferredRHIName == TEXT("opengl"))
					{
						RHIPreference = EWindowsRHI::OpenGL;
					}
					else
					{
						UE_LOGF(LogRHI, Error, "unknown RHI name \"%ls\" in game user settings, using default", *PreferredRHIName);
					}
				}
			}
			else
			{
				UE_LOGF(LogRHI, Log, "Ignoring PreferredRHI %ls, which is not used", *PreferredRHIName);
				GDynamicRHIIgnoredPreferredRHI = PreferredRHIName;
			}
		}
		else if (GConfig->GetBool(TEXT("D3DRHIPreference"), TEXT("bUseD3D12InGame"), bUseD3D12InGame, GGameUserSettingsIni) &&
			GConfig->GetBool(TEXT("D3DRHIPreference"), TEXT("bPreferFeatureLevelES31"), bPreferFeatureLevelES31, GGameUserSettingsIni))
		{
			// Respect legacy GameUserSettings values if present
			UE_LOGF(LogRHI, Log, "Found D3DRHIPreference bUseD3D12InGame: %s", bUseD3D12InGame ? "true" : "false");
			UE_LOGF(LogRHI, Log, "Found D3DRHIPreference bPreferFeatureLevelES31: %s", bPreferFeatureLevelES31 ? "true" : "false");

			if (bUseD3D12InGame)
			{
				RHIPreference = EWindowsRHI::D3D12;
			}
			else if (bPreferFeatureLevelES31)
			{
				RHIPreference = EWindowsRHI::D3D11;
			}
		}
	}

	return RHIPreference;
}

static TOptional<EWindowsRHI> ChooseForcedRHI(TOptional<ERHIFeatureLevel::Type> ForcedFeatureLevel, const FParsedWindowsDynamicRHIConfig& Config)
{
	TOptional<EWindowsRHI> ForcedRHI = {};

	// Command line overrides
	uint32 Sum = 0;
	if (FParse::Param(FCommandLine::Get(), TEXT("vulkan")))
	{
		ForcedRHI = EWindowsRHI::Vulkan;
		Sum++;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("opengl")))
	{
		ForcedRHI = EWindowsRHI::OpenGL;
		Sum++;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("d3d11")) || FParse::Param(FCommandLine::Get(), TEXT("dx11")))
	{
		ForcedRHI = EWindowsRHI::D3D11;
		Sum++;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("d3d12")) || FParse::Param(FCommandLine::Get(), TEXT("dx12")))
	{
		ForcedRHI = EWindowsRHI::D3D12;
		Sum++;
	}
	if (GetOverrideRHI() != nullptr)
	{
		ForcedRHI = EWindowsRHI::Imported;
		Sum++;
	}

#if WITH_EDITOR
	if (!ForcedRHI)
	{
		if (TOptional<ERHIBindlessConfiguration> ForcedConfig = RHIGetForcedBindlessConfiguration(); ForcedConfig && ForcedConfig != ERHIBindlessConfiguration::Disabled)
		{
			ForcedRHI = EWindowsRHI::D3D12;
		}
	}
#endif

	if (Sum > 1)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("RHIOptionsError", "-d3d12/dx12, -d3d11/dx11, -vulkan, -opengl, and -platformrhi are mutually exclusive options, but more than one was specified on the command-line."));
		UE_LOGF(LogRHI, Fatal, "-d3d12, -d3d11, -vulkan, -opengl, and -platformrhi are mutually exclusive options, but more than one was specified on the command-line.");
	}

	// FeatureLevelES31 is also a command line override, so it will determine the underlying RHI unless one is specified
	if (FPlatformProperties::RequiresCookedData() && ForcedFeatureLevel)
	{
		if (ForcedRHI)
		{
			if (!Config.IsFeatureLevelTargeted(ForcedRHI.GetValue(), ForcedFeatureLevel.GetValue()))
			{
				FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("RHINotSupported", "{0} does not support {1}."), GetTextName(ForcedRHI.GetValue()), GetTextName(ForcedFeatureLevel.GetValue())));
				FPlatformMisc::RequestExit(true, TEXT("ChooseForcedRHI"));
			}
		}
		else if ((ForcedRHI = Config.GetFirstRHIWithFeatureLevelSupport(ForcedFeatureLevel.GetValue())))
		{
			UE_LOGF(LogRHI, Log, "Forcing RHI to %ls since Feature Level %ls was forced", GetLogName(ForcedRHI.GetValue()), GetLogName(ForcedFeatureLevel.GetValue()));
		}
	}

	return ForcedRHI;
}

static TOptional<ERHIFeatureLevel::Type> GetForcedFeatureLevel()
{
	TOptional<ERHIFeatureLevel::Type> ForcedFeatureLevel{};

	if (GetOverrideRHI() != nullptr)
	{
		ForcedFeatureLevel = GetOverrideRHI()->FeatureLevel;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("es31")) || FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES31")) || FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES3_1")))
	{
		ForcedFeatureLevel = ERHIFeatureLevel::ES3_1;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("sm5")))
	{
		ForcedFeatureLevel = ERHIFeatureLevel::SM5;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("sm6")))
	{
		ForcedFeatureLevel = ERHIFeatureLevel::SM6;
	}

#if WITH_EDITOR
	if (TOptional<ERHIBindlessConfiguration> ForcedConfig = RHIGetForcedBindlessConfiguration(); ForcedConfig && ForcedConfig != ERHIBindlessConfiguration::Disabled)
	{
		ForcedFeatureLevel = ERHIFeatureLevel::SM6;
	}
#endif

	return ForcedFeatureLevel;
}

static ERHIFeatureLevel::Type ChooseFeatureLevel(EWindowsRHI ChosenRHI, const TOptional<ERHIFeatureLevel::Type> ForcedFeatureLevel, const FParsedWindowsDynamicRHIConfig& Config, bool& bOutForceD3D11)
{
	bOutForceD3D11 = false;

	if (ForcedFeatureLevel)
	{
		// Allow the forced feature level if we're in a position to compile its shaders
		if (!FPlatformProperties::RequiresCookedData())
		{
			UE_LOGF(LogRHI, Log, "Using Forced Feature Level in Editor: %ls", GetLogName(ForcedFeatureLevel.GetValue()));
			return ForcedFeatureLevel.GetValue();
		}

		// Make sure the feature level is supported by the runtime, otherwise fall back to the default
		if (Config.IsFeatureLevelTargeted(ChosenRHI, ForcedFeatureLevel.GetValue()))
		{
			UE_LOGF(LogRHI, Log, "Using Forced Feature Level the Game is configured for: %ls", GetLogName(ForcedFeatureLevel.GetValue()));
			return ForcedFeatureLevel.GetValue();
		}
	}

	TOptional<ERHIFeatureLevel::Type> FeatureLevel = GetPreferredFeatureLevel(ChosenRHI, Config, bOutForceD3D11);

	if (!FeatureLevel || (FPlatformProperties::RequiresCookedData() && !Config.IsFeatureLevelTargeted(ChosenRHI, FeatureLevel.GetValue())))
	{
		FeatureLevel = Config.GetHighestSupportedFeatureLevel(ChosenRHI);

		// If we were forced to a specific RHI while not forced to a specific feature level and the project isn't configured for it, find the default Feature Level for that RHI
		if (!FeatureLevel)
		{
			FeatureLevel = GetDefaultFeatureLevelForRHI(ChosenRHI);

			if (FPlatformProperties::RequiresCookedData())
			{
				FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("MissingTargetedShaderFormats", "Unable to launch with RHI '{0}' since the project is not configured to support it."), GetTextName(ChosenRHI)));
				UE_LOGF(LogRHI, Fatal, "Unable to launch with RHI '%ls' since the project is not configured to support it.", GetLogName(ChosenRHI));
			}
			else
			{
				UE_LOGF(LogRHI, Warning, "User requested RHI '%ls' but that is not supported by this project's data. Defaulting to Feature Level '%ls'.", GetLogName(ChosenRHI), GetLogName(FeatureLevel.GetValue()));
			}
		}
		else
		{
			UE_LOGF(LogRHI, Log, "Using Highest Feature Level of %ls: %ls", GetLogName(ChosenRHI), GetLogName(FeatureLevel.GetValue()));
		}
	}

	// If the user wanted to force a feature level and we couldn't set it, log out why and what we're actually running with
	if (ForcedFeatureLevel)
	{
		UE_LOGF(LogRHI, Warning, "User requested Feature Level '%ls' but that is not supported by this project. Falling back to Feature Level '%ls'.", GetLogName(ForcedFeatureLevel.GetValue()), GetLogName(FeatureLevel.GetValue()));
	}

	return FeatureLevel.GetValue();
}

static bool HandleUnsupportedFeatureLevel(EWindowsRHI& WindowsRHI, ERHIFeatureLevel::Type& FeatureLevel, TOptional<ERHIFeatureLevel::Type> ForcedFeatureLevel, const FParsedWindowsDynamicRHIConfig& Config)
{
	if (ForcedFeatureLevel)
	{
		if (WindowsRHI == EWindowsRHI::D3D12 && FeatureLevel == ERHIFeatureLevel::SM6)
		{
			UE_LOGF(LogRHI, Log, "RHI %ls with Feature Level %ls is not supported on your system.", GetLogName(WindowsRHI), GetLogName(FeatureLevel));

			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("RequiredDX12SM6", "DirectX 12 with Feature Level SM6 is not supported on your system. Try running without the -sm6 command line argument."));
			FPlatformMisc::RequestExit(true, TEXT("HandleUnsupportedFeatureLevel"));
		}

		return false;
	}

	if (FeatureLevel == ERHIFeatureLevel::SM6)
	{
		GDynamicRHIFailedToInitializeAdvancedPlatform = true;
	}

	if (TOptional<ERHIFeatureLevel::Type> FallbackFeatureLevel = Config.GetNextHighestTargetedFeatureLevel(WindowsRHI, FeatureLevel))
	{
		FeatureLevel = FallbackFeatureLevel.GetValue();
		return true;
	}

	return false;
}

static bool HandleUnsupportedRHI(EWindowsRHI& WindowsRHI, ERHIFeatureLevel::Type& FeatureLevel, TOptional<EWindowsRHI> ForcedRHI, const FParsedWindowsDynamicRHIConfig& Config)
{
	if (ForcedRHI)
	{
		if (ForcedRHI == EWindowsRHI::D3D12)
		{
			UE_LOGF(LogRHI, Log, "RHI %ls with Feature Level %ls is not supported on your system.", GetLogName(WindowsRHI), GetLogName(FeatureLevel));

			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("RequiredDX12", "DirectX 12 is not supported on your system. Try running without the -dx12 or -d3d12 command line argument."));
			FPlatformMisc::RequestExit(true, TEXT("HandleUnsupportedRHI.ForcedRHI"));
			return false;
		}
	}

	if (WindowsRHI == EWindowsRHI::D3D12)
	{
		if (TOptional<ERHIFeatureLevel::Type> D3D11FeatureLevel = Config.GetHighestSupportedFeatureLevel(EWindowsRHI::D3D11))
		{
			WindowsRHI = EWindowsRHI::D3D11;
			FeatureLevel = D3D11FeatureLevel.GetValue();
			return true;
		}
	}

	UE_LOGF(LogRHI, Log, "RHI %ls is not supported on your system.", GetLogName(WindowsRHI));

	if (WindowsRHI == EWindowsRHI::D3D12)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("RequiredDX12", "DirectX 12 is not supported on your system. Try running without the -dx12 or -d3d12 command line argument."));
		FPlatformMisc::RequestExit(true, TEXT("HandleUnsupportedRHI.D3D12"));
	}

	if (WindowsRHI == EWindowsRHI::D3D11)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("RequiredDX11Feature_11_SM5", "A D3D11-compatible GPU (Feature Level 11.0, Shader Model 5.0) is required to run the engine."));
		FPlatformMisc::RequestExit(true, TEXT("HandleUnsupportedRHI.D3D11"));
	}

	if (WindowsRHI == EWindowsRHI::Vulkan)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("RequiredVulkan", "A compatible Vulkan Driver is required to run the engine."));
		FPlatformMisc::RequestExit(true, TEXT("HandleUnsupportedRHI.Vulkan"));
	}

	if (WindowsRHI == EWindowsRHI::OpenGL)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("RequiredOpenGL", "OpenGL 4.3 is required to run the engine."));
		FPlatformMisc::RequestExit(true, TEXT("HandleUnsupportedRHI.OpenGL"));
	}

	return false;
}

static void LogTDRSummary()
{
	FTDRInfo TDRInfo = FWindowsPlatformMisc::GetTDRInfo();
	FString TDRSummary;
	if (TDRInfo.Level == ETDRLevel::Recover &&
		TDRInfo.Delay == FTDRInfo::DefaultDelay &&
		TDRInfo.DdiDelay == FTDRInfo::DefaultDdiDelay)
	{
		TDRSummary += FString::Printf(TEXT("\r\n\t- TDR settings OK"));
	}
	else
	{
		TDRSummary += FString::Printf(TEXT("\r\n\t- TDR using non-default settings"));
	}

	TDRSummary += FString::Printf(TEXT(" - Level: %s, Delay: %d, DdiDelay: %d"), LexToString(TDRInfo.Level), TDRInfo.Delay, TDRInfo.DdiDelay);
	UE_LOGF(LogRHI, Display, "Timeout Detection and Recovery (TDR) summary: %ls", *TDRSummary)
}

static IDynamicRHIModule* LoadDynamicRHIModule(ERHIFeatureLevel::Type& DesiredFeatureLevel, const TCHAR*& LoadedRHIModuleName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LoadDynamicRHIModule);

	// Make sure the DDSPI is initialized before we try and read from it
	FGenericDataDrivenShaderPlatformInfo::Initialize();

	bool bUseGPUCrashDebugging = false;
	if (!GIsEditor && GConfig->GetBool(TEXT("D3DRHIPreference"), TEXT("bUseGPUCrashDebugging"), bUseGPUCrashDebugging, GGameUserSettingsIni))
	{
		auto GPUCrashDebuggingCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GPUCrashDebugging"));
		*GPUCrashDebuggingCVar = bUseGPUCrashDebugging;
	}

	const FParsedWindowsDynamicRHIConfig Config = ParseWindowsDynamicRHIConfig();

	// RHI is chosen by the project settings (first DefaultGraphicsRHI, then TargetedRHIs are consulted, "Default" maps to D3D12). 
	// After this, a separate game-only setting (does not affect editor) bPreferD3D12InGame selects between D3D12 or D3D11 (but will not have any effect if Vulkan or OpenGL are chosen).
	// Commandline switches apply after this and can force an arbitrary RHIs. If RHI isn't supported, the game will refuse to start.

	EWindowsRHI DefaultRHI = ChooseDefaultRHI(Config);
	const TOptional<EWindowsRHI> PreferredRHI = ChoosePreferredRHI(DefaultRHI);
	const TOptional<ERHIFeatureLevel::Type> ForcedFeatureLevel = GetForcedFeatureLevel();
	const TOptional<EWindowsRHI> ForcedRHI = ChooseForcedRHI(ForcedFeatureLevel, Config);

	EWindowsRHI ChosenRHI = DefaultRHI;
	if (ForcedRHI)
	{
		ChosenRHI = ForcedRHI.GetValue();

		UE_LOGF(LogRHI, Log, "Using Forced RHI: %ls", GetLogName(ChosenRHI));
	}
	else if (PreferredRHI)
	{
		if (Config.RHIConfigs[static_cast<uint32>(PreferredRHI.GetValue())].ShaderPlatforms.IsEmpty())
		{
			UE_LOGF(LogRHI, Log, "Preferred RHI has no ShaderPlatforms available: %ls", GetLogName(PreferredRHI.GetValue()));
		}
		else
		{
			ChosenRHI = PreferredRHI.GetValue();
			UE_LOGF(LogRHI, Log, "Using Preferred RHI: %ls", GetLogName(ChosenRHI));
		}
	}
	else
	{
		UE_LOGF(LogRHI, Log, "Using Default RHI: %ls", GetLogName(ChosenRHI));
	}

	bool bOutForceD3D11 = false;
	DesiredFeatureLevel = ChooseFeatureLevel(ChosenRHI, ForcedFeatureLevel, Config, bOutForceD3D11);
	if (bOutForceD3D11)
	{
		ChosenRHI = EWindowsRHI::D3D11;
		UE_LOGF(LogRHI, Log, "Using Device RHI: %ls", GetLogName(ChosenRHI));
	}

	// Load the dynamic RHI module.

	bool bTryWithNewConfig = false;
	do
	{
		const FString RHIName = GetRHINameFromWindowsRHI(ChosenRHI, DesiredFeatureLevel);
		FApp::SetGraphicsRHI(RHIName);

		const TCHAR* ModuleName = ModuleNameFromWindowsRHI(ChosenRHI);

		UE_LOGF(LogRHI, Log, "Loading RHI module %ls", ModuleName);

		IDynamicRHIModule* DynamicRHIModule = FModuleManager::LoadModulePtr<IDynamicRHIModule>(ModuleName);

		UE_LOGF(LogRHI, Log, "Checking if RHI %ls with Feature Level %ls is supported by your system.", GetLogName(ChosenRHI), GetLogName(DesiredFeatureLevel));

		if (DynamicRHIModule && DynamicRHIModule->IsSupported(DesiredFeatureLevel))
		{
			UE_LOGF(LogRHI, Log, "RHI %ls with Feature Level %ls is supported and will be used.", GetLogName(ChosenRHI), GetLogName(DesiredFeatureLevel));

			LoadedRHIModuleName = ModuleName;
			return DynamicRHIModule;
		}

		const EWindowsRHI PreviousRHI = ChosenRHI;
		const ERHIFeatureLevel::Type PreviousFeatureLevel = DesiredFeatureLevel;

		bTryWithNewConfig = HandleUnsupportedFeatureLevel(ChosenRHI, DesiredFeatureLevel, ForcedFeatureLevel, Config);

		if (!bTryWithNewConfig)
		{
			bTryWithNewConfig = HandleUnsupportedRHI(ChosenRHI, DesiredFeatureLevel, ForcedRHI, Config);
		}

		if (bTryWithNewConfig)
		{
			UE_LOGF(LogRHI, Log, "RHI %ls with Feature Level %ls is not supported on your system, attempting to fall back to RHI %ls with Feature Level %ls",
				GetLogName(PreviousRHI), GetLogName(PreviousFeatureLevel),
				GetLogName(ChosenRHI), GetLogName(DesiredFeatureLevel));
		}
	} while (bTryWithNewConfig);

	UE_LOGF(LogRHI, Log, "RHI %ls with Feature Level %ls is not supported on your system. No RHI was supported, failing initialization.", GetLogName(ChosenRHI), GetLogName(DesiredFeatureLevel));

	return nullptr;
}

FDynamicRHI* PlatformCreateDynamicRHI()
{
	FDynamicRHI* DynamicRHI = nullptr;

	ERHIFeatureLevel::Type RequestedFeatureLevel;
	const TCHAR* LoadedRHIModuleName;
	IDynamicRHIModule* DynamicRHIModule = LoadDynamicRHIModule(RequestedFeatureLevel, LoadedRHIModuleName);

	FWindowsD3D::ReleaseCachedAdapterDevices();

	if (DynamicRHIModule)
	{
		// Run ConfigRules to allow hotfixing RHI. Things like GRHIDeviceId, GRHIVendorId are only available at this stage
#if WITH_WINDOWS_CONFIGRULES
		{
			SCOPED_BOOT_TIMING("FWindowsConfigRules::Parse");
			FWindowsConfigRules::SetRHIPredefinedVars(LoadedRHIModuleName, LexToString(RequestedFeatureLevel));
			FWindowsConfigRules::Parse();
		}
		{
			SCOPED_BOOT_TIMING("ApplyWindowsCompatConfigRules");
			FWindowsPlatformMisc::ApplyWindowsCompatConfigRules();
		}
#endif // WITH_WINDOWS_CONFIGRULES
	
		// Create the dynamic RHI.
		DynamicRHI = DynamicRHIModule->CreateRHI(RequestedFeatureLevel);
		GLoadedRHIModuleName = LoadedRHIModuleName;
	}

	LogTDRSummary();

	return DynamicRHI;
}

const TCHAR* GetSelectedDynamicRHIModuleName(bool bCleanup)
{
	check(FApp::CanEverRender());

	if (GDynamicRHI)
	{
		check(!!GLoadedRHIModuleName);
		return GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1 ? GetES31ModuleName(GDynamicRHI->GetInterfaceType()) : GLoadedRHIModuleName;
	}
	else
	{
		ERHIFeatureLevel::Type DesiredFeatureLevel;
		const TCHAR* RHIModuleName;
		IDynamicRHIModule* DynamicRHIModule = LoadDynamicRHIModule(DesiredFeatureLevel, RHIModuleName);
		check(DynamicRHIModule);
		check(RHIModuleName);
		if (bCleanup)
		{
			FModuleManager::Get().UnloadModule(RHIModuleName);
		}

		return DesiredFeatureLevel == ERHIFeatureLevel::ES3_1 ? GetES31ModuleNameByString(RHIModuleName) : RHIModuleName;
	}
}

#undef LOCTEXT_NAMESPACE

