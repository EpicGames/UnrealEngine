// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FDataDrivenShaderPlatformInfo final : public FGenericDataDrivenShaderPlatformInfo
{
public:
	IMPLEMENT_DDPSPI_SETTING(GetTargetsTiledGPU,				true);
	IMPLEMENT_DDPSPI_SETTING(GetSupportsDxc,					true);
	IMPLEMENT_DDPSPI_SETTING(GetSupportsUniformBufferObjects,	true);

#if USE_STATIC_FEATURE_LEVEL_ENUMS
	IMPLEMENT_DDPSPI_SETTING_WITH_RETURN_TYPE(ERHIFeatureLevel::Type, GetMaxFeatureLevel, FStaticFeatureLevel(UE_ANDROID_STATIC_FEATURE_LEVEL));
	IMPLEMENT_DDPSPI_SETTING(GetIsMobile, (UE_ANDROID_STATIC_FEATURE_LEVEL == ERHIFeatureLevel::ES3_1));
	IMPLEMENT_DDPSPI_SETTING(GetSupportsManualVertexFetch, (UE_ANDROID_STATIC_FEATURE_LEVEL > ERHIFeatureLevel::ES3_1));
	IMPLEMENT_DDPSPI_SETTING(GetSupportsMobileMultiView, (UE_ANDROID_STATIC_FEATURE_LEVEL == ERHIFeatureLevel::ES3_1));
#endif
};
