// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RHI.h"
#include "Android/AndroidWindow.h"

namespace UE
{
	namespace FAndroidPlatformDynamicRHI
	{
		typedef TUniqueFunction<void(TOptional<FAndroidWindow::FNativeAccessor> WindowContainer)> FRHIReInitWindowCallbackType;
		FRHIReInitWindowCallbackType& GetRHIOnReInitWindowCallback();
		void SetRHIOnReInitWindowCallback(FRHIReInitWindowCallbackType&& InRHIOnReInitWindowCallback);

		typedef TUniqueFunction<void(TOptional<FAndroidWindow::FNativeAccessor> WindowContainer)> FRHIReleaseWindowCallbackType;
		FRHIReleaseWindowCallbackType& GetRHIOnReleaseWindowCallback();
		void SetRHIOnReleaseWindowCallback(FRHIReleaseWindowCallbackType&& InOnReleaseWindowCallback);

		typedef TUniqueFunction<void()> FRHIOnWindowCreatedCallbackType;
		FRHIOnWindowCreatedCallbackType& GetRHIOnWindowCreatedCallback();
		void SetRHIOnWindowCreatedCallback(FRHIOnWindowCreatedCallbackType&& InOnWindowCreatedCallback);

		// Helper class to convert FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType to the uint64 required by android's external PSO service.
		class FPSOServicePriInfo
		{
			using EPSOPrecacheCompileType = FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType;
		public:
			FPSOServicePriInfo(EPSOPrecacheCompileType PSOCompileType);
			uint64 GetPriorityInfo() const { return PriInfo; }
		private:
			void SetSchedPolicy(char SchedPolicy, char SchedPri) { check(SchedPri < 128); PriInfo = PriInfo | 1 << 0 | (SchedPolicy << 8) | ((SchedPri + 128) & 0xFF) << 16; }
			void SetNice(char Nice) { check(Nice < 128); PriInfo = PriInfo | 1 << 1 | ((Nice + 128) & 0xFF) << 24; }
			void SetAffinity(uint32 AffinityMask) { PriInfo = PriInfo | 1 << 2 | ((uint64)AffinityMask << 32); }
			uint64 PriInfo = 0;
		};

		int32 GetPSOServiceFailureThreshold();
	} // namespace FAndroidPlatformDynamicRHI
} // namespace UE

namespace FPlatformDynamicRHI = UE::FAndroidPlatformDynamicRHI;
