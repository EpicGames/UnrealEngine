// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"

#if defined(UE_ANDROID_SHADER_PLATFORM_OPENGL_ES3_1)
	#define UE_ANDROID_STATIC_SHADER_PLATFORM OPENGL_ES3_1_ANDROID
#elif defined(UE_ANDROID_SHADER_PLATFORM_VULKAN_ES3_1)
	#define UE_ANDROID_STATIC_SHADER_PLATFORM VULKAN_ES3_1_ANDROID
#elif defined(UE_ANDROID_SHADER_PLATFORM_VULKAN_SM5)
	#define UE_ANDROID_STATIC_SHADER_PLATFORM VULKAN_SM5_ANDROID
#else
	#error "Unknown Android static shader platform"
#endif

struct FStaticShaderPlatform
{
	inline FStaticShaderPlatform(const EShaderPlatform InPlatform)
	{
		checkSlow(UE_ANDROID_STATIC_SHADER_PLATFORM == InPlatform);
	}

	inline operator EShaderPlatform() const
	{
		return UE_ANDROID_STATIC_SHADER_PLATFORM;
	}

	inline bool operator == (const EShaderPlatform Other) const
	{
		return Other == UE_ANDROID_STATIC_SHADER_PLATFORM;
	}
	
	inline bool operator != (const EShaderPlatform Other) const
	{
		return Other != UE_ANDROID_STATIC_SHADER_PLATFORM;
	}
};
