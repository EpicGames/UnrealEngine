// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BoundShaderStateCache.h: Bound shader state cache definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"

/**
* Key used to map a set of unique decl/vs/ps combinations to
* a vertex shader resource
*/
class FBoundShaderStateKey
{
public:
	/** Initialization constructor. */
	FBoundShaderStateKey(
		FRHIVertexDeclaration* InVertexDeclaration,
		FRHIVertexShader* InVertexShader,
		FRHIPixelShader* InPixelShader,
		FRHIGeometryShader* InGeometryShader = nullptr
		)
		: VertexDeclaration(InVertexDeclaration)
		, VertexShader(InVertexShader)
		, PixelShader(InPixelShader)
		, GeometryShader(InGeometryShader)
	{}

	/** Initialization constructor. */
	FBoundShaderStateKey(
		FRHIMeshShader* InMeshShader,
		FRHIAmplificationShader* InAmplificationShader,
		FRHIPixelShader* InPixelShader
	)
		: MeshShader(InMeshShader)
		, AmplificationShader(InAmplificationShader)
		, PixelShader(InPixelShader)
	{}

	/**
	 * Get the RHI shader for the given frequency.
	 */
	inline FRHIVertexShader*   GetVertexShader() const   { return VertexShader; }
	inline FRHIMeshShader*     GetMeshShader() const   { return MeshShader; }
	inline FRHIAmplificationShader*   GetAmplificationShader() const   { return AmplificationShader; }
	inline FRHIPixelShader*    GetPixelShader() const    { return PixelShader; }
	inline FRHIGeometryShader* GetGeometryShader() const { return GeometryShader; }

	/**
	* Get the RHI vertex declaration.
	*/
	inline FRHIVertexDeclaration* GetVertexDeclaration() const { return VertexDeclaration; }

private:
	/**
	 * Note: We intentionally do use ...Ref, not ...ParamRef to get 
	 * AddRef() for object to prevent and rare issue that happened before.
	 * When changing and recompiling a shader it got the same memory
	 * pointer and because the caching is happening with pointer comparison
	 * the BoundShaderstate cache was holding on to the old pointer
	 * it was not creating a new entry.
	 */

	/** vertex decl for this combination */
	FVertexDeclarationRHIRef VertexDeclaration;
	/** vs for this combination */
	FVertexShaderRHIRef VertexShader;
	/** ms for this combination */
	FMeshShaderRHIRef MeshShader;
	/** as for this combination */
	FAmplificationShaderRHIRef AmplificationShader;
	/** ps for this combination */
	FPixelShaderRHIRef PixelShader;
	/** gs for this combination */
	FGeometryShaderRHIRef GeometryShader;

	friend class FBoundShaderStateLookupKey;
};

// Non-reference-counted version of shader state key.
// This structure is used as the actual key type for TMap, which avoids reference counting overhead during lookup.
// Note that FCachedBoundShaderStateLink contains a full-fat reference-counted FBoundShaderStateKey, ensuring
// correct lifetime management.
class FBoundShaderStateLookupKey
{
public:

	// Note: implicit cast is allowed/expected for this constructor
	FBoundShaderStateLookupKey(const FBoundShaderStateKey& Key)
		: VertexDeclaration(Key.VertexDeclaration.GetReference())
		, VertexShader(Key.VertexShader.GetReference())
		, MeshShader(Key.MeshShader.GetReference())
		, AmplificationShader(Key.AmplificationShader.GetReference())
		, PixelShader(Key.PixelShader.GetReference())
		, GeometryShader(Key.GeometryShader.GetReference())
	{}

	FBoundShaderStateLookupKey(
		FRHIVertexDeclaration* InVertexDeclaration,
		FRHIVertexShader* InVertexShader,
		FRHIPixelShader* InPixelShader,
		FRHIGeometryShader* InGeometryShader = nullptr
	)
		: VertexDeclaration(InVertexDeclaration)
		, VertexShader(InVertexShader)
		, PixelShader(InPixelShader)
		, GeometryShader(InGeometryShader)
	{}

	FBoundShaderStateLookupKey(
		FRHIMeshShader* InMeshShader,
		FRHIAmplificationShader* InAmplificationShader,
		FRHIPixelShader* InPixelShader
	)
		: MeshShader(InMeshShader)
		, AmplificationShader(InAmplificationShader)
		, PixelShader(InPixelShader)
	{}

	/**
	* Equality is based on decl, vertex shader and pixel shader
	* @param Other - instance to compare against
	* @return true if equal
	*/
	friend bool operator == (const FBoundShaderStateLookupKey& A, const FBoundShaderStateLookupKey& B)
	{
		return	A.VertexDeclaration == B.VertexDeclaration &&
			A.VertexShader == B.VertexShader &&
			A.MeshShader == B.MeshShader &&
			A.AmplificationShader == B.AmplificationShader &&
			A.PixelShader == B.PixelShader &&
			A.GeometryShader == B.GeometryShader;
	}

	/**
	* Get the hash for this type.
	* @param Key - struct to hash
	* @return dword hash based on type
	*/
	friend uint32 GetTypeHash(const FBoundShaderStateLookupKey& Key)
	{
		return GetTypeHash(Key.VertexDeclaration) ^
			GetTypeHash(Key.VertexShader) ^
			GetTypeHash(Key.MeshShader) ^
			GetTypeHash(Key.AmplificationShader) ^
			GetTypeHash(Key.PixelShader) ^
			GetTypeHash(Key.GeometryShader);
	}

private:
	const FRHIVertexDeclaration* VertexDeclaration = nullptr;
	const FRHIVertexShader* VertexShader = nullptr;
	const FRHIMeshShader* MeshShader = nullptr;
	const FRHIAmplificationShader* AmplificationShader = nullptr;
	const FRHIPixelShader* PixelShader = nullptr;
	const FRHIGeometryShader* GeometryShader = nullptr;
};

/**
 * Encapsulates a bound shader state's entry in the cache.
 * Handles removal from the bound shader state cache on destruction.
 * RHIs that use cached bound shader states should create one for each bound shader state.
 */
class FCachedBoundShaderStateLink
{
public:

	/**
	 * The cached bound shader state.  This is not a reference counted pointer because we rely on the RHI to destruct this object
	 * when the bound shader state this references is destructed.
	 */
	FRHIBoundShaderState* BoundShaderState;

	/** Adds the bound shader state to the cache. */
	RHI_API FCachedBoundShaderStateLink(
		FRHIVertexDeclaration* VertexDeclaration,
		FRHIVertexShader* VertexShader,
		FRHIPixelShader* PixelShader,
		FRHIBoundShaderState* InBoundShaderState,
		bool bAddToSingleThreadedCache = true
		);

	/** Adds the bound shader state to the cache. */
	RHI_API FCachedBoundShaderStateLink(
		FRHIVertexDeclaration* VertexDeclaration,
		FRHIVertexShader* VertexShader,
		FRHIPixelShader* PixelShader,
		FRHIGeometryShader* GeometryShader,
		FRHIBoundShaderState* InBoundShaderState,
		bool bAddToSingleThreadedCache = true
	);

	/** Adds the bound shader state to the cache. */
	RHI_API FCachedBoundShaderStateLink(
		FRHIMeshShader* MeshShader,
		FRHIAmplificationShader* AmplificationShader,
		FRHIPixelShader* PixelShader,
		FRHIBoundShaderState* InBoundShaderState,
		bool bAddToSingleThreadedCache = true
	);

	/** Destructor.  Removes the bound shader state from the cache. */
	RHI_API ~FCachedBoundShaderStateLink();

	/**
	 * Get the RHI shader for the given frequency.
	 */
	inline FRHIVertexShader*   GetVertexShader() const   { return Key.GetVertexShader(); }
	inline FRHIMeshShader*     GetMeshShader() const     { return Key.GetMeshShader(); }
	inline FRHIAmplificationShader* GetAmplificationShader() const { return Key.GetAmplificationShader(); }
	inline FRHIPixelShader*    GetPixelShader() const    { return Key.GetPixelShader(); }
	inline FRHIGeometryShader* GetGeometryShader() const { return Key.GetGeometryShader(); }

	/**
	* Get the RHI vertex declaration.
	*/
	inline FRHIVertexDeclaration* GetVertexDeclaration() const { return Key.GetVertexDeclaration(); }

protected:
	FBoundShaderStateKey Key;
	bool bAddedToSingleThreadedCache;
};


/**
 * Searches for a cached bound shader state.
 * @return If a bound shader state matching the parameters is cached, it is returned; otherwise NULL is returned.
 */
extern RHI_API FCachedBoundShaderStateLink* GetCachedBoundShaderState(
	FRHIVertexDeclaration* VertexDeclaration,
	FRHIVertexShader* VertexShader,
	FRHIPixelShader* PixelShader,
	FRHIGeometryShader* GeometryShader = nullptr,
	FRHIMeshShader* MeshShader = nullptr,
	FRHIAmplificationShader* AmplificationShader = nullptr
	);

extern RHI_API void EmptyCachedBoundShaderStates();

class FCachedBoundShaderStateLink_Threadsafe : public FCachedBoundShaderStateLink
{
public:
	/** Adds the bound shader state to the cache. */
	FCachedBoundShaderStateLink_Threadsafe(
		FRHIVertexDeclaration* VertexDeclaration,
		FRHIVertexShader* VertexShader,
		FRHIPixelShader* PixelShader,
		FRHIBoundShaderState* InBoundShaderState
		)
		: FCachedBoundShaderStateLink(VertexDeclaration, VertexShader, PixelShader, InBoundShaderState, false)
	{
	}

	/** Adds the bound shader state to the cache. */
	FCachedBoundShaderStateLink_Threadsafe(
		FRHIVertexDeclaration* VertexDeclaration,
		FRHIVertexShader* VertexShader,
		FRHIPixelShader* PixelShader,
		FRHIGeometryShader* GeometryShader,
		FRHIBoundShaderState* InBoundShaderState
		)
		: FCachedBoundShaderStateLink(VertexDeclaration, VertexShader, PixelShader, GeometryShader, InBoundShaderState, false)
	{
	}

	/** Adds the bound shader state to the cache. */
	FCachedBoundShaderStateLink_Threadsafe(
		FRHIMeshShader* MeshShader,
		FRHIAmplificationShader* AmplificationShader,
		FRHIPixelShader* PixelShader,
		FRHIBoundShaderState* InBoundShaderState
	)
		: FCachedBoundShaderStateLink(MeshShader, AmplificationShader, PixelShader, InBoundShaderState, false)
	{
	}

	RHI_API void AddToCache();
	RHI_API void RemoveFromCache();
};

/**
 * Searches for a cached bound shader state. Threadsafe version.
 * @return If a bound shader state matching the parameters is cached, it is returned; otherwise NULL is returned.
 */
extern RHI_API FBoundShaderStateRHIRef GetCachedBoundShaderState_Threadsafe(
	FRHIVertexDeclaration* VertexDeclaration,
	FRHIVertexShader* VertexShader,
	FRHIPixelShader* PixelShader,
	FRHIGeometryShader* GeometryShader = nullptr,
	FRHIMeshShader* MeshShader = nullptr,
	FRHIAmplificationShader* AmplificationShader = nullptr
	);

