// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"
#include "RHIShaderPlatform.h"
#include "RHIFeatureLevel.h"
#include "UObject/NameTypes.h"

class FText;

extern RHI_API const FName LANGUAGE_D3D;
extern RHI_API const FName LANGUAGE_Metal;
extern RHI_API const FName LANGUAGE_OpenGL;
extern RHI_API const FName LANGUAGE_Vulkan;
extern RHI_API const FName LANGUAGE_Sony;
extern RHI_API const FName LANGUAGE_Nintendo;

class FGenericDataDrivenShaderPlatformInfo
{
	FName Name;
	FName PlatformName;
	FName Language;
	ERHIFeatureLevel::Type MaxFeatureLevel;
	FName ShaderFormat;
	uint32 ShaderPropertiesHash;
	uint32 bIsMobile : 1;
	uint32 bIsPC : 1;
	uint32 bIsConsole : 1;
	uint32 bIsAndroidOpenGLES : 1;

	uint32 bSupportsDebugViewShaders : 1;
	uint32 bSupportsMobileMultiView : 1;
	uint32 bSupportsArrayTextureCompression : 1;
	uint32 bSupportsDistanceFields : 1; // used for DFShadows and DFAO - since they had the same checks
	uint32 bSupportsDiaphragmDOF : 1;
	uint32 bSupportsRGBColorBuffer : 1;
	uint32 bSupportsPercentageCloserShadows : 1;
	uint32 bSupportsIndexBufferUAVs : 1;
	uint32 bSupportsInstancedStereo : 1;
	uint32 SupportsMultiViewport : int32(ERHIFeatureSupport::NumBits);
	uint32 bSupportsMSAA : 1;
	uint32 bSupports4ComponentUAVReadWrite : 1;
	uint32 bSupportsShaderRootConstants : 1;
	uint32 bSupportsShaderBundleDispatch : 1;
	uint32 bSupportsRenderTargetWriteMask : 1;
	uint32 bSupportsRayTracing : 1;
	uint32 bSupportsRayTracingCallableShaders : 1;
	uint32 bSupportsRayTracingProceduralPrimitive : 1;
	uint32 bSupportsRayTracingLinearSweptSpheres : 1;
	uint32 bSupportsRayTracingTraversalStatistics : 1;
	uint32 bSupportsRayTracingIndirectInstanceData : 1; // Whether instance transforms can be copied from the GPU to the TLAS instances buffer
	uint32 bSupportsRayTracingClusterOps : 1; // Accelerated building and ray tracing of Nanite clusters
	uint32 bSupportsPathTracing : 1; // Whether real-time path tracer is supported on this platform (avoids compiling unnecessary shaders)
	uint32 bSupportsShaderExecutionReordering : 1; // Does the platform support Shader Execution Reordering extensions?
	uint32 bSupportsGPUScene : 1;
	uint32 bSupportsUnrestrictedHalfFloatBuffers : 1;
	uint32 bSupportsPrimitiveShaders : 1;
	uint32 bSupportsUInt64ImageAtomics : 1;
	uint32 bRequiresVendorExtensionsForAtomics : 1;
	uint32 bSupportsNanite : 1;
	uint32 bSupportsLumenGI : 1;
	uint32 bSupportsSSDIndirect : 1;
	uint32 bSupportsTemporalHistoryUpscale : 1;
	uint32 bSupportsRTIndexFromVS : 1;
	uint32 bSupportsWaveOperations : int32(ERHIFeatureSupport::NumBits);
	uint32 bSupportsWavePermute : 1;
	uint32 MinimumWaveSize : 8;
	uint32 MaximumWaveSize : 8;
	uint32 bSupportsIntrinsicWaveOnce : 1;
	uint32 bSupportsConservativeRasterization : 1;
	uint32 bStripNaniteVertexShaders : 1;
	uint32 bRequiresExplicit128bitRT : 1;
	uint32 bSupportsGen5TemporalAA : 1;
	uint32 bTargetsTiledGPU : 1;
	uint32 bNeedsOfflineCompiler : 1;
	uint32 bSupportsComputeFramework : 1;
	uint32 bSupportsAnisotropicMaterials : 1;
	uint32 bSupportsDualSourceBlending : 1;
	uint32 bRequiresGeneratePrevTransformBuffer : 1;
	uint32 bRequiresRenderTargetDuringRaster : 1;
	uint32 bRequiresDisableForwardLocalLights : 1;
	uint32 bCompileSignalProcessingPipeline : 1;
	uint32 bSupportsMeshShadersTier0 : 1;
	uint32 bSupportsMeshShadersTier1 : 1;
	uint32 bSupportsMeshShadersWithClipDistance : 1;
	uint32 MaxMeshShaderThreadGroupSize : 10;
	uint32 bRequiresUnwrappedMeshShaderArgs : 1;
	uint32 bSupportsPerPixelDBufferMask : 1;
	uint32 bSupportsDxc : 1; // Whether DirectXShaderCompiler (DXC) is supported
	uint32 bIsSPIRV : 1;
	uint32 bSupportsVariableRateShading : 1;
	uint32 NumberOfComputeThreads : 10;
	uint32 bWaterUsesSimpleForwardShading : 1;
	uint32 bSupportsHairStrandGeometry : 1;
	uint32 bSupportsDOFHybridScattering : 1;
	uint32 bNeedsExtraMobileFrames : 1;
	uint32 bSupportsHZBOcclusion : 1;
	uint32 bSupportsWaterIndirectDraw : 1;
	uint32 bSupportsAsyncPipelineCompilation : 1;
	uint32 bSupportsVertexShaderSRVs : 1; // Whether SRVs can be bound to vertex shaders (may be independent from ManualVertexFetch)
	uint32 bSupportsVertexShaderUAVs : int32(ERHIFeatureSupport::NumBits); // Whether UAVs can be bound to vertex shaders. Requires run-time check of GRHIGlobals.SupportsVertexShaderUAVs.
	uint32 bSupportsTypedBufferSRVs : 1; // Buffer<>, texelbuffer/texture buffer, SRV with Format
	uint32 bSupportsManualVertexFetch : 1;
	uint32 bRequiresReverseCullingOnMobile : 1;
	uint32 bOverrideFMaterial_NeedsGBufferEnabled : 1;
	uint32 bSupportsFFTBloom : 1;
	uint32 bSupportsInlineRayTracing : 1;
	uint32 bInlineRayTracingRequiresBindless : 1;
	uint32 bSupportsRayTracingShaders : 1;
	uint32 bSupportsVertexShaderLayer : 1;
	uint32 bSupportsBindless : 1;
	uint32 bSupportsDescriptorRange : 1;
	uint32 StaticShaderBindingLayoutSupport : int32(ERHIStaticShaderBindingLayoutSupport::NumBits);
	uint32 bSupportsVolumeTextureAtomics : 1;
	uint32 bSupportsROV : 1;
	uint32 bSupportsOIT : 1;
	uint32 bSupportsRealTypes : int32(ERHIFeatureSupport::NumBits);
	uint32 EnablesHLSL2021ByDefault : 2; // 0: disabled, 1: global shaders only, 2: all shaders
	uint32 bSupportsSceneDataCompressedTransforms : 1;
	uint32 bIsPreviewPlatform : 1;
	uint32 bSupportsSwapchainUAVs : 1;
	uint32 bSupportsClipDistance : 1;
	uint32 bSupportsNNEShaders: 1;
	uint32 bSupportsShaderPipelines : 1;
	uint32 bSupportsUniformBufferObjects : 1;
	uint32 bRequiresBindfulUtilityShaders : 1;
	uint32 MaxSamplers : 8;
	uint32 SupportsBarycentricsIntrinsics : 1;
	uint32 SupportsBarycentricsSemantic : int32(ERHIFeatureSupport::NumBits);
	uint32 bSupportsWave64 : 1;
	uint32 bSupportsIndependentSamplers : 1;
	uint32 bSupportsWorkGraphs : 1;
	uint32 bSupportsWorkGraphsTier1_1 : 1;
	uint32 bSupportsDLSSShaders : 1;
	uint32 bSupportsAdaptiveGBuffer : 1;
	uint32 bSupportsSpecializationConstants : 1;

	// NOTE: When adding fields, you must also add to ParseDataDrivenShaderInfo!
	uint32 bContainsValidPlatformInfo : 1;

	FGenericDataDrivenShaderPlatformInfo()
	{
		FMemory::Memzero(this, sizeof(*this));

		SetDefaultValues();
	}

	FGenericDataDrivenShaderPlatformInfo(const FGenericDataDrivenShaderPlatformInfo&) = default;

	RHI_API void SetDefaultValues();

public:
	RHI_API static void Initialize();
	RHI_API static const EShaderPlatform GetShaderPlatformFromName(const FName ShaderPlatformName);

	static inline const FName GetName(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].Name;
	}

	static inline const FName GetPlatformName(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].PlatformName;
	}

	static inline const FName GetShaderFormat(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].ShaderFormat;
	}

	static inline uint32 GetShaderPlatformPropertiesHash(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].ShaderPropertiesHash;
	}

	static inline const bool GetIsLanguageD3D(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].Language == LANGUAGE_D3D;
	}

	static inline const bool GetIsLanguageMetal(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].Language == LANGUAGE_Metal;
	}

	static inline const bool GetIsLanguageOpenGL(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].Language == LANGUAGE_OpenGL;
	}

	static inline const bool GetIsLanguageVulkan(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].Language == LANGUAGE_Vulkan;
	}

	static inline const bool GetIsLanguageSony(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].Language == LANGUAGE_Sony;
	}

	static inline const bool GetIsLanguageNintendo(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].Language == LANGUAGE_Nintendo;
	}

	static inline const FName GetLanguage(const FStaticShaderPlatform Platform)
	{
		if(IsValid(Platform))
		{
			return Infos[Platform].Language;
		}
		return NAME_None;
	}

	static inline const ERHIFeatureLevel::Type GetMaxFeatureLevel(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].MaxFeatureLevel;
	}

	static inline const bool GetIsMobile(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bIsMobile;
	}

	static inline const bool GetIsMetalMobileSM5(const FStaticShaderPlatform Platform)
	{
		return Platform == SP_METAL_SM5_IOS;
	}

	static inline const bool GetIsMetalMobileSM6(const FStaticShaderPlatform Platform)
	{
		return Platform == SP_METAL_SM6_IOS;
	}

	static inline const bool GetIsPC(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bIsPC;
	}

	static inline const bool GetIsConsole(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bIsConsole;
	}

	static inline const bool GetIsAndroidOpenGLES(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bIsAndroidOpenGLES;
	}

	static inline const bool GetSupportsDebugViewShaders(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsDebugViewShaders;
	}

	static inline const bool GetSupportsMobileMultiView(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsMobileMultiView;
	}

	static inline const bool GetSupportsArrayTextureCompression(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsArrayTextureCompression;
	}

	static inline const bool GetSupportsDistanceFields(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsDistanceFields;
	}

	static inline const bool GetSupportsDiaphragmDOF(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsDiaphragmDOF;
	}

	static inline const bool GetSupportsRGBColorBuffer(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRGBColorBuffer;
	}

	static inline const bool GetSupportsPercentageCloserShadows(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsPercentageCloserShadows;
	}

	static inline const bool GetSupportsIndexBufferUAVs(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsIndexBufferUAVs;
	}

	static inline const bool GetSupportsInstancedStereo(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsInstancedStereo;
	}

	static inline const ERHIFeatureSupport GetSupportsMultiViewport(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return ERHIFeatureSupport(Infos[Platform].SupportsMultiViewport);
	}

	static inline const bool GetSupportsMSAA(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsMSAA;
	}

	static inline const bool GetSupports4ComponentUAVReadWrite(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupports4ComponentUAVReadWrite;
	}

	static inline const bool GetSupportsSwapchainUAVs(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsSwapchainUAVs;
	}

	static inline const bool GetSupportsShaderRootConstants(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsShaderRootConstants;
	}

	static inline const bool GetSupportsShaderBundleDispatch(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsShaderBundleDispatch;
	}

	static inline const bool GetSupportsRenderTargetWriteMask(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRenderTargetWriteMask;
	}

	static inline const bool GetSupportSceneDataCompressedTransforms(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsSceneDataCompressedTransforms;
	}

	static inline const bool GetSupportsRayTracing(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRayTracing;
	}

	static inline const bool GetSupportsRayTracingShaders(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));

		return Infos[Platform].bSupportsRayTracing && Infos[Platform].bSupportsRayTracingShaders;
	}

	static inline const bool GetSupportsInlineRayTracing(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRayTracing && Infos[Platform].bSupportsInlineRayTracing;
	}

	static inline const bool GetRequiresBindlessForInlineRayTracing(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bInlineRayTracingRequiresBindless;
	}

	static inline const bool GetSupportsRayTracingCallableShaders(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRayTracing && Infos[Platform].bSupportsRayTracingCallableShaders;
	}

	static inline const bool GetSupportsRayTracingProceduralPrimitive(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRayTracing && Infos[Platform].bSupportsRayTracingProceduralPrimitive;
	}

	static inline const bool GetSupportsRayTracingLinearSweptSpheres(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRayTracing && Infos[Platform].bSupportsRayTracingLinearSweptSpheres;
	}

	static inline const bool GetSupportsRayTracingTraversalStatistics(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRayTracing && Infos[Platform].bSupportsRayTracingTraversalStatistics;
	}

	static inline const bool GetSupportsRayTracingIndirectInstanceData(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRayTracing && Infos[Platform].bSupportsRayTracingIndirectInstanceData;
	}

	static inline const bool GetSupportsRayTracingClusterOps(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRayTracing && Infos[Platform].bSupportsRayTracingClusterOps;
	}

	static inline const bool GetSupportsPathTracing(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRayTracing && Infos[Platform].bSupportsPathTracing;
	}

	static inline const bool GetSupportsShaderExecutionReordering(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsShaderExecutionReordering;
	}

	static inline const bool GetSupportsComputeFramework(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsComputeFramework;
	}

	static inline const bool GetSupportsAnisotropicMaterials(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsAnisotropicMaterials;
	}

	static inline const bool GetTargetsTiledGPU(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bTargetsTiledGPU;
	}

	static inline const bool GetNeedsOfflineCompiler(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bNeedsOfflineCompiler;
	}

	static inline const bool GetSupportsUnrestrictedHalfFloatBuffers(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsUnrestrictedHalfFloatBuffers;
	}

	static inline const ERHIFeatureSupport GetSupportsWaveOperations(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return ERHIFeatureSupport(Infos[Platform].bSupportsWaveOperations);
	}

	static inline const bool GetSupportsWavePermute(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsWavePermute;
	}

	static inline const uint32 GetMinimumWaveSize(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].MinimumWaveSize;
	}

	static inline const uint32 GetMaximumWaveSize(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].MaximumWaveSize;
	}

	static inline const bool GetSupportsTemporalHistoryUpscale(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsTemporalHistoryUpscale;
	}

	static inline const bool GetSupportsGPUScene(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsGPUScene;
	}

	static inline const bool GetRequiresExplicit128bitRT(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bRequiresExplicit128bitRT;
	}

	static inline const bool GetStripNaniteVertexShaders(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bStripNaniteVertexShaders;
	}

	static inline const bool GetSupportsPrimitiveShaders(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsPrimitiveShaders;
	}

	static inline const bool GetSupportsUInt64ImageAtomics(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsUInt64ImageAtomics;
	}

	static inline const bool GetRequiresVendorExtensionsForAtomics(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bRequiresVendorExtensionsForAtomics;
	}

	static inline const bool GetSupportsNanite(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsNanite;
	}

	static inline const bool GetSupportsLumenGI(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsLumenGI;
	}

	static inline const bool GetSupportsSSDIndirect(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsSSDIndirect;
	}

	static inline const bool GetSupportsRTIndexFromVS(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsRTIndexFromVS;
	}

	static inline const bool GetSupportsIntrinsicWaveOnce(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsIntrinsicWaveOnce;
	}

	static inline const bool GetSupportsConservativeRasterization(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsConservativeRasterization;
	}

	static inline const bool GetSupportsGen5TemporalAA(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsGen5TemporalAA;
	}

	static inline const bool GetSupportsDualSourceBlending(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsDualSourceBlending;
	}

	static inline const bool GetRequiresGeneratePrevTransformBuffer(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bRequiresGeneratePrevTransformBuffer;
	}

	static inline const bool GetRequiresRenderTargetDuringRaster(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bRequiresRenderTargetDuringRaster;
	}

	static inline const bool GetRequiresDisableForwardLocalLights(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bRequiresDisableForwardLocalLights;
	}

	static inline const bool GetCompileSignalProcessingPipeline(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bCompileSignalProcessingPipeline;
	}

	static inline const bool GetSupportsMeshShadersTier0(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsMeshShadersTier0;
	}

	static inline const bool GetSupportsMeshShadersTier1(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsMeshShadersTier1;
	}

	static inline const bool GetSupportsMeshShadersWithClipDistance(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsMeshShadersWithClipDistance;
	}

	static inline const uint32 GetMaxMeshShaderThreadGroupSize(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].MaxMeshShaderThreadGroupSize;
	}

	static inline const bool GetRequiresUnwrappedMeshShaderArgs(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bRequiresUnwrappedMeshShaderArgs;
	}

	static inline const bool GetSupportsPerPixelDBufferMask(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsPerPixelDBufferMask;
	}

	UE_DEPRECATED(5.8, "GetIsHlslcc is always false so it will be removed.")
	static inline const bool GetIsHlslcc(const FStaticShaderPlatform Platform)
	{
		return false;
	}

	static inline const bool GetSupportsDxc(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsDxc;
	}

	static inline const bool GetIsSPIRV(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bIsSPIRV;
	}

	static inline const bool GetSupportsVariableRateShading(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsVariableRateShading;
	}

	static inline const uint32 GetNumberOfComputeThreads(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].NumberOfComputeThreads;
	}

	static inline const bool GetWaterUsesSimpleForwardShading(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bWaterUsesSimpleForwardShading;
	}

	static inline const bool GetSupportsHairStrandGeometry(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsHairStrandGeometry;
	}

	static inline const bool GetSupportsDOFHybridScattering(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsDOFHybridScattering;
	}

	static inline const bool GetNeedsExtraMobileFrames(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bNeedsExtraMobileFrames;
	}

	static inline const bool GetSupportsHZBOcclusion(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsHZBOcclusion;
	}

	static inline const bool GetSupportsWaterIndirectDraw(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsWaterIndirectDraw;
	}

	static inline const bool GetSupportsAsyncPipelineCompilation(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsAsyncPipelineCompilation;
	}

	static inline const bool GetSupportsVertexShaderSRVs(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsVertexShaderSRVs;
	}

	static inline const ERHIFeatureSupport GetSupportsVertexShaderUAVs(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return ERHIFeatureSupport(Infos[Platform].bSupportsVertexShaderUAVs);
	}

	static inline const bool GetSupportsTypedBufferSRVs(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsTypedBufferSRVs;
	}

	static inline const bool GetSupportsManualVertexFetch(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsManualVertexFetch;
	}

	static inline const bool GetRequiresReverseCullingOnMobile(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bRequiresReverseCullingOnMobile;
	}

	static inline const bool GetOverrideFMaterial_NeedsGBufferEnabled(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bOverrideFMaterial_NeedsGBufferEnabled;
	}

	static inline const bool GetSupportsFFTBloom(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsFFTBloom;
	}

	static inline const bool GetSupportsVertexShaderLayer(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsVertexShaderLayer;
	}

	static inline const bool GetSupportsBindless(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsBindless;
	}

	static inline const bool GetSupportsDescriptorRange(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsDescriptorRange;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "GetBindlessSupport is now GetSupportsBindless")
	static inline const ERHIBindlessSupport GetBindlessSupport(const FStaticShaderPlatform Platform)
	{
		return GetSupportsBindless(Platform) ? ERHIBindlessSupport::AllShaderTypes : ERHIBindlessSupport::Unsupported;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static inline const ERHIStaticShaderBindingLayoutSupport GetStaticShaderBindingLayoutSupport(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return static_cast<ERHIStaticShaderBindingLayoutSupport>(Infos[Platform].StaticShaderBindingLayoutSupport);
	}

	static inline const bool GetSupportsVolumeTextureAtomics(const FStaticShaderPlatform Platform)
	{
		return Infos[Platform].bSupportsVolumeTextureAtomics;
	}

	static inline const bool GetSupportsPipelineShaders(const FStaticShaderPlatform Platform)
	{
		return Infos[Platform].bSupportsShaderPipelines;
	}

	static inline const bool GetSupportsROV(const FStaticShaderPlatform Platform)
	{
		return Infos[Platform].bSupportsROV;
	}

	static inline const bool GetSupportsOIT(const FStaticShaderPlatform Platform)
	{
		return Infos[Platform].bSupportsOIT;
	}

	static inline const bool GetIsPreviewPlatform(const FStaticShaderPlatform Platform)
	{
		return Infos[Platform].bIsPreviewPlatform;
	}

	static inline const ERHIFeatureSupport GetSupportsRealTypes(const FStaticShaderPlatform Platform)
	{
		return ERHIFeatureSupport(Infos[Platform].bSupportsRealTypes);
	}

	static inline const uint32 GetEnablesHLSL2021ByDefault(const FStaticShaderPlatform Platform)
	{
		return Infos[Platform].EnablesHLSL2021ByDefault;
	}
	
	static inline const bool GetSupportsClipDistance(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsClipDistance;
	}

	static inline const bool GetSupportsNNEShaders(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsNNEShaders;
	}
	
	static inline const bool GetSupportsUniformBufferObjects(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsUniformBufferObjects;
	}
	
	static inline const bool GetRequiresBindfulUtilityShaders(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bRequiresBindfulUtilityShaders;
	}
	
	static inline const uint32 GetMaxSamplers(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].MaxSamplers;
	}

	static inline const bool GetSupportsBarycentricsIntrinsics(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].SupportsBarycentricsIntrinsics;
	}

	static inline const ERHIFeatureSupport GetSupportsBarycentricsSemantic(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return ERHIFeatureSupport(Infos[Platform].SupportsBarycentricsSemantic);
	}

	static inline const bool GetSupportsWave64(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsWave64;
	}

	static inline const bool GetSupportsIndependentSamplers(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsIndependentSamplers;
	}

	static inline const bool GetSupportsWorkGraphs(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsWorkGraphs;
	}

	static inline const bool GetSupportsWorkGraphsTier1_1(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsWorkGraphsTier1_1;
	}

	static inline const bool GetSupportsDLSSShaders(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsDLSSShaders;
	}

	static inline const bool GetSupportsAdaptiveGBuffer(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsAdaptiveGBuffer;
	}
	
	static inline const bool GetSupportsSpecializationConstants(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return Infos[Platform].bSupportsSpecializationConstants;
	}

	static inline const bool IsValid(const FStaticShaderPlatform Platform)
	{
		return Infos[Platform].bContainsValidPlatformInfo;
	}

#if WITH_EDITOR
	RHI_API static void UpdatePreviewPlatforms();
	RHI_API static FText GetFriendlyName(EShaderPlatform Platform, FName DeviceProfileName = NAME_None);
	RHI_API static const EShaderPlatform GetPreviewShaderPlatformParent(EShaderPlatform Platform);
	RHI_API static const EShaderPlatform GetPreviewRuntimePlatform(EShaderPlatform Platform);
	RHI_API static const bool CanUseForMaterialValidation(EShaderPlatform Platform);
	RHI_API static FName GetPreferredShaderCodeFormat(EShaderPlatform Platform);

	RHI_API static TMap<FString, TFunction<bool(const FStaticShaderPlatform Platform)>> PropertyToShaderPlatformFunctionMap;
#endif

	static inline const void OverrideShaderFormatForShaderPlatform(const FStaticShaderPlatform Platform, FName ShaderFormat)
	{
		check(IsValid(Platform));
		Infos[Platform].ShaderFormat = ShaderFormat;
	}

private:
	RHI_API static void ParseDataDrivenShaderInfo(const FConfigSection& Section, uint32 Index);

	RHI_API static FGenericDataDrivenShaderPlatformInfo Infos[SP_NumPlatforms];
};

#if USE_STATIC_SHADER_PLATFORM_ENUMS || USE_STATIC_FEATURE_LEVEL_ENUMS || USE_STATIC_SHADER_PLATFORM_INFO

#define IMPLEMENT_DDPSPI_SETTING_WITH_RETURN_TYPE(ReturnType, Function, Value) \
	static inline const ReturnType Function(const FStaticShaderPlatform Platform) \
	{ \
		checkSlow(!FGenericDataDrivenShaderPlatformInfo::IsValid(Platform) || FGenericDataDrivenShaderPlatformInfo::Function(Platform) == Value); \
		return Value; \
	}
#define IMPLEMENT_DDPSPI_SETTING(Function, Value) IMPLEMENT_DDPSPI_SETTING_WITH_RETURN_TYPE(bool, Function, Value)

#include COMPILED_PLATFORM_HEADER(DataDrivenShaderPlatformInfo.inl)

#else
using FDataDrivenShaderPlatformInfo = FGenericDataDrivenShaderPlatformInfo;
#endif

inline bool IsPCPlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsPC(Platform);
}

/** Whether the shader platform corresponds to the ES3.1/Metal/Vulkan feature level. */
inline bool IsMobilePlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(Platform) == ERHIFeatureLevel::ES3_1;
}

inline bool IsOpenGLPlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsLanguageOpenGL(Platform);
}

inline bool IsMetalPlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsLanguageMetal(Platform);
}

inline bool IsMetalMobilePlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsLanguageMetal(Platform)
		&& FDataDrivenShaderPlatformInfo::GetIsMobile(Platform);
}

inline bool IsMetalMobileSM5Platform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsMetalMobileSM5(Platform);
}

inline bool IsMetalMobileSM6Platform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsMetalMobileSM6(Platform);
}

inline bool IsMetalSM5Platform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsLanguageMetal(Platform)
		&& FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(Platform) == ERHIFeatureLevel::SM5;
}

inline bool IsMetalSM6Platform(const FStaticShaderPlatform Platform)
{
    return FDataDrivenShaderPlatformInfo::GetIsLanguageMetal(Platform)
        && FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(Platform) == ERHIFeatureLevel::SM6;
}

inline bool IsConsolePlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsConsole(Platform);
}

// @todo: data drive uses of this function
inline bool IsAndroidPlatform(const FStaticShaderPlatform Platform)
{
	return (Platform == SP_VULKAN_ES3_1_ANDROID) || (Platform == SP_VULKAN_SM5_ANDROID) || (Platform == SP_OPENGL_ES3_1_ANDROID);
}

inline bool IsVulkanPlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsLanguageVulkan(Platform);
}

// @todo: data drive uses of this function
inline bool IsVulkanMobileSM5Platform(const FStaticShaderPlatform Platform)
{
	return Platform == SP_VULKAN_SM5_ANDROID;
	// 	return FDataDrivenShaderPlatformInfo::GetIsLanguageVulkan(Platform)
	// 		&& FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(Platform) == ERHIFeatureLevel::SM5
	// 		&& FDataDrivenShaderPlatformInfo::GetIsMobile(Platform);
}

inline bool IsAndroidOpenGLESPlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsAndroidOpenGLES(Platform);
}

inline bool IsVulkanMobilePlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsLanguageVulkan(Platform)
		//&& FDataDrivenShaderPlatformInfo::GetIsMobile(Platform)
		// This was limited to the ES3_1 platforms when hard coded
		&& FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(Platform) == ERHIFeatureLevel::ES3_1;
}

inline bool IsD3DPlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsLanguageD3D(Platform);
}

UE_DEPRECATED(5.8, "IsHlslccShaderPlatform is always false so it will be removed.")
inline bool IsHlslccShaderPlatform(const FStaticShaderPlatform Platform)
{
	return false;
}

inline FStaticFeatureLevel GetMaxSupportedFeatureLevel(const FStaticShaderPlatform InShaderPlatform)
{
	return FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(InShaderPlatform);
}

/* Returns true if the shader platform Platform is used to simulate a mobile feature level on a PC platform. */
inline bool IsSimulatedPlatform(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(Platform);
}

#if WITH_EDITOR
inline EShaderPlatform GetSimulatedPlatform(FStaticShaderPlatform Platform)
{
	if (IsSimulatedPlatform(Platform))
	{
		return FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(Platform);
	}

	return Platform;
}
#endif // WITH_EDITOR

/** Returns true if the feature level is supported by the shader platform. */
inline bool IsFeatureLevelSupported(const FStaticShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel)
{
	return InFeatureLevel <= GetMaxSupportedFeatureLevel(InShaderPlatform);
}

inline bool RHISupportsSeparateMSAAAndResolveTextures(const FStaticShaderPlatform Platform)
{
	// Metal mobile devices and Android ES3.1 need to handle MSAA and resolve textures internally (unless RHICreateTexture2D was changed to take an optional resolve target)
	return !IsMetalMobilePlatform(Platform);
}

inline bool RHISupportsGeometryShaders(const FStaticShaderPlatform Platform)
{
	return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5)
		&& !IsMetalPlatform(Platform)
		&& !IsVulkanMobilePlatform(Platform)
		&& !IsVulkanMobileSM5Platform(Platform)
		&& !(FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(Platform) && FDataDrivenShaderPlatformInfo::GetIsSPIRV(Platform));
}

inline bool RHIHasTiledGPU(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetTargetsTiledGPU(Platform);
}

inline bool RHISupportsMobileMultiView(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsMobileMultiView(Platform);
}

inline bool RHISupportsNativeShaderLibraries(const FStaticShaderPlatform Platform)
{
	return IsMetalPlatform(Platform);
}

inline bool RHISupportsShaderPipelines(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsPipelineShaders(Platform);
}

inline bool RHISupportsDualSourceBlending(const FStaticShaderPlatform Platform)
{
	// Check if the platform supports dual src blending from DDPI
	return FDataDrivenShaderPlatformInfo::GetSupportsDualSourceBlending(Platform);
}

// helper to check that the shader platform supports creating a UAV off an index buffer.
inline bool RHISupportsIndexBufferUAVs(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsIndexBufferUAVs(Platform);
}



inline bool RHISupportsInstancedStereo(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsInstancedStereo(Platform);
}

/**
 * Can this platform implement instanced stereo rendering by rendering to multiple viewports.
 * Note: run-time users should always check GRHISupportsArrayIndexFromAnyShader as well, since for some SPs (particularly PCD3D_SM5) minspec does not guarantee that feature.
 **/
inline bool RHISupportsMultiViewport(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsMultiViewport(Platform) != ERHIFeatureSupport::Unsupported;
}

inline bool RHISupportsMSAA(const FStaticShaderPlatform Platform)
{
	// @todo platplug: Maybe this should become bDisallowMSAA to default of 0 is a better default (since now MSAA is opt-out more than opt-in) 
	return FDataDrivenShaderPlatformInfo::GetSupportsMSAA(Platform);
}

inline bool RHISupportsBufferLoadTypeConversion(const FStaticShaderPlatform Platform)
{
	return !IsMetalPlatform(Platform) && !IsOpenGLPlatform(Platform);
}

/** Whether the platform supports reading from volume textures (does not cover rendering to volume textures). */
inline bool RHISupportsVolumeTextures(const FStaticFeatureLevel FeatureLevel)
{
	return FeatureLevel >= ERHIFeatureLevel::SM5;
}

inline bool RHISupportsVertexShaderLayer(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderLayer(Platform);
}

/** Return true if and only if the GPU support rendering to volume textures (2D Array, 3D) is guaranteed supported for a target platform.
	if PipelineVolumeTextureLUTSupportGuaranteedAtRuntime is true then it is guaranteed that GSupportsVolumeTextureRendering is true at runtime.
*/
inline bool RHIVolumeTextureRenderingSupportGuaranteed(const FStaticShaderPlatform Platform)
{
	return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5)
		&& (!IsMetalPlatform(Platform) || RHISupportsVertexShaderLayer(Platform)) // For Metal only shader platforms & versions that support vertex-shader-layer can render to volume textures - this is a compile/cook time check.
		&& !IsOpenGLPlatform(Platform);		// Apparently, some OpenGL 3.3 cards support SM4 but can't render to volume textures
}

inline bool RHISupports4ComponentUAVReadWrite(const FStaticShaderPlatform Platform)
{
	// Must match usf PLATFORM_SUPPORTS_4COMPONENT_UAV_READ_WRITE
	// D3D11 does not support multi-component loads from a UAV: "error X3676: typed UAV loads are only allowed for single-component 32-bit element types"
	return FDataDrivenShaderPlatformInfo::GetSupports4ComponentUAVReadWrite(Platform);
}

/** Whether Manual Vertex Fetch is supported for the specified shader platform.
	Shader Platform must not use the mobile renderer, and for Metal, the shader language must be at least 2. */
inline bool RHISupportsManualVertexFetch(const FStaticShaderPlatform InShaderPlatform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsManualVertexFetch(InShaderPlatform);
}

inline bool RHISupportsSwapchainUAVs(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsSwapchainUAVs(Platform);
}

/**
 * Returns true if SV_VertexID contains BaseVertexIndex passed to the draw call, false if shaders must manually construct an absolute VertexID.
 */
inline bool RHISupportsAbsoluteVertexID(const FStaticShaderPlatform InShaderPlatform)
{
	return IsVulkanPlatform(InShaderPlatform) || IsVulkanMobilePlatform(InShaderPlatform);
}

/** Whether this platform can build acceleration structures and use full ray tracing pipelines or inline ray tracing (ray queries).
 *  To use at runtime, also check GRHISupportsRayTracing and r.RayTracing CVar (see IsRayTracingEnabled() helper).
 *  Check GRHISupportsRayTracingShaders before using full ray tracing pipeline state objects.
 *  Check GRHISupportsInlineRayTracing before using inline ray tracing features in compute and other shaders.
 **/
inline bool RHISupportsRayTracing(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsRayTracing(Platform);
}

/** Whether this platform can compile ray tracing shaders (regardless of project settings).
 *  To use at runtime, also check GRHISupportsRayTracing and r.RayTracing CVar (see IsRayTracingEnabled() helper).
 **/
inline bool RHISupportsRayTracingShaders(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsRayTracingShaders(Platform);
}

/** Whether this platform can compile shaders with inline ray tracing features.
 *  To use at runtime, also check GRHISupportsRayTracing and r.RayTracing CVar (see IsRayTracingEnabled() helper).
 **/
inline bool RHISupportsInlineRayTracing(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsInlineRayTracing(Platform);
}

/** Whether this platform can compile ray tracing callable shaders.
 *  To use at runtime, also check GRHISupportsRayTracing and r.RayTracing CVar (see IsRayTracingEnabled() helper).
 **/
inline bool RHISupportsRayTracingCallableShaders(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsRayTracingCallableShaders(Platform);
}

/** Can this platform compile mesh shaders with tier0 capability.
 *  To use at runtime, also check GRHISupportsMeshShadersTier0.
 **/
inline bool RHISupportsMeshShadersTier0(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsMeshShadersTier0(Platform);
}

/** Can this platform compile mesh shaders with tier1 capability.
 *  To use at runtime, also check GRHISupportsMeshShadersTier1.
 **/
inline bool RHISupportsMeshShadersTier1(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsMeshShadersTier1(Platform);
}

inline uint32 RHIMaxMeshShaderThreadGroupSize(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetMaxMeshShaderThreadGroupSize(Platform);
}

/** Can this platform compile shaders that use shader model 6.0 wave intrinsics.
 *  To use such shaders at runtime, also check GRHISupportsWaveOperations.
 **/
inline bool RHISupportsWaveOperations(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(Platform) != ERHIFeatureSupport::Unsupported;
}

/** True if the given shader platform supports shader root constants */
inline bool RHISupportsShaderRootConstants(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsShaderRootConstants(Platform);
}

/** True if the given shader platform supports shader bundle dispatch */
inline bool RHISupportsShaderBundleDispatch(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsShaderBundleDispatch(Platform);
}

/** True if the given shader platform supports a render target write mask */
inline bool RHISupportsRenderTargetWriteMask(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsRenderTargetWriteMask(Platform);
}

/** True if the given shader platform supports overestimated conservative rasterization */
inline bool RHISupportsConservativeRasterization(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsConservativeRasterization(Platform);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_DEPRECATED(5.7, "RHIGetBindlessSupport is deprecated in favor of FDataDrivenShaderPlatformInfo::GetSupportsBindless")
inline ERHIBindlessSupport RHIGetBindlessSupport(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetBindlessSupport(Platform);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** True if the given shader platform supports static shader resource tables. */
inline ERHIStaticShaderBindingLayoutSupport RHIGetStaticShaderBindingLayoutSupport(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetStaticShaderBindingLayoutSupport(Platform);
}

inline bool RHISupportsVolumeTextureAtomics(EShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsVolumeTextureAtomics(Platform);
}

/** True if the platform supports wave size of 64 */
inline bool RHISupportsWaveSize64(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsWave64(Platform);
}

/** True if the platform supports Work Graphs */
inline bool RHISupportsWorkGraphs(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsWorkGraphs(Platform);
}

inline bool RHISupportsWorkGraphsTier1_1(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsWorkGraphsTier1_1(Platform);
}
