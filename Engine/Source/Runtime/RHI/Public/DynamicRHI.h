// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
DynamicRHI.h: Dynamically bound Render Hardware Interface definitions.
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "RHIBufferInitializer.h"
#include "RHIContext.h"
#include "RHIDescriptorRange.h"
#include "RHIFeatureLevel.h"
#include "RHIShaderLibrary.h"
#include "RHITextureInitializer.h"
#include "MultiGPU.h"
#include "Serialization/MemoryLayout.h"
#include "Containers/ArrayView.h"
#include "Misc/EnumClassFlags.h"
#include "Async/TaskGraphInterfaces.h"
#include "RHIStats.h"

class FBlendStateInitializerRHI;
class FGraphicsPipelineStateInitializer;
class FReadSurfaceDataFlags;
class FRHICommandList;
class FRHICommandListBase;
class FRHIComputeFence;
class FRayTracingPipelineState;
class IRHITransientResourceAllocator;
struct FDepthStencilStateInitializerRHI;
struct FDisplayInformation;
struct FRasterizerStateInitializerRHI;
struct FRHIBufferCreateDesc;
struct FRHIResourceCollectionMember;
struct FRHIResourceCreateInfo;
struct FRHIResourceInfo;
struct FRHIUniformBufferLayout;
struct FSamplerStateInitializerRHI;
struct FTextureMemoryStats;
struct FRHIGPUMask;
struct FUpdateTexture3DData;
struct FResourceArrayUploadInterface;

typedef TArray<FScreenResolutionRHI> FScreenResolutionArray;
using FDisplayInformationArray = TArray<FDisplayInformation>;

//UE_DEPRECATED(5.3, "Use the CreateShaderResourceView function that takes an FRHIBufferSRVCreateDesc.")
struct FShaderResourceViewInitializer : public FRHIViewDesc::FBufferSRV::FInitializer
{
	FRHIBuffer* Buffer;

	RHI_API FShaderResourceViewInitializer(FRHIBuffer* InBuffer, EPixelFormat InFormat, uint32 InStartOffsetBytes, uint32 InNumElements);
	RHI_API FShaderResourceViewInitializer(FRHIBuffer* InBuffer, EPixelFormat InFormat);
	RHI_API FShaderResourceViewInitializer(FRHIBuffer* InBuffer, uint32 InStartOffsetBytes, uint32 InNumElements);
	RHI_API FShaderResourceViewInitializer(FRHIBuffer* InBuffer, FRHIRayTracingScene* InRayTracingScene, uint32 InStartOffsetBytes);
	RHI_API FShaderResourceViewInitializer(FRHIBuffer* InBuffer);
};


/*
* FRawBufferShaderResourceViewInitializer can be used to explicitly create a raw view for any buffer,
* even if it was not created with EBufferUsageFlags::ByteAddressBuffer flag.
* Can only be used if GRHIGlobals.SupportsRawViewsForAnyBuffer is set.
*/
//UE_DEPRECATED(5.3, "Use the CreateShaderResourceView function that takes an FRHIBufferSRVCreateDesc, and call SetRawAccess(true).")
struct FRawBufferShaderResourceViewInitializer : public FShaderResourceViewInitializer
{
	FRawBufferShaderResourceViewInitializer(FRHIBuffer* InBuffer)
		: FShaderResourceViewInitializer(InBuffer)
	{
		SetType(FRHIViewDesc::EBufferType::Raw);
	}
};

class FDynamicRHI;

class FDefaultRHIRenderQueryPool final : public FRHIRenderQueryPool
{
public:
	FDefaultRHIRenderQueryPool(ERenderQueryType InQueryType)
		: QueryType(InQueryType)
	{}

	RHI_API virtual ~FDefaultRHIRenderQueryPool();

private:
	RHI_API virtual FRHIPooledRenderQuery AllocateQuery() override;
	RHI_API virtual void ReleaseQuery(TRefCountPtr<FRHIRenderQuery>&& Query) override;

	const ERenderQueryType QueryType;
	uint32 AllocatedQueries = 0;
	TArray<TRefCountPtr<FRHIRenderQuery>> Queries;
	UE::FMutex QueriesMutex;
};

struct FRHICalcTextureSizeResult
{
	// The total size of the texture, in bytes.
	uint64 Size;

	// The required address alignment for the texture.
	uint32 Align;
};

struct FRHILockedTextureDesc
{
	FRHITexture* Texture = nullptr;
	uint32       FaceIndex = 0;
	uint32       ArrayIndex = 0;
	uint32       MipIndex = 0;

	bool operator==(const FRHILockedTextureDesc& Other) const
	{
		return Texture    == Other.Texture
			&& FaceIndex  == Other.FaceIndex
			&& ArrayIndex == Other.ArrayIndex
			&& MipIndex   == Other.MipIndex;
	}
	bool operator!=(const FRHILockedTextureDesc& Other) const
	{
		return !(*this == Other);
	}
};

struct FRHILockTextureArgs : public FRHILockedTextureDesc
{
	static inline FRHILockTextureArgs Lock2D(FRHITexture* InTexture, uint32 InMipIndex, EResourceLockMode InLockMode, bool bInLockWithinMiptail, bool bFlushRHIThread = true)
	{
		FRHILockTextureArgs Result;

		Result.Texture = InTexture;
		Result.LockMode = InLockMode;
		Result.MipIndex = InMipIndex;
		Result.bLockWithinMiptail = bInLockWithinMiptail;
		Result.bNeedsDefaultRHIFlush = bFlushRHIThread;

		return Result;
	}
	static inline FRHILockTextureArgs Lock2DArray(FRHITexture* InTexture, uint32 InArrayIndex, uint32 InMipIndex, EResourceLockMode InLockMode, bool bInLockWithinMiptail)
	{
		FRHILockTextureArgs Result;

		Result.Texture = InTexture;
		Result.LockMode = InLockMode;
		Result.ArrayIndex = InArrayIndex;
		Result.MipIndex = InMipIndex;
		Result.bLockWithinMiptail = bInLockWithinMiptail;

		return Result;
	}
	static inline FRHILockTextureArgs LockCubeFace(FRHITexture* InTexture, uint32 InFaceIndex, uint32 InArrayIndex, uint32 InMipIndex, EResourceLockMode InLockMode, bool bInLockWithinMiptail)
	{
		FRHILockTextureArgs Result;

		Result.Texture = InTexture;
		Result.LockMode = InLockMode;
		Result.FaceIndex = InFaceIndex;
		Result.ArrayIndex = InArrayIndex;
		Result.MipIndex = InMipIndex;
		Result.bLockWithinMiptail = bInLockWithinMiptail;

		return Result;
	}

	EResourceLockMode LockMode = RLM_Num;
	bool              bLockWithinMiptail = false;
	bool              bNeedsDefaultRHIFlush = false;

protected:
	// Default constructor restricted to use in static functions
	FRHILockTextureArgs() = default;
};

struct FRHILockTextureResult
{
	void*  Data      = nullptr;
	uint64 ByteCount = 0;
	uint32 Stride    = 0;
};

struct FRHIEndFrameArgs
{
	// Increments once per call to RHIEndFrame
	uint32 FrameNumber;

#if WITH_RHI_BREADCRUMBS
	const TRHIPipelineArray<FRHIBreadcrumbNode*>& GPUBreadcrumbs;
#endif
#if STATS
	TOptional<int64> StatsFrame;
#endif

	bool bProfileNextFrame = false;
};

/** The interface which is implemented by the dynamically bound RHI. */
class FDynamicRHI
{
public:
	using FRHICalcTextureSizeResult = ::FRHICalcTextureSizeResult;

	/** Declare a virtual destructor, so the dynamic RHI can be deleted without knowing its type. */
	RHI_API virtual ~FDynamicRHI();

	/** Initializes the RHI; separate from IDynamicRHIModule::CreateRHI so that GDynamicRHI is set when it is called. */
	virtual void Init() = 0;

	/** Called after the RHI is initialized; before the render thread is started. */
	virtual void PostInit() {}

	/** Shutdown the RHI; handle shutdown and resource destruction before the RHI's actual destructor is called (so that all resources of the RHI are still available for shutdown). */
	virtual void Shutdown() = 0;

	virtual const TCHAR* GetName() = 0;

	virtual ERHIInterfaceType GetInterfaceType() const { return ERHIInterfaceType::Hidden; }
	virtual FDynamicRHI* GetNonValidationRHI() { return this; }

	/** Called after PostInit to initialize the pixel format info, which is needed for some commands default implementations */
	void InitPixelFormatInfo(const TArray<uint32>& PixelFormatBlockBytesIn)
	{
		PixelFormatBlockBytes = PixelFormatBlockBytesIn;
	}

	/////// RHI Methods

	RHI_API virtual void RHIEndFrame_RenderThread(FRHICommandListImmediate& RHICmdList);
	virtual void RHIEndFrame(const FRHIEndFrameArgs& Args) = 0;

	// FlushType: Thread safe
	virtual FSamplerStateRHIRef RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer) = 0;

	// FlushType: Thread safe
	virtual FRasterizerStateRHIRef RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer) = 0;

	// FlushType: Thread safe
	virtual FDepthStencilStateRHIRef RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer) = 0;

	// FlushType: Thread safe
	virtual FBlendStateRHIRef RHICreateBlendState(const FBlendStateInitializerRHI& Initializer) = 0;

	// FlushType: Wait RHI Thread
	virtual FVertexDeclarationRHIRef RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements) = 0;

	// FlushType: Wait RHI Thread
	virtual FPixelShaderRHIRef RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc) = 0;

	// FlushType: Wait RHI Thread
	virtual FVertexShaderRHIRef RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc) = 0;

	// FlushType: Wait RHI Thread
	virtual FGeometryShaderRHIRef RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc) = 0;

	// FlushType: Wait RHI Thread
	virtual FMeshShaderRHIRef RHICreateMeshShader(const FRHICreateShaderDesc& CreateShaderDesc)
	{
		return FMeshShaderRHIRef();
	}

	// FlushType: Wait RHI Thread
	virtual FAmplificationShaderRHIRef RHICreateAmplificationShader(const FRHICreateShaderDesc& CreateShaderDesc)
	{
		return FAmplificationShaderRHIRef();
	}

	// Some RHIs can have pending messages/logs for error tracking, or debug modes
	virtual void FlushPendingLogs() {}

	// FlushType: Wait RHI Thread
	virtual FComputeShaderRHIRef RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc) = 0;

	// FlushType: Wait RHI Thread
	virtual FWorkGraphShaderRHIRef RHICreateWorkGraphShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency)
	{
		return FWorkGraphShaderRHIRef();
	}

	/**
	 * Attempts to open a shader library for the given shader platform & name within the provided directory.
	 * @param Platform The shader platform for shaders withing the library.
	 * @param FilePath The directory in which the library should exist.
	 * @param Name The name of the library, i.e. "Global" or "Unreal" without shader-platform or file-extension qualification.
	 * @return The new library if one exists and can be constructed, otherwise nil.
	 */
	 // FlushType: Must be Thread-Safe.
	virtual FRHIShaderLibraryRef RHICreateShaderLibrary(EShaderPlatform Platform, FString const& FilePath, FString const& Name)
	{
		return nullptr;
	}

	virtual FGPUFenceRHIRef RHICreateGPUFence(const FName &Name) = 0;

	//
	// Called by the thread recording an RHI command list (via RHICmdList.WriteGPUFence()).
	// Allows the platform RHI to perform operations on the GPU fence at the top-of-pipe.
	// Default implementation just enqueues an RHI command to call IRHIComputeContext::WriteGPUFence().
	//
	RHI_API virtual void RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI);

	virtual void RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo)
	{
	}

	virtual void RHIReleaseTransition(FRHITransition* Transition)
	{
	}

	/**
	* Create a new transient resource allocator
	*/
	virtual IRHITransientResourceAllocator* RHICreateTransientResourceAllocator() { return nullptr; }

	/**
	* Creates a staging buffer, which is memory visible to the cpu without any locking.
	* @return The new staging-buffer.
	*/
	// FlushType: Thread safe.	
	virtual FStagingBufferRHIRef RHICreateStagingBuffer()
	{
		return new FGenericRHIStagingBuffer();
	}

	/**
	 * Lock a staging buffer to read contents on the CPU that were written by the GPU, without having to stall.
	 * @discussion This function requires that you have issued an CopyToStagingBuffer invocation and verified that the FRHIGPUFence has been signaled before calling.
	 * @param StagingBuffer The buffer to lock.
	 * @param Fence An optional fence synchronized with the last buffer update.
	 * @param Offset The offset in the buffer to return.
	 * @param SizeRHI The length of the region in the buffer to lock.
	 * @returns A pointer to the data starting at 'Offset' and of length 'SizeRHI' from 'StagingBuffer', or nullptr when there is an error.
	 */
	RHI_API virtual void* RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI);

	/**
	 * Unlock a staging buffer previously locked with RHILockStagingBuffer.
	 * @param StagingBuffer The buffer that was previously locked.
	 */
	RHI_API virtual void RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer);

	/**
	 * Lock a staging buffer to read contents on the CPU that were written by the GPU, without having to stall.
	 * @discussion This function requires that you have issued an CopyToStagingBuffer invocation and verified that the FRHIGPUFence has been signaled before calling.
	 * @param RHICmdList The command-list to execute on or synchronize with.
	 * @param StagingBuffer The buffer to lock.
	 * @param Fence An optional fence synchronized with the last buffer update.
	 * @param Offset The offset in the buffer to return.
	 * @param SizeRHI The length of the region in the buffer to lock.
	 * @returns A pointer to the data starting at 'Offset' and of length 'SizeRHI' from 'StagingBuffer', or nullptr when there is an error.
	 */
	RHI_API virtual void* LockStagingBuffer_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI);

	/**
	 * Unlock a staging buffer previously locked with LockStagingBuffer_RenderThread.
	 * @param RHICmdList The command-list to execute on or synchronize with.
	 * @param StagingBuffer The buffer what was previously locked.
	 */
	RHI_API virtual void UnlockStagingBuffer_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHIStagingBuffer* StagingBuffer);

	/**
	* Creates a bound shader state instance which encapsulates a decl, vertex shader and pixel shader
	* CAUTION: Even though this is marked as threadsafe, it is only valid to call from the render thread or the RHI thread. It need not be threadsafe unless the RHI support parallel translation.
	* CAUTION: Platforms that support RHIThread but don't actually have a threadsafe implementation must flush internally with FScopedRHIThreadStaller StallRHIThread(FRHICommandListImmediate::Get()); when the call is from the render thread
	* @param VertexDeclaration - existing vertex decl
	* @param VertexShader - existing vertex shader
	* @param GeometryShader - existing geometry shader
	* @param PixelShader - existing pixel shader
	*/
	// FlushType: Thread safe, but varies depending on the RHI
	virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIVertexDeclaration* VertexDeclaration, FRHIVertexShader* VertexShader, FRHIPixelShader* PixelShader, FRHIGeometryShader* GeometryShader) = 0;

#if PLATFORM_SUPPORTS_MESH_SHADERS && PLATFORM_USE_FALLBACK_PSO
	/**
	* Creates a bound shader state instance which encapsulates an amplification shader, a mesh shader, and pixel shader
	* CAUTION: Even though this is marked as threadsafe, it is only valid to call from the render thread or the RHI thread. It need not be threadsafe unless the RHI support parallel translation.
	* CAUTION: Platforms that support RHIThread but don't actually have a threadsafe implementation must flush internally with FScopedRHIThreadStaller StallRHIThread(FRHICommandListImmediate::Get()); when the call is from the render thread
	* @param AmplificationShader - existing amplification shader
	* @param MeshShader - existing mesh shader
	* @param PixelShader - existing pixel shader
	*/
	// FlushType: Thread safe, but varies depending on the RHI
	virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIAmplificationShader* AmplificationShader, FRHIMeshShader* MeshShader, FRHIPixelShader* PixelShader) = 0;
#endif

	/**
	* Creates a graphics pipeline state object (PSO) that represents a complete gpu pipeline for rendering.
	* This function should be considered expensive to call at runtime and may cause hitches as pipelines are compiled.
	* @param Initializer - Descriptor object defining all the information needed to create the PSO, as well as behavior hints to the RHI.
	* @return FGraphicsPipelineStateRHIRef that can be bound for rendering; nullptr if the compilation fails.
	* CAUTION: On certain RHI implementations (eg, ones that do not support runtime compilation) a compilation failure is a Fatal error and this function will not return.
	* CAUTION: Even though this is marked as threadsafe, it is only valid to call from the render thread or the RHI thread. It need not be threadsafe unless the RHI support parallel translation.
	* CAUTION: Platforms that support RHIThread but don't actually have a threadsafe implementation must flush internally with FScopedRHIThreadStaller StallRHIThread(FRHICommandListImmediate::Get()); when the call is from the render thread
	*/
	// FlushType: Thread safe
	virtual FGraphicsPipelineStateRHIRef RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer) = 0;

	// FlushType: Thread safe
	virtual FComputePipelineStateRHIRef RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer) = 0;

	virtual FWorkGraphPipelineStateRHIRef RHICreateWorkGraphPipelineState(const FWorkGraphPipelineStateInitializer& Initializer)
	{
		checkNoEntry();
		return nullptr;
	}

	/**
	* Creates a uniform buffer.  The contents of the uniform buffer are provided in a parameter, and are immutable.
	* CAUTION: Even though this is marked as threadsafe, it is only valid to call from the render thread or the RHI thread. Thus is need not be threadsafe on platforms that do not support or aren't using an RHIThread
	* @param Contents - A pointer to a memory block of size NumBytes that is copied into the new uniform buffer.
	* @param NumBytes - The number of bytes the uniform buffer should contain.
	* @return The new uniform buffer.
	*/
	// FlushType: Thread safe, but varies depending on the RHI
	virtual FUniformBufferRHIRef RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation) = 0;

	virtual void RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents) = 0;

	/**
	 * Transfer metadata and underlying resource from src to dest and release any resource owned by dest.
	 * @param DestBuffer - the buffer to update
	 * @param SrcBuffer - don't use after call. If null, will release any resource owned by DestBuffer
	 */
	virtual void RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos) = 0;

	[[nodiscard]] virtual FRHIBufferInitializer RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc) = 0;

	RHI_API virtual void* RHILockBuffer(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 Size, EResourceLockMode LockMode);
	RHI_API virtual void* RHILockBufferMGPU(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 GPUIndex, uint32 Offset, uint32 Size, EResourceLockMode LockMode);

	// FlushType: Flush RHI Thread
	RHI_API virtual void RHIUnlockBuffer(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer);
	RHI_API virtual void RHIUnlockBufferMGPU(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 GPUIndex);

	RHI_API virtual void RHIUpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITextureReference* TextureRef, FRHITexture* NewTexture);

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	virtual void RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) = 0;
#endif

	/**
	* Computes the total GPU memory a texture resource with the specified parameters will occupy on the current RHI platform.
	* Also returns the required alignment for the resource.
	*
	* @param Desc          - The texture descriptor (width, height, format etc)
	* @param FirstMipIndex - The index of the most detailed mip to consider in the memory size calculation.
	* @return              - The computed size and alignment of the platform texture resource.
	*/
	// FlushType: Thread safe
	virtual FRHICalcTextureSizeResult RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex) = 0;

	/**
	* Gets the minimum alignment (in bytes) required for creating a shader resource view on a buffer-backed resource.
	* @param Format - EPixelFormat texture format of the SRV.
	*/
	// FlushType: Thread safe
	RHI_API virtual uint64 RHIGetMinimumAlignmentForBufferBackedSRV(EPixelFormat Format);

	/**
	* Retrieves texture memory stats.
	* safe to call on the main thread
	*/
	// FlushType: Thread safe
	virtual void RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats) = 0;

	/**
	* Fills a texture with to visualize the texture pool memory.
	*
	* @param	TextureData		Start address
	* @param	SizeX			Number of pixels along X
	* @param	SizeY			Number of pixels along Y
	* @param	Pitch			Number of bytes between each row
	* @param	PixelSize		Number of bytes each pixel represents
	*
	* @return true if successful, false otherwise
	*/
	// FlushType: Flush Immediate
	virtual bool RHIGetTextureMemoryVisualizeData(FColor* TextureData, int32 SizeX, int32 SizeY, int32 Pitch, int32 PixelSize) = 0;

	// Create a texture initializer, used for creating a RHITexture with data.
	[[nodiscard]]
	virtual FRHITextureInitializer RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc) = 0;

	/**
	* Thread-safe function that can be used to create a texture outside of the
	* rendering thread. This function can ONLY be called if GRHISupportsAsyncTextureCreation
	* is true.  Cannot create rendertargets with this method.
	* @param SizeX - width of the texture to create
	* @param SizeY - height of the texture to create
	* @param Format - EPixelFormat texture format
	* @param NumMips - number of mips to generate or 0 for full mip pyramid
	* @param Flags - ETextureCreateFlags creation flags
	* @param InitialMipData - pointers to mip data with which to create the texture
	* @param NumInitialMips - how many mips are provided in InitialMipData
	* @param OutCompletionEvent - An event signaled on operation completion. Can return null. Operation can still be pending after function returns (e.g. initial data upload in-flight)
	* @returns a reference to a 2D texture resource
	*/
	// FlushType: Thread safe
	virtual FTextureRHIRef RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent) = 0;

		

	/** Create a texture reference. InReferencedTexture can be null. */
	RHI_API virtual FTextureReferenceRHIRef RHICreateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* InReferencedTexture);

	// SRV / UAV creation functions
	virtual FShaderResourceViewRHIRef  RHICreateShaderResourceView (class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) = 0;
	virtual FUnorderedAccessViewRHIRef RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) = 0;

	virtual FRHIResourceCollectionRef RHICreateResourceCollection(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers)
	{
		return FRHIResourceCollectionRef{};
	}

	virtual void RHIUpdateResourceCollection(FRHICommandListBase& RHICmdList, FRHIResourceCollection* InResourceCollection, uint32 InStartIndex, TConstArrayView<FRHIResourceCollectionMember> InMemberUpdates)
	{
	}

	virtual FDescriptorRangeRHIRef RHICreateDescriptorRange(FRHICommandListBase& RHICmdList, const FRHIDescriptorRangeDesc& Desc, TConstArrayView<FRHIDescriptorResource> InitialResources)
	{
		return FDescriptorRangeRHIRef{};
	}

	virtual void RHIUpdateDescriptorRange(FRHICommandListBase& RHICmdList, FRHIDescriptorRange* DescriptorRange, uint32 FirstResourceIndex, TConstArrayView<FRHIDescriptorResource> NewResources)
	{
	}

	/**
	* Computes the size in memory required by a given texture.
	*
	* @param	TextureRHI		- Texture we want to know the size of, 0 is safely ignored
	* @return					- Size in Bytes
	*/
	// FlushType: Thread safe
	virtual uint32 RHIComputeMemorySize(FRHITexture* TextureRHI) = 0;

	virtual void RHIGetMemoryStats(FRHIMemoryStats& OutStats) { /*OutStats*/ }

	virtual FRHILockTextureResult RHILockTexture  (FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) = 0;
	virtual void                  RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) = 0;

	/**
	* Updates a region of a 2D texture from system memory
	* @param Texture - the RHI texture resource to update
	* @param MipIndex - mip level index to be modified
	* @param UpdateRegion - The rectangle to copy source image data from
	* @param SourcePitch - size in bytes of each row of the source image
	* @param SourceData - source image data, starting at the upper left corner of the source rectangle (in same pixel format as texture)
	*/
	virtual void RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData) = 0;

	/**
	* Updates a region of a 2D texture from GPU memory provided by the given buffer (may not be implemented on every platform)
	* @param Texture - the RHI texture resource to update
	* @param MipIndex - mip level index to be modified
	* @param UpdateRegion - The rectangle to copy source image data from
	* @param SourcePitch - size in bytes of each row of the source image
	* @param Buffer, BufferOffset - source image data, starting at the upper left corner of the source rectangle (in same pixel format as texture)
	*/
	virtual void RHIUpdateFromBufferTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, FRHIBuffer* Buffer, uint32 BufferOffset)
	{
		checkNoEntry();
	}

	/**
	* Updates a region of a 3D texture from system memory
	* @param Texture - the RHI texture resource to update
	* @param MipIndex - mip level index to be modified
	* @param UpdateRegion - The rectangle to copy source image data from
	* @param SourceRowPitch - size in bytes of each row of the source image, usually Bpp * SizeX
	* @param SourceDepthPitch - size in bytes of each depth slice of the source image, usually Bpp * SizeX * SizeY
	* @param SourceData - source image data, starting at the upper left corner of the source rectangle (in same pixel format as texture)
	*/
	virtual void RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData) = 0;

	RHI_API virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList,FRHITexture* Texture, const TCHAR* Name);
	RHI_API virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, const TCHAR* Name);
	RHI_API virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIUnorderedAccessView* UnorderedAccessViewRHI, const TCHAR* Name);

	/**
	* Reads the contents of a texture to an output buffer (non MSAA and MSAA) and returns it as a FColor array.
	* If the format or texture type is unsupported the OutData array will be size 0
	*/
	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIReadSurfaceData(FRHITexture* Texture, FIntRect Rect, TArray<FColor>& OutData, FReadSurfaceDataFlags InFlags) = 0;

	// Default fallback; will not work for non-8-bit surfaces and it's extremely slow.
	RHI_API virtual void RHIReadSurfaceData(FRHITexture* Texture, FIntRect Rect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags);

	/** Watch out for OutData to be 0 (can happen on DXGI_ERROR_DEVICE_REMOVED), don't call RHIUnmapStagingSurface in that case. */
	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIMapStagingSurface(FRHITexture* Texture, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex = 0) = 0;

	/** call after a succesful RHIMapStagingSurface() call */
	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIUnmapStagingSurface(FRHITexture* Texture, uint32 GPUIndex = 0) = 0;

	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIReadSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, TArray<FFloat16Color>& OutData, ECubeFace CubeFace, int32 ArrayIndex, int32 MipIndex) = 0;

	// FlushType: Flush Immediate (seems wrong)
	RHI_API virtual void RHIReadSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags InFlags);

	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData) = 0;

	// FlushType: Flush Immediate (seems wrong)
	RHI_API virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags InFlags);

	// FlushType: Wait RHI Thread
	virtual FRenderQueryRHIRef RHICreateRenderQuery(ERenderQueryType QueryType) = 0;

	virtual void RHIBeginRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType) {}
	virtual void RHIEndRenderQueryBatch_TopOfPipe  (FRHICommandListBase& RHICmdList, ERenderQueryType QueryType) {}

	RHI_API virtual void RHIBeginRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery);
	RHI_API virtual void RHIEndRenderQuery_TopOfPipe  (FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery);

	// CAUTION: Even though this is marked as threadsafe, it is only valid to call from the render thread. It is need not be threadsafe on platforms that do not support or aren't using an RHIThread
	// FlushType: Thread safe, but varies by RHI
	virtual bool RHIGetRenderQueryResult(FRHIRenderQuery* RenderQuery, uint64& OutResult, bool bWait, uint32 GPUIndex = INDEX_NONE) = 0;

	// FlushType: Thread safe
	virtual uint32 RHIGetViewportNextPresentGPUIndex(FRHIViewport* Viewport)
	{
		return 0; // By default, viewport need to be rendered on GPU0.
	}

	// With RHI thread, this is the current backbuffer from the perspective of the render thread.
	// FlushType: Thread safe
	virtual FTextureRHIRef RHIGetViewportBackBuffer(FRHIViewport* Viewport) = 0;
	
	virtual FTextureRHIRef RHIGetViewportBackBufferAtLayerIndex(FRHIViewport* Viewport, uint32 DrawLayerIndex) 
	{
		return nullptr;
	}

	virtual uint32 RHIGetHTilePlatformConfig(uint32 DepthWidth, uint32 DepthHeight) const
	{
		return 0;
	}

	virtual uint32 RHIGetHTilePlatformConfig(const FRHITextureDesc& DepthDesc) const
	{
		return 0;
	}

	virtual void RHIAliasTextureResources(FTextureRHIRef& DestTexture, FTextureRHIRef& SrcTexture)
	{
		checkNoEntry();
	}

	virtual FTextureRHIRef RHICreateAliasedTexture(FTextureRHIRef& SourceTexture)
	{
		checkNoEntry();
		return nullptr;
	}

	virtual void RHIGetDisplaysInformation(FDisplayInformationArray& OutDisplayInformation)
	{
	}
	
	// Compute the hash of the state components of the PSO initializer for PSO Precaching (only hash data relevant for the RHI specific PSO)
	RHI_API virtual uint64 RHIComputeStatePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer);

	// Compute the hash of the PSO initializer for PSO Precaching (only hash data relevant for the RHI specific PSO)
	RHI_API virtual uint64 RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer);

	// Check if PSO Initializers are the same used during PSO Precaching (only compare data relevant for the RHI specific PSO)
	RHI_API virtual bool RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS);

	//
	// Acquires or releases ownership of the platform-specific rendering context for the calling thread.
	// Only required by OpenGL RHI.
	//
	virtual void RHIAcquireThreadOwnership() {}
	virtual void RHIReleaseThreadOwnership() {}

	//  must be called from the main thread.
	// FlushType: Thread safe
	virtual FViewportRHIRef RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) = 0;

	//  must be called from the main thread.
	// FlushType: Thread safe
	virtual void RHIResizeViewport(FRHIViewport* Viewport, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) = 0;

	// Enqueues the presentation of the given viewport, and advances the viewport's swapchain to the next available back buffer.
	virtual void RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* Viewport, FRHIPresentArgs const& PresentArgs) = 0;

	virtual void RHIHandleDisplayChange() {}

	//  must be called from the main thread.
	// FlushType: Thread safe
	virtual void RHITick(float DeltaTime) = 0;

	// Blocks the CPU until the GPU catches up and goes idle.
	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIBlockUntilGPUIdle() = 0;

	// Tells the RHI we're about to suspend it
	virtual void RHIBeginSuspendRendering() {};

	// Operations to suspend title rendering and yield control to the system
	// FlushType: Thread safe
	virtual void RHISuspendRendering() {};

	// FlushType: Thread safe
	virtual void RHIResumeRendering() {};

	// FlushType: Flush Immediate
	virtual bool RHIIsRenderingSuspended() { return false; };

	/**
	*	Retrieve available screen resolutions.
	*
	*	@param	Resolutions			TArray<FScreenResolutionRHI> parameter that will be filled in.
	*	@param	bIgnoreRefreshRate	If true, ignore refresh rates.
	*
	*	@return	bool				true if successfully filled the array
	*/
	// FlushType: Thread safe
	virtual bool RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate) = 0;

	/**
	*	Retrieve available screen resolutions for the display.
	*
	*	@param	Resolutions			TArray<FScreenResolutionRHI> parameter that will be filled in.
	*	@param	bIgnoreRefreshRate	If true, ignore refresh rates.
	*	@param	NativeMonitorHandle	Platform-specific handle for the monitor.
	*
	*	@return	bool				true if successfully filled the array
	*/
	// FlushType: Thread safe
	virtual bool RHIGetAvailableResolutionsForDisplay(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate, void* NativeMonitorHandle)
	{
		return RHIGetAvailableResolutions(Resolutions, bIgnoreRefreshRate);
	}

	/**
	* Returns a supported screen resolution that most closely matches input.
	* @param Width - Input: Desired resolution width in pixels. Output: A width that the platform supports.
	* @param Height - Input: Desired resolution height in pixels. Output: A height that the platform supports.
	*/
	// FlushType: Thread safe
	virtual void RHIGetSupportedResolution(uint32& Width, uint32& Height) = 0;

	/**
	* Function that is used to allocate / free space used for virtual texture mip levels.
	* Make sure you also update the visible mip levels.
	* @param Texture - the texture to update, must have been created with TexCreate_Virtual
	* @param FirstMip - the first mip that should be in memory
	*/
	// FlushType: Wait RHI Thread
	RHI_API virtual void RHIVirtualTextureSetFirstMipInMemory(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, uint32 FirstMip);

	/**
	* Function that can be used to update which is the first visible mip to the GPU.
	* @param Texture - the texture to update, must have been created with TexCreate_Virtual
	* @param FirstMip - the first mip that should be visible to the GPU
	*/
	// FlushType: Wait RHI Thread
	RHI_API virtual void RHIVirtualTextureSetFirstMipVisible(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, uint32 FirstMip);

	/**
	* Provides access to the native device. Generally this should be avoided but is useful for third party plugins.
	*/
	// FlushType: Flush RHI Thread
	virtual void* RHIGetNativeDevice() = 0;

	/**
	* Provides access to the native device. Generally this should be avoided but is useful for third party plugins.
	*/
	// FlushType: Flush RHI Thread
	virtual void* RHIGetNativePhysicalDevice() 
	{
		// Currently only exists on Vulkan, so no need to force every backend to implement this.
		return nullptr;
	}

	/**
	* Provides access to the native graphics command queue. Generally this should be avoided but is useful for third party plugins.
	*/
	// FlushType: Flush RHI Thread
	virtual void* RHIGetNativeGraphicsQueue() 
	{
		return nullptr;
	}

	/**
	* Provides access to the native compute command queue. Generally this should be avoided but is useful for third party plugins.
	*/
	// FlushType: Flush RHI Thread
	virtual void* RHIGetNativeComputeQueue() 
	{
		return nullptr;
	}


	// FlushType: Thread safe
	virtual IRHICommandContext* RHIGetDefaultContext() = 0;

	//
	// Retrieves a new command context to begin the recording of a new platform command list.
	// The returned context is specific to the given pipeline. It can later be converted to an IRHIPlatformCommandList
	// by calling RHIFinalizeContexts(), and then submitted to the GPU by calling RHISubmitCommandLists().
	//
	// Called by parallel worker threads, and the render thread. Platform implementations must be thread safe.
	//
	virtual IRHIComputeContext* RHIGetCommandContext(ERHIPipeline Pipeline) = 0;
	
	//
	// Retrieves a new command context to begin the recording of a new platform command list.
	// The returned context is only for graphics pipelines and is used in conjunction with SubCommandList
	// The returned context is expected to be in a state to execute draw commands immediately, i.e. renderpasses must be created and state set.
	//
	// Called by parallel worker threads, and the render thread. Platform implementations must be thread safe.
	//
	virtual IRHICommandContext* RHIGetParallelCommandContext(FRHIParallelRenderPassInfo const& ParallelRenderPass)
	{
		checkNoEntry();
		return nullptr;
	}
	
	virtual IRHIUploadContext* RHIGetUploadContext()
	{
		return nullptr;
	};
	
	//
	// Finalizes (i.e. closes) the specified command context, returning the completed platform command list object.
	// The returned command list can later be submitted to the GPU by calling RHISubmitCommandLists().
	//
	// The context may be destroyed or recycled, so should not be used again. Call RHIGetCommandContext() to get a new context.
	//
	// Called by parallel worker threads, and the RHI thread. Platform implementations must be thread safe.
	//
	struct FRHIFinalizeContextsArgs
	{
		TRHIPipelineArray<IRHIComputeContext*> Contexts { InPlace, nullptr };
		IRHIUploadContext* UploadContext = nullptr;
	};
	
	struct FRHIFinalizeContextsResult
	{
		TRHIPipelineArray<IRHIPlatformCommandList*> CommandLists { InPlace, nullptr };
		IRHIPlatformCommandList* UploadCommandList = nullptr;
	};

	virtual FRHIFinalizeContextsResult RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args) = 0;

	//
	// Close the current translate chain, finalizing the specified contexts.
	// Provided as a hook for the RHI validation layer to handle addition logic for parallel render passes.
	//
	struct FRHICloseTranslateChainArgs : FRHIFinalizeContextsArgs
	{
		bool bShouldFinalize = true;
	};

	using FRHICloseTranslateChainResult = FRHIFinalizeContextsResult;

#if ENABLE_RHI_VALIDATION
	virtual
#endif
	FRHICloseTranslateChainResult RHICloseTranslateChain(FRHICloseTranslateChainArgs&& Args)
	{
		return Args.bShouldFinalize
			? RHIFinalizeContexts(MoveTemp(Args))
			: FRHICloseTranslateChainResult();
	}

	//
	// Submits a batch of previously recorded/finalized command lists to the GPU. 
	// Command lists are well-ordered in the array view. Platform implementations must submit in this order for correct rendering.
	//
	// Called by the RHI thread. 
	//
	struct FRHISubmitCommandListsArgs
	{
		TArray<IRHIPlatformCommandList*> CommandLists;
	};
	virtual void RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args) = 0;

	//
	// Platform RHIs should implement this function to process their internal GPU resource/memory delete queues.
	// Called only from RHI command list management code. Do not call directly.
	//
	virtual void RHIProcessDeleteQueue() {}

	RHI_API virtual FUpdateTexture3DData RHIBeginUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion);
	RHI_API virtual void RHIEndUpdateTexture3D(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData);

	RHI_API virtual void RHIEndMultiUpdateTexture3D(FRHICommandListBase& RHICmdList, TArray<FUpdateTexture3DData>& UpdateDataArray);

	RHI_API virtual void RHIMapStagingSurface_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, uint32 GPUIndex, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight);
	RHI_API virtual void RHIUnmapStagingSurface_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, uint32 GPUIndex);
	RHI_API virtual void RHIReadSurfaceFloatData_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, FIntRect Rect, TArray<FFloat16Color>& OutData, ECubeFace CubeFace, int32 ArrayIndex, int32 MipIndex);
	RHI_API virtual void RHIReadSurfaceFloatData_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, FIntRect Rect, TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags Flags);

	// Buffer Lock/Unlock
	virtual void* LockBuffer_BottomOfPipe(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode)
	{
		// Either this function or RHILockBuffer must be implemented by the platform RHI.
		checkNoEntry();
		return nullptr;
	}

	virtual void UnlockBuffer_BottomOfPipe(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer)
	{
		// Either this function or RHIUnlockBuffer must be implemented by the platform RHI.
		checkNoEntry();
	}

	//Utilities
	static RHI_API void EnableIdealGPUCaptureOptions(bool bEnable);

	virtual FRHIFlipDetails RHIWaitForFlip(double TimeoutInSeconds) { return FRHIFlipDetails(); }
	virtual void RHISignalFlipEvent() { }


	virtual uint16 RHIGetPlatformTextureMaxSampleCount() { return 8; };

	virtual FRayTracingAccelerationStructureSize RHICalcRayTracingSceneSize(const FRayTracingSceneInitializer& Initializer)
	{
		checkNoEntry();
		return {};
	}

	virtual FRayTracingAccelerationStructureSize RHICalcRayTracingGeometrySize(const FRayTracingGeometryInitializer& Initializer)
	{
		checkNoEntry();
		return {};
	}

	virtual FRayTracingClusterOperationSize RHICalcRayTracingClusterOperationSize(const FRayTracingClusterOperationInitializer& Initializer)
	{
		checkNoEntry();
		return {};
	}

	virtual FRayTracingAccelerationStructureOfflineMetadata RHIGetRayTracingGeometryOfflineMetadata(const FRayTracingGeometryOfflineDataHeader& OfflineDataHeader)
	{
		checkNoEntry();
		return {};
	}

	virtual FRayTracingGeometryRHIRef RHICreateRayTracingGeometry(FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& Initializer)
	{
		checkNoEntry();
		return nullptr;
	}
	
	virtual FRayTracingSceneRHIRef RHICreateRayTracingScene(FRayTracingSceneInitializer Initializer)
	{
		checkNoEntry();
		return nullptr;
	}

	virtual FRayTracingShaderRHIRef RHICreateRayTracingShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency)
	{
		checkNoEntry();
		return nullptr;
	}

	virtual FRayTracingPipelineStateRHIRef RHICreateRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer)
	{
		checkNoEntry();
		return nullptr;
	}

	virtual FShaderBindingTableRHIRef RHICreateShaderBindingTable(FRHICommandListBase& RHICmdList, const FRayTracingShaderBindingTableInitializer& Initializer)
	{
		checkNoEntry();
		return nullptr;
	}

#if !UE_BUILD_SHIPPING
	virtual void RHISerializeAccelerationStructure(FRHICommandListImmediate& RHICmdList, FRHIRayTracingScene* Scene, const TCHAR* Path)
	{
		checkNoEntry();
	}
#endif

	virtual FShaderBundleRHIRef RHICreateShaderBundle(const FShaderBundleCreateInfo& CreateInfo)
	{
		checkNoEntry();
		return nullptr;
	}

	virtual void RHIWriteResourceGPUAddresses(FRHICommandListBase& RHICmdList, uint32 GPUIndex, TConstArrayView<FRHIResource*> Resources, uint64* AddressesPtr)
	{
		checkNoEntry();
	}

protected:
	TArray<uint32> PixelFormatBlockBytes;
	friend class FValidationRHI;
};

/** A global pointer to the dynamically bound RHI implementation. */
extern RHI_API FDynamicRHI* GDynamicRHI;

// Dynamic RHI for RHIs that do not support real Graphics/Compute Pipelines.
class FDynamicRHIPSOFallback : public FDynamicRHI
{
public:
	virtual FGraphicsPipelineStateRHIRef RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer) override
	{
		return new FRHIGraphicsPipelineStateFallBack(Initializer);
	}

	virtual FComputePipelineStateRHIRef RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer) override
	{
		return new FRHIComputePipelineStateFallback(Initializer.ComputeShader);
	}
};

inline ERHIInterfaceType RHIGetInterfaceType()
{
	return GDynamicRHI->GetInterfaceType();
}

template<typename TRHI>
inline TRHI* CastDynamicRHI(FDynamicRHI* InDynamicRHI)
{
	return static_cast<TRHI*>(InDynamicRHI->GetNonValidationRHI());
}

template<typename TRHI>
inline TRHI* GetDynamicRHI()
{
	return CastDynamicRHI<TRHI>(GDynamicRHI);
}

inline FPixelShaderRHIRef RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreatePixelShader(CreateShaderDesc);
}

inline FVertexShaderRHIRef RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateVertexShader(CreateShaderDesc);
}

inline FMeshShaderRHIRef RHICreateMeshShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateMeshShader(CreateShaderDesc);
}

inline FAmplificationShaderRHIRef RHICreateAmplificationShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateAmplificationShader(CreateShaderDesc);
}

inline FGeometryShaderRHIRef RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateGeometryShader(CreateShaderDesc);
}

inline FComputeShaderRHIRef RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateComputeShader(CreateShaderDesc);
}

inline FWorkGraphShaderRHIRef RHICreateWorkGraphShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateWorkGraphShader(CreateShaderDesc, ShaderFrequency);
}

inline FGPUFenceRHIRef RHICreateGPUFence(const FName& Name)
{
	return GDynamicRHI->RHICreateGPUFence(Name);
}

inline FStagingBufferRHIRef RHICreateStagingBuffer()
{
	return GDynamicRHI->RHICreateStagingBuffer();
}

inline FSamplerStateRHIRef RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateSamplerState"));
	return GDynamicRHI->RHICreateSamplerState(Initializer);
}

inline FRasterizerStateRHIRef RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateRasterizerState"));
	return GDynamicRHI->RHICreateRasterizerState(Initializer);
}

inline FDepthStencilStateRHIRef RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateDepthStencilState"));
	return GDynamicRHI->RHICreateDepthStencilState(Initializer);
}

inline FBlendStateRHIRef RHICreateBlendState(const FBlendStateInitializerRHI& Initializer)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateBlendState"));
	return GDynamicRHI->RHICreateBlendState(Initializer);
}

inline FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIVertexDeclaration* VertexDeclaration, FRHIVertexShader* VertexShader, FRHIPixelShader* PixelShader, FRHIGeometryShader* GeometryShader)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateBoundShaderState(VertexDeclaration, VertexShader, PixelShader, GeometryShader);
}

#if PLATFORM_SUPPORTS_MESH_SHADERS && PLATFORM_USE_FALLBACK_PSO
inline FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIAmplificationShader* AmplificationShader, FRHIMeshShader* MeshShader, FRHIPixelShader* PixelShader)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateBoundShaderState(AmplificationShader, MeshShader, PixelShader);
}
#endif

/** Before using this directly go through PipelineStateCache::GetAndOrCreateGraphicsPipelineState() */
inline FGraphicsPipelineStateRHIRef RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateGraphicsPipelineState(Initializer);
}

/** Before using this directly go through PipelineStateCache::GetOrCreateVertexDeclaration() */
inline FVertexDeclarationRHIRef RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements)
{
	return GDynamicRHI->RHICreateVertexDeclaration(Elements);
}

inline FComputePipelineStateRHIRef RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateComputePipelineState(Initializer);
}

inline TRefCountPtr<FRHIWorkGraphPipelineState> RHICreateWorkGraphPipelineState(const FWorkGraphPipelineStateInitializer& Initializer)
{
	LLM_SCOPE(ELLMTag::Shaders);
	return GDynamicRHI->RHICreateWorkGraphPipelineState(Initializer);
}

inline FRenderQueryRHIRef RHICreateRenderQuery(ERenderQueryType QueryType)
{
	return GDynamicRHI->RHICreateRenderQuery(QueryType);
}

inline TRefCountPtr<FRHIRayTracingPipelineState> RHICreateRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer)
{
	return GDynamicRHI->RHICreateRayTracingPipelineState(Initializer);
}

inline FUniformBufferLayoutRHIRef RHICreateUniformBufferLayout(const FRHIUniformBufferLayoutInitializer& Initializer)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateUniformBufferLayout"));
	return new FRHIUniformBufferLayout(Initializer);
}

inline FUniformBufferRHIRef RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation = EUniformBufferValidation::ValidateResources)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateUniformBuffer"));
	FUniformBufferRHIRef Result = GDynamicRHI->RHICreateUniformBuffer(Contents, Layout, Usage, Validation);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE::RHI::UniformBufferContentMap::Update(Result.GetReference(), Contents);
#endif
	return Result;
}

inline FRHICalcTextureSizeResult RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex = 0)
{
	if ( ! Desc.IsValid() )
	{
		// Invalid texture desc; return zero to indicate failure
		FRHICalcTextureSizeResult ZeroResult{};
		return ZeroResult;
	}

	return GDynamicRHI->RHICalcTexturePlatformSize(Desc, FirstMipIndex);
}

inline uint64 RHIGetMinimumAlignmentForBufferBackedSRV(EPixelFormat Format)
{
	return GDynamicRHI->RHIGetMinimumAlignmentForBufferBackedSRV(Format);
}

inline void RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats)
{
	GDynamicRHI->RHIGetTextureMemoryStats(OutStats);
}

inline uint32 RHIComputeMemorySize(FRHITexture* TextureRHI)
{
	return GDynamicRHI->RHIComputeMemorySize(TextureRHI);
}

inline void RHIGetMemoryStats(FRHIMemoryStats& OutStats)
{
	GDynamicRHI->RHIGetMemoryStats(OutStats);
}

inline bool RHIGetRenderQueryResult(FRHIRenderQuery* RenderQuery, uint64& OutResult, bool bWait, uint32 GPUIndex = INDEX_NONE)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(RHIGetRenderQueryResult_GPU_Wait, bWait);
	return GDynamicRHI->RHIGetRenderQueryResult(RenderQuery, OutResult, bWait, GPUIndex);
}

inline uint32 RHIGetViewportNextPresentGPUIndex(FRHIViewport* Viewport)
{
	return GDynamicRHI->RHIGetViewportNextPresentGPUIndex(Viewport);
}

inline FTextureRHIRef RHIGetViewportBackBuffer(FRHIViewport* Viewport)
{
	return GDynamicRHI->RHIGetViewportBackBuffer(Viewport);
}

inline FTextureRHIRef RHIGetViewportBackBufferAtLayerIndex(FRHIViewport* Viewport, uint32 DrawLayerIndex)
{
	return GDynamicRHI->RHIGetViewportBackBufferAtLayerIndex(Viewport, DrawLayerIndex);
}

UE_DEPRECATED(5.8, "RHIGetViewportBackBufferUAV has been deprecated and always returns null. Use the standard UAV creation functions instead. The platform must support back buffer UAVs, which is indicated by the TexCreate_UAV flag in the back buffer texture descriptor.")
inline FUnorderedAccessViewRHIRef RHIGetViewportBackBufferUAV(FRHIViewport* Viewport)
{
	return nullptr;
}

UE_DEPRECATED(5.7, "RHIGetHTilePlatformConfig(uint32,uint32) has been deprecated. Please use RHIGetHTilePlatformConfig(const FRHITextureDesc&) instead.")
inline uint32 RHIGetHTilePlatformConfig(uint32 DepthWidth, uint32 DepthHeight)
{
	return GDynamicRHI->RHIGetHTilePlatformConfig(DepthWidth, DepthHeight);
}

inline uint32 RHIGetHTilePlatformConfig(const FRHITextureDesc& DepthDesc)
{
	return GDynamicRHI->RHIGetHTilePlatformConfig(DepthDesc);
}

UE_DEPRECATED(5.7, "RHIAdvanceFrameForGetViewportBackBuffer is deprecated. This happens automatically when RHIEndDrawingViewport is called. Remove calls to this function.")
inline void RHIAdvanceFrameForGetViewportBackBuffer(FRHIViewport* Viewport)
{
}

RHI_API uint32 RHIGetGPUFrameCycles(uint32 GPUIndex = 0);

inline FViewportRHIRef RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
	LLM_SCOPE(ELLMTag::RenderTargets);
	return GDynamicRHI->RHICreateViewport(WindowHandle, SizeX, SizeY, bIsFullscreen, PixelFormat);
}

inline void RHIResizeViewport(FRHIViewport* Viewport, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
	LLM_SCOPE(ELLMTag::RenderTargets);
	GDynamicRHI->RHIResizeViewport(Viewport, SizeX, SizeY, bIsFullscreen, PixelFormat);
}

// UE_DEPRECATED(5.5, "This method is no longer used.")
inline EColorSpaceAndEOTF RHIGetColorSpace(FRHIViewport* Viewport)
{
	return EColorSpaceAndEOTF::ERec709_sRGB;
}

UE_DEPRECATED(5.8, "RHICheckViewportHDRStatus is deprecated and unnecessary. Remove any remaining calls to this function.")
inline void RHICheckViewportHDRStatus(FRHIViewport* Viewport)
{
}

inline void RHIHandleDisplayChange()
{
	GDynamicRHI->RHIHandleDisplayChange();
}

inline void RHITick(float DeltaTime)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/RHITick"));
	GDynamicRHI->RHITick(DeltaTime);
}

inline void RHIBeginSuspendRendering()
{
	GDynamicRHI->RHIBeginSuspendRendering();
}

inline void RHISuspendRendering()
{
	GDynamicRHI->RHISuspendRendering();
}

inline void RHIResumeRendering()
{
	GDynamicRHI->RHIResumeRendering();
}

inline bool RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate)
{
	return GDynamicRHI->RHIGetAvailableResolutions(Resolutions, bIgnoreRefreshRate);
}

inline bool RHIGetAvailableResolutionsForDisplay(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate, void* NativeMonitorHandle)
{
	return GDynamicRHI->RHIGetAvailableResolutionsForDisplay(Resolutions, bIgnoreRefreshRate, NativeMonitorHandle);
}

inline void RHIGetSupportedResolution(uint32& Width, uint32& Height)
{
	GDynamicRHI->RHIGetSupportedResolution(Width, Height);
}

UE_DEPRECATED(5.8, "RHIGetDefaultContext is deprecated. Do not call this function. RHI contexts are a private implementation detail of the platform RHI API, and should not be retrieved by high-level code directly via this function.")
inline class IRHICommandContext* RHIGetDefaultContext()
{
	return GDynamicRHI->RHIGetDefaultContext();
}

RHI_API FRenderQueryPoolRHIRef RHICreateRenderQueryPool(ERenderQueryType QueryType, uint32 NumQueries = UINT32_MAX);

RHI_API FRHITransition* RHICreateTransition(const FRHITransitionCreateInfo& CreateInfo);

inline void RHIReleaseTransition(FRHITransition* Transition)
{
	GDynamicRHI->RHIReleaseTransition(Transition);
}

inline IRHITransientResourceAllocator* RHICreateTransientResourceAllocator()
{
	return GDynamicRHI->RHICreateTransientResourceAllocator();
}

inline void RHIGetDisplaysInformation(FDisplayInformationArray& OutDisplayInformation)
{
	GDynamicRHI->RHIGetDisplaysInformation(OutDisplayInformation);
}

inline uint64 RHIComputeStatePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	return GDynamicRHI->RHIComputeStatePrecachePSOHash(Initializer);
}

inline uint64 RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	return GDynamicRHI->RHIComputePrecachePSOHash(Initializer);
}

inline bool RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS)
{
	return GDynamicRHI->RHIMatchPrecachePSOInitializers(LHS, RHS);
}

inline FRayTracingAccelerationStructureSize RHICalcRayTracingSceneSize(const FRayTracingSceneInitializer& Initializer)
{
	return GDynamicRHI->RHICalcRayTracingSceneSize(Initializer);
}

inline FRayTracingSceneRHIRef RHICreateRayTracingScene(FRayTracingSceneInitializer Initializer)
{
	return GDynamicRHI->RHICreateRayTracingScene(MoveTemp(Initializer));
}

inline FRayTracingShaderRHIRef RHICreateRayTracingShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency)
{
	return GDynamicRHI->RHICreateRayTracingShader(CreateShaderDesc, ShaderFrequency);
}

inline FPixelShaderRHIRef RHICreatePixelShader(TArrayView<const uint8> Code, const FShaderHash& Hash)
{
	return RHICreatePixelShader(FRHICreateShaderDesc(Code));
}

inline FVertexShaderRHIRef RHICreateVertexShader(TArrayView<const uint8> Code, const FShaderHash& Hash)
{
	return RHICreateVertexShader(FRHICreateShaderDesc(Code));
}

inline FMeshShaderRHIRef RHICreateMeshShader(TArrayView<const uint8> Code, const FShaderHash& Hash)
{
	return RHICreateMeshShader(FRHICreateShaderDesc(Code));
}

inline FAmplificationShaderRHIRef RHICreateAmplificationShader(TArrayView<const uint8> Code, const FShaderHash& Hash)
{
	return RHICreateAmplificationShader(FRHICreateShaderDesc(Code));
}

inline FGeometryShaderRHIRef RHICreateGeometryShader(TArrayView<const uint8> Code, const FShaderHash& Hash)
{
	return RHICreateGeometryShader(FRHICreateShaderDesc(Code));
}

inline FComputeShaderRHIRef RHICreateComputeShader(TArrayView<const uint8> Code, const FShaderHash& Hash)
{
	return RHICreateComputeShader(FRHICreateShaderDesc(Code));
}

inline FWorkGraphShaderRHIRef RHICreateWorkGraphShader(TArrayView<const uint8> Code, const FShaderHash& Hash, EShaderFrequency ShaderFrequency)
{
	return RHICreateWorkGraphShader(FRHICreateShaderDesc(Code), ShaderFrequency);
}

inline FRayTracingShaderRHIRef RHICreateRayTracingShader(TArrayView<const uint8> Code, const FShaderHash& Hash, EShaderFrequency ShaderFrequency)
{
	return RHICreateRayTracingShader(FRHICreateShaderDesc(Code), ShaderFrequency);
}

inline FShaderBundleRHIRef RHICreateShaderBundle(const FShaderBundleCreateInfo& CreateInfo)
{
	return GDynamicRHI->RHICreateShaderBundle(CreateInfo);
}

/**
* Defragment the texture pool.
*/
inline void appDefragmentTexturePool() {}

/**
* Checks if the texture data is allocated within the texture pool or not.
*/
inline bool appIsPoolTexture(FRHITexture* TextureRHI) { return false; }

/**
* Log the current texture memory stats.
*
* @param Message	This text will be included in the log
*/
inline void appDumpTextureMemoryStats(const TCHAR* /*Message*/) {}

/** Structs that describe how intensively a GPU is being used. */
struct FRHIGPUUsageFractions
{
	// Fraction on how much much the GPU clocks has been scaled down by driver for energy savings.
	float ClockScaling = 1.0f;

	// Fraction of GPU resource dedicated for our own process at current clock scaling.
	float CurrentProcessMHz = 0.0f;

	// Fraction of GPU resource dedicated for other processes at current clock scaling.
	float ExternalProcessesMHz = 0.0f;

	// Amount of VRAM used by the process in bytes.
	uint64_t CurrentProcessMemoryUsage = 0;

	// Amount of VRAM used by other processes in bytes.
	uint64_t ExternalProcessMemoryUsage = 0;

	// Remaining fraction of GPU resource that is idle.
	inline float GetUnused()
	{
		return FMath::Clamp(1.0f - CurrentProcessMHz - ExternalProcessesMHz, 0.0f, 1.0f);
	}
};

/** Get how much the GPU is getting used.
*
* Requires GRHISupportsGPUUsage=true before use.
*/
typedef FRHIGPUUsageFractions (*RHIGetGPUUsageType)(uint32);
extern RHI_API RHIGetGPUUsageType RHIGetGPUUsage;

/** Defines the interface of a module implementing a dynamic RHI. */
class IDynamicRHIModule : public IModuleInterface
{
public:

	/** Checks whether the RHI is supported by the current system. */
	virtual bool IsSupported() = 0;

	virtual bool IsSupported(ERHIFeatureLevel::Type RequestedFeatureLevel) { return IsSupported(); }

	/** Creates a new instance of the dynamic RHI implemented by the module. */
	virtual FDynamicRHI* CreateRHI(ERHIFeatureLevel::Type RequestedFeatureLevel = ERHIFeatureLevel::Num) = 0;
};

/**
*	Each platform that utilizes dynamic RHIs should implement this function
*	Called to create the instance of the dynamic RHI.
*/
FDynamicRHI* PlatformCreateDynamicRHI();

// Name of the RHI module that will be created when PlatformCreateDynamicRHI is called
// NOTE: This function is very slow when called before RHIInit
extern RHI_API const TCHAR* GetSelectedDynamicRHIModuleName(bool bCleanup = true);

extern RHI_API bool GDynamicRHIFailedToInitializeAdvancedPlatform;
extern RHI_API bool GDynamicRHIIgnoredSM5Preference;
extern RHI_API FString GDynamicRHIIgnoredPreferredRHI;

//
// Helper for acquiring and releasing thread ownership of the RHI within a scope.
// For private use by the RHI and render thread management code only.
//
struct FScopedRHIThreadOwnership
{
	bool const bCondition;

	FScopedRHIThreadOwnership(bool bCondition)
		: bCondition(bCondition)
	{
		if (bCondition)
		{
			SCOPED_NAMED_EVENT(RHIAcquireThreadOwnership, FColor::Red);
			GDynamicRHI->RHIAcquireThreadOwnership();
		}
	}

	~FScopedRHIThreadOwnership()
	{
		if (bCondition)
		{
			SCOPED_NAMED_EVENT(RHIReleaseThreadOwnership, FColor::Red);
			GDynamicRHI->RHIReleaseThreadOwnership();
		}
	}
};
