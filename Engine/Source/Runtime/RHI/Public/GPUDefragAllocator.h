// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "Stats/Stats.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/IConsoleManager.h"
#include "Containers/List.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/MemoryTrace.h"

#define LOG_EVERY_ALLOCATION			0
#define DUMP_ALLOC_FREQUENCY			0 // 100

#define VALIDATE_SYNC_SIZE				0 //validates that the CPU is returning memory in an allocate that's part of an active GPU move.
#define VALIDATE_MOVES					0 //validates that GPU moves in a given frame do not overlap destination areas.
#define TRACK_RELOCATIONS (VALIDATE_SYNC_SIZE || VALIDATE_MOVES)
#define VALIDATE_MEMORY_PROTECTION		0

#define USE_ALLOCATORFIXEDSIZEFREELIST	0

class FAsyncReallocationRequest;
class FScopedGPUDefragLock;

/*-----------------------------------------------------------------------------
Custom fixed size pool best fit texture memory allocator
-----------------------------------------------------------------------------*/

// Forward declaration
class FAsyncReallocationRequest;

class FScopedGPUDefragLock;
/**
* Simple best fit allocator, splitting and coalescing whenever/ wherever possible.
* NOT THREAD-SAFE.
*
* - uses TMap to find memory chunk given a Pointer (potentially colliding with malloc/ free from main thread)
* - uses separate linked list for free allocations, assuming that relatively few free chunks due to coalescing
*/
class FGPUDefragAllocator
{
public:

	typedef TDoubleLinkedList<FAsyncReallocationRequest*> FRequestList;
	typedef TDoubleLinkedList<FAsyncReallocationRequest*>::TDoubleLinkedListNode FRequestNode;

	/**
	* Container for allocator settings.
	*/
	struct FSettings
	{
		FSettings()
		: MaxDefragRelocations(128 * 1024)
		, MaxDefragDownShift(32 * 1024)
		, OverlappedBandwidthScale(1)
		{
		}

		/** Maximum number of uint8s to relocate, in total, during a partial defrag. */
		int32		MaxDefragRelocations;
		/** Maximum number of uint8s to relocate during a partial defrag by brute-force downshifting. */
		int32		MaxDefragDownShift;
		/** Amount of extra bandwidth used when doing overlapped relocations */
		int32		OverlappedBandwidthScale;
	};

	enum EMemoryElementType
	{
		MET_Allocated,
		MET_Free,
		MET_Locked,
		MET_Relocating,
		MET_Resizing,
		MET_Resized,
		MET_Max
	};

	struct FMemoryLayoutElement
	{
		FMemoryLayoutElement()
		: Size(0)
		, Type(MET_Allocated)
		{
		}
		FMemoryLayoutElement(int32 InSize, EMemoryElementType InType)
			: Size(InSize)
			, Type(InType)
		{
		}
		int32				Size;
		EMemoryElementType	Type;

		friend FArchive& operator<<(FArchive& Ar, FMemoryLayoutElement& Element)
		{
			Ar << Element.Size;
			uint32 ElementType = Element.Type;
			Ar << ElementType;
			Element.Type = EMemoryElementType(ElementType);
			return Ar;
		}
	};

	/**
	* Container for allocator relocation stats.
	*/
	struct FRelocationStats
	{
		FRelocationStats()
		: NumBytesRelocated(0)
		, NumBytesDownShifted(0)
		, LargestHoleSize(0)
		, NumRelocations(0)		
		, NumHoles(0)
		, NumLockedChunks(0)
		{
		}

		/** Number of uint8s relocated, in total. */
		int64 NumBytesRelocated;

		/** Number of uint8s relocated by brute-force downshifting. */
		int64 NumBytesDownShifted;

		/** Size of the largest free consecutive memory region, before any relocations were made. */
		int64 LargestHoleSize;

		/** Number of relocations initiated. */
		int32 NumRelocations;		

		/** Number of disjoint32 free memory regions, before any relocations were made. */
		int32 NumHoles;

		/** Number of chunks that are locked and cannot be relocated */
		int32 NumLockedChunks;
	};

	/**
	* Contains information of a single allocation or free block.
	*/
	class FMemoryChunk
	{
	public:
		/**
		* Private constructor.
		*
		* @param	InBase					Pointer to base of chunk
		* @param	InSize					Size of chunk
		* @param	ChunkToInsertAfter		Chunk to insert this after.
		* @param	FirstFreeChunk			Reference to first free chunk Pointer.
		*/
		FMemoryChunk(uint8* InBase, int64 InSize, FGPUDefragAllocator& InBestFitAllocator, FMemoryChunk*& ChunkToInsertAfter, TStatId InStat)
			: Base(InBase)
			, Size(InSize)
			, OrigSize(0)
			, bIsAvailable(false)
			, LockCount(0)
			, DefragCounter(0)
			, BestFitAllocator(InBestFitAllocator)
			, SyncIndex(0)
			, SyncSize(0)
			, UserPayload(0)			
			, Stat(InStat)
			, bTail(false)
		{
			Link(ChunkToInsertAfter);
			// This is going to change bIsAvailable.
			LinkFree(ChunkToInsertAfter);
		}

		/**
		* Unlinks/ removes the chunk from the linked lists it belongs to.
		*/
		~FMemoryChunk()
		{
			// Remove from linked lists.
			Unlink();
			UnlinkFree();
			//todo: add check for numlocks == 0
		}

		bool IsLocked() const
		{			
			return (LockCount != 0);
		}

		/**
		* Inserts this chunk after the passed in one.
		*
		* @param	ChunkToInsertAfter	Chunk to insert after
		*/
		void	Link(FMemoryChunk*& ChunkToInsertAfter)
		{
			if (ChunkToInsertAfter)
			{
				NextChunk = ChunkToInsertAfter->NextChunk;
				PreviousChunk = ChunkToInsertAfter;
				ChunkToInsertAfter->NextChunk = this;
				if (NextChunk)
				{
					NextChunk->PreviousChunk = this;
				}
				else
				{
					BestFitAllocator.LastChunk = this;
				}
			}
			else
			{
				PreviousChunk = nullptr;
				NextChunk = nullptr;
				ChunkToInsertAfter = this;
				BestFitAllocator.LastChunk = this;
			}
		}

		/**
		* Inserts this chunk at the head of the free chunk list.
		*/
		void	LinkFree(FMemoryChunk* FirstFreeChunkToSearch);

		/**
		* Removes itself for linked list.
		*/
		void	Unlink()
		{
			if (PreviousChunk)
			{
				PreviousChunk->NextChunk = NextChunk;
			}
			else
			{
				BestFitAllocator.FirstChunk = NextChunk;
			}

			if (NextChunk)
			{
				NextChunk->PreviousChunk = PreviousChunk;
			}
			else
			{
				BestFitAllocator.LastChunk = PreviousChunk;
			}

			PreviousChunk = nullptr;
			NextChunk = nullptr;
		}

		/**
		* Removes itself for linked "free" list. Maint32ains the free-list order.
		*/
		void	UnlinkFree()
		{
			check(bIsAvailable);
			bIsAvailable = false;

			if (PreviousFreeChunk)
			{
				PreviousFreeChunk->NextFreeChunk = NextFreeChunk;
			}
			else
			{
				BestFitAllocator.FirstFreeChunk = NextFreeChunk;
			}

			if (NextFreeChunk)
			{
				NextFreeChunk->PreviousFreeChunk = PreviousFreeChunk;
			}

			PreviousFreeChunk = nullptr;
			NextFreeChunk = nullptr;
		}		

		/**
		*	Returns true if the Chunk is being asynchronously relocated due to reallocation or defrag.
		*/
		bool	IsRelocating() const
		{
			return SyncIndex > BestFitAllocator.CompletedSyncIndex;
		}

		/**
		* Returns the number of uint8s that can be allocated from this chunk.
		*/
		int64		GetAvailableSize() const
		{
			if (bIsAvailable)
			{
				return IsRelocating() ? (Size - SyncSize) : Size;
			}
			else
			{
				return 0;
			}
		}

		/**
		*	Returns the current size (in uint8s), or the final size if it has a reallocating request.
		*/
		int64		GetFinalSize() const;

		/**
		* Sets the relocation sync index.
		* @param InSyncIndex	GPU synchronization identifier that can be compared with BestFitAllocator::CompletedSyncIndex
		* @param InSyncSize	Number of uint8s that require GPU synchronization (starting from the beginning of the chunk)
		*/
		void	SetSyncIndex(uint32 InSyncIndex, int64 InSyncSize)
		{			
			SyncIndex = InSyncIndex;
			SyncSize = InSyncSize;
		}

		/**
		*	Returns the relocation sync index.
		*/
		uint32	GetSyncIndex() const
		{
			return SyncIndex;
		}

		/**
		*	Comparison function for Sort(), etc, based on increasing base address.
		*/
		static uint64 Compare(const FMemoryChunk* A, const FMemoryChunk* B)
		{
			return B->Base - A->Base;
		}

#if USE_ALLOCATORFIXEDSIZEFREELIST
		/** Custom new/delete */
		void* operator new(size_t Size);
		void operator delete(void *RawMemory);
#endif

		/** Base of chunk.								*/
		uint8*					Base;
		/** Size of chunk.								*/
		int64					Size;
		int64					OrigSize;
		/** Whether the chunk is available.				*/
		bool					bIsAvailable;
		/** Whether the chunk has been locked.			*/		
		int32					LockCount;
		/** Defrag counter. If this chunk failed to defrag, it won't try it again until the counter is 0. */
		uint16					DefragCounter;

		/** Allows access to FBestFitAllocator members such as FirstChunk, FirstFreeChunk and LastChunk. */
		FGPUDefragAllocator&		BestFitAllocator;
		/** Pointer to previous chunk.					*/
		FMemoryChunk*			PreviousChunk;
		/** Pointer to next chunk.						*/
		FMemoryChunk*			NextChunk;

		/** Pointer to previous free chunk.				*/
		FMemoryChunk*			PreviousFreeChunk;
		/** Pointer to next free chunk.					*/
		FMemoryChunk*			NextFreeChunk;
		/** SyncIndex that must be exceeded before accessing the data within this chunk. */
		uint32					SyncIndex;

		/** Number of uint8s covered by the SyncIndex (starting from the beginning of the chunk). */
		int64					SyncSize;
		/** User payload, e.g. platform-specific texture Pointer. Only chunks with payload can be relocated. */
		void*					UserPayload;		

		//stat associated with this allocation
		TStatId Stat;
		bool bTail;
	};

	/** Constructor, zero initializing all member variables */
	FGPUDefragAllocator()
		: MemorySize(0)
		, MemoryBase(nullptr)
		, AllocationAlignment(0)
		, FirstChunk(nullptr)
		, LastChunk(nullptr)
		, FirstFreeChunk(nullptr)
		, TimeSpentInAllocator(0.0)
		, PaddingWasteSize(0)
		, AllocatedMemorySize(0)
		, AvailableMemorySize(0)
		, PendingMemoryAdjustment(0)
		, CurrentSyncIndex(1)
		, CompletedSyncIndex(0)
		, NumRelocationsInProgress(0)
		, PlatformSyncFence(0)
		, CurrentLargestHole(0)
		, CurrentNumHoles(0)
		, TotalNumRelocations(0)
		, TotalNumBytesRelocated(0)		
		, MinLargestHole(MAX_int64)
		, MaxNumHoles(0)
		, NumFinishedAsyncReallocations(0)
		, NumFinishedAsyncAllocations(0)
		, NumCanceledAsyncRequests(0)
		, BlockedCycles(0)
		, NumLockedChunks(0)
		, bBenchmarkMode(false)
	{}

	virtual ~FGPUDefragAllocator() = default;

	/**
	* Initialize this allocator with a preallocated block of memory.
	*
	* @param InMemoryBase			Base address for the block of memory
	* @param InMemorySize			Size of the block of memory, in uint8s
	* @param InAllocationAlignment	Alignment for all allocations, in uint8s
	*/
	virtual void	Initialize(uint8* InMemoryBase, int64 InMemorySize, int32 InAllocationAlignment)
	{
		// Update size, Pointer and alignment.
		MemoryBase = InMemoryBase;
		MemorySize = InMemorySize;
		AllocationAlignment = InAllocationAlignment;
		check(Align(MemoryBase, AllocationAlignment) == MemoryBase);

		// Update stats in a thread safe way.
		FPlatformAtomics::InterlockedExchange(&AvailableMemorySize, MemorySize);
		// Allocate initial chunk.
		FirstChunk = new FMemoryChunk(MemoryBase, MemorySize, *this, FirstChunk, TStatId());
		LastChunk = FirstChunk;
	}

	virtual void	Initialize(uint8* InMemoryBase, int64 InMemorySize)
	{
		Initialize(InMemoryBase, InMemorySize, 64);		
	}

	/**
	* Returns the current allocator settings.
	*
	* @param OutSettings	[out] Current allocator settings
	*/
	void	GetSettings(FSettings& OutSettings)
	{
		OutSettings = Settings;
	}

	/**
	* Sets new allocator settings.
	*
	* @param InSettings	New allocator settings to replace the old ones.
	*/
	void	SetSettings(const FSettings& InSettings)
	{
		Settings = InSettings;
	}

	/**
	* Returns whether allocator has been initialized.
	*/
	bool	IsInitialized()
	{
		return MemoryBase != nullptr;
	}

	/**
	* Allocate physical memory.
	*
	* @param	AllocationSize	Size of allocation
	* @param	bAllowFailure	Whether to allow allocation failure or not
	* @return	Pointer to allocated memory
	*/
	virtual void*	Allocate(int64 AllocationSize, int32 Alignment, TStatId InStat, bool bAllowFailure);	

	/**
	* Frees allocation associated with the specified Pointer.
	*
	* @param Pointer		Pointer to free.
	*/
	virtual void	Free(void* Pointer);

	/**
	* Locks an FMemoryChunk
	*
	* @param Pointer		Pointer indicating which chunk to lock
	*/
	virtual void	Lock(const void* Pointer);

	/**
	* Unlocks an FMemoryChunk
	*
	* @param Pointer		Pointer indicating which chunk to unlock
	*/
	virtual void	Unlock(const void* Pointer);

	/**
	* Sets the user payload for an FMemoryChunk
	*
	* @param Pointer		Pointer indicating a chunk
	* @param UserPayload	User payload to set
	*/
	void	SetUserPayload(const void* Pointer, void* UserPayload);

	/**
	* Returns the user payload for an FMemoryChunk
	*
	* @param Pointer		Pointer indicating a chunk
	* return				The chunk's user payload
	*/
	void*	GetUserPayload(const void* Pointer);

	/**
	* Returns the amount of memory allocated for the specified address.
	*
	* @param Pointer		Pointer to check.
	* @return				Number of uint8s allocated
	*/
	int64		GetAllocatedSize(void* Pointer);

	/**
	* Tries to reallocate texture memory in-place (without relocating),
	* by adjusting the base address of the allocation but keeping the end address the same.
	* Note: Newly freed memory due to shrinking won't be available for allocation right away (need GPU sync).
	*
	* @param	OldBaseAddress	Pointer to the original allocation
	* @return	New base address if it succeeded, otherwise nullptr
	**/
	void*	Reallocate(void* OldBaseAddress, int64 NewSize);

	bool IsValidPoolMemory(const void* Pointer) const
	{
		return (Pointer != nullptr) && (Pointer >= MemoryBase) && (Pointer < (MemoryBase + MemorySize));
	}

#if 0
	/**
	* Requests an async allocation or reallocation.
	* The caller must hold on to the request until it has been completed or canceled.
	*
	* @param ReallocationRequest	The request
	* @param bForceRequest			If true, the request will be accepted even if there's currently not enough free space
	* @return		GetMemoryStats				true if the request was accepted
	*/
	bool	AsyncReallocate(FAsyncReallocationRequest* ReallocationRequest, bool bForceRequest);
#endif

	/**
	* Dump allocation information.
	*/
	void	DumpAllocs(FOutputDevice& Ar = *GLog);

	/**
	* Retrieves allocation stats.
	*
	* @param	OutAllocatedMemorySize		[out] Size of allocated memory
	* @param	OutAvailableMemorySize		[out] Size of available memory
	* @param	OutPendingMemoryAdjustment	[out] Size of pending allocation change (due to async reallocation)
	*/
	virtual void	GetMemoryStats(int64& OutAllocatedMemorySize, int64& OutAvailableMemorySize, int64& OutPendingMemoryAdjustment, int64& OutPaddingWasteSize)
	{
		OutAllocatedMemorySize = AllocatedMemorySize;
		OutAvailableMemorySize = AvailableMemorySize;
		OutPendingMemoryAdjustment = PendingMemoryAdjustment;
		OutPaddingWasteSize = PaddingWasteSize;
	}

	int64 GetTotalSize() const
	{
		return MemorySize;
	}

	/**
	* Scans the free chunks and returns the largest size you can allocate.
	*
	* @param OutNumFreeChunks	Upon return, contains the total number of free chunks. May be nullptr.
	* @return					The largest size of all free chunks.
	*/
	int32		GetLargestAvailableAllocation(int32* OutNumFreeChunks = nullptr);

	/**
	* Returns the amount of time blocked by a platform fence since the beginning of the last call to Tick(), in appCycles.
	*/
	uint32	GetBlockedCycles() const
	{
		return BlockedCycles;
	}

	/**
	* Returns whether we're in benchmark mode or not.
	*/
	bool	InBenchmarkMode() const
	{
		return bBenchmarkMode;
	}

	/**
	* Fills a texture with to visualize the texture pool memory.
	*
	* @param	TextureData		Start address
	* @param	SizeX			Number of pixels along X
	* @param	SizeY			Number of pixels along Y
	* @param	Pitch			Number of uint8s between each row
	* @param	PixelSize		Number of uint8s each pixel represents
	*
	* @return true if successful, false otherwise
	*/
	bool	GetTextureMemoryVisualizeData(FColor* TextureData, int32 SizeX, int32 SizeY, int32 Pitch, const int32 PixelSize);

	void	GetMemoryLayout(TArray<FMemoryLayoutElement>& MemoryLayout);

	EMemoryElementType	GetChunkType(FMemoryChunk* Chunk) const;

	/**
	* Fully defragments the memory and blocks until it's done.
	*
	* @param Stats			[out] Stats
	*/
	void	DefragmentMemory(FRelocationStats& Stats);

	/**
	* Partially defragments the memory and tries to process all async reallocation requests at the same time.
	* Call this once per frame.
	*
	* @param Stats			[out] Stats
	* @param bPanicDefrag	If true, performs a full defrag and ignores all reallocation requests
	* @return				Num bytes relocated.
	*/
	virtual int32		Tick(FRelocationStats& Stats, bool bPanicDefrag);

	/**
	* Blocks the calling thread until all relocations and reallocations that were initiated by Tick() have completed.
	*
	* @return		true if there were any relocations in progress before this call
	*/
	bool	FinishAllRelocations();

	/**
	* Blocks the calling thread until the specified request has been completed.
	*
	* @param Request	Request to wait for. Must be a valid request.
	*/
	void	BlockOnAsyncReallocation(FAsyncReallocationRequest* Request);

	/**
	* Cancels the specified reallocation request.
	* Note that the allocator doesn't keep track of requests after it's been completed,
	* so the user must provide the current base address. This may not match any of the
	* addresses in the (old) request since the memory may have been relocated since then.
	*
	* @param Request				Request to cancel. Must be a valid request.
	* @param CurrentBaseAddress	Current baseaddress used by the allocation.
	*/
	void	CancelAsyncReallocation(FAsyncReallocationRequest* Request, const void* CurrentBaseAddress);

	/**
	* Performs a benchmark of the allocator and outputs the result to the log.
	*
	* @param MinChunkSize	Minimum number of uint8s per random chunk
	* @param MaxChunkSize	Maximum number of uint8s per random chunk
	* @param FreeRatio		Free 0.0-1.0 of the memory before benchmarking
	* @param LockRatio		Lock 0.0-1.0 % of the memory before benchmarking
	* @param bFullDefrag	Whether to test full defrag (true) or continuous defrag (false)
	* @param bSaveImages	Whether to save before/after images to hard disk (TexturePoolBenchmark-*.bmp)
	* @param Filename		[opt] Filename to a previously saved memory layout to use for benchmarking, or nullptr
	*/
	void	Benchmark(int32 MinChunkSize, int32 MaxChunkSize, float FreeRatio, float LockRatio, bool bFullDefrag, bool bSaveImages, const TCHAR* Filename);
	
	static inline bool IsAligned(const volatile void* Ptr, const uint32 Alignment)
	{
		return !(UPTRINT(Ptr) & (Alignment - 1));
	}

	int32 GetAllocationAlignment() const
	{
		return AllocationAlignment;
	}

protected:

#if TRACK_RELOCATIONS
	struct FRelocationEntry
	{
		FRelocationEntry(const uint8* InOldBase, const uint8* InNewBase, uint64 InSize, uint64 InSyncIndex);

		const uint8* OldBase;
		const uint8* NewBase;
		uint64 Size;
		uint64 SyncIndex;
	};

	void ValidateRelocations(uint8* UsedBaseAddress, uint64 Size);
	
	TArray<FRelocationEntry> Relocations;
#endif

#if VALIDATE_MEMORY_PROTECTION
	struct FMemProtectTracker
	{
		FMemProtectTracker(const void* InMemory, const void* InUserPayload, uint64 InBlockSize, uint32 InSyncIndex)
		: Memory(InMemory)
		, UserPayload(InUserPayload)
		, BlockSize(InBlockSize)
		, SyncIndex(InSyncIndex)
		{

		}
		const void* Memory;
		const void* UserPayload;
		uint64 BlockSize;
		uint32 SyncIndex;
	};

	TArray<FMemProtectTracker> BlocksToProtect;
	TArray<FMemProtectTracker> BlocksToUnProtect;

	static int32 GGPUDefragDumpRelocationsToTTY;
	static FAutoConsoleVariableRef CVarGPUDefragDumpRelocationsToTTY;

	/** Sets Static memory privileges on blocks that have completed relocations. */
	void SetStaticMemoryPrivileges();

	/** Removes all CPU and GPU read/write privileges from the given Block.  Used for Free memory that is not part of a relocation. */
	virtual void	PlatformSetNoMemoryPrivileges(const FMemProtectTracker& Block) {};

	/** Allows all CPU and GPU read/write privileges on the given Block.  Used to reset privileges on allocation */
	virtual void	PlatformSetStandardMemoryPrivileges(const FMemProtectTracker& Block) {};
	
	/** Allows all CPU R/W, GPU Read, and GPUWrite Platform implementation can determine that it should do so. Used when UserPayload is set.  Used during defraggable registration, and after waiting on the appropriate fence. */
	virtual void	PlatformSetStaticMemoryPrivileges(const FMemProtectTracker& BlocksToAllow) {};

	/** Allows Only GPU R/W.  CPU access is a bug.  This is used when this memory is part of an active relocation */
	virtual void	PlatformSetRelocationMemoryPrivileges(const FMemProtectTracker& BlocksToAllow) {};
	virtual void	PlatformSetRelocationMemoryPrivileges(const TArray<FMemProtectTracker>& BlocksToRemove) {};
#endif


	/**
	* Copy memory from one location to another. If it returns false, the defragmentation
	* process will assume the memory is not relocatable and keep it in place.
	* Note: Source and destination may overlap.
	*
	* @param Dest			Destination memory start address
	* @param Source		Source memory start address
	* @param Size			Number of uint8s to copy
	* @param UserPayload	User payload for this allocation
	*/
	virtual void	PlatformRelocate(void* Dest, const void* Source, int64 Size, void* UserPayload) = 0;

	

	/**
	* Inserts a fence to synchronize relocations.
	* The fence can be blocked on at a later time to ensure that all relocations initiated
	* so far has been fully completed.
	*
	* @return		New fence value
	*/
	virtual uint64	PlatformInsertFence() = 0;

	/**
	* Blocks the calling thread until all relocations initiated before the fence
	* was added has been fully completed.
	*
	* @param Fence		Fence to block on
	*/
	virtual void	PlatformBlockOnFence(uint64 Fence) = 0;

	/**
	* Allows each platform to decide whether an allocation can be relocated at this time.
	*
	* @param Source		Base address of the allocation
	* @param UserPayload	User payload for the allocation
	* @return				true if the allocation can be relocated at this time
	*/
	virtual bool	PlatformCanRelocate(const void* Source, void* UserPayload) const = 0;

	/**
	* Notifies the platform that an async reallocation request has been completed.
	*
	* @param FinishedRequest	The request that got completed
	* @param UserPayload		User payload for the allocation
	*/
	virtual void	PlatformNotifyReallocationFinished(FAsyncReallocationRequest* FinishedRequest, void* UserPayload) = 0;

	/**
	* Copy memory from one location to another. If it returns false, the defragmentation
	* process will assume the memory is not relocatable and keep it in place.
	* Note: Source and destination may overlap.
	*
	* @param Stats			[out] Stats
	* @param Dest			Destination memory chunk
	* @param DestOffset	Destination offset, counted from the base address of the destination memory chunk, in uint8s
	* @param Source		Base address of the source memory
	* @param Size			Number of uint8s to copy
	* @param UserPayload	User payload for the allocation
	*/
	void Relocate(FRelocationStats& Stats, FMemoryChunk* Dest, int64 DestOffset, const void* Source, int64 Size, void* UserPayload)
	{
		MemoryTrace_ReallocFree((uint64)Source);
		MemoryTrace_ReallocAlloc((uint64)Dest->Base, Size, 4);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAllocMoved(ELLMTracker::Default, Dest->Base, Source));

		uint8* DestAddr = Dest->Base + DestOffset;
		int64 MemDistance = (int64)(DestAddr) - (int64)(Source);
		int64 AbsDistance = FMath::Abs(MemDistance);
		bool bOverlappedMove = AbsDistance < Size;

		if (!bBenchmarkMode)
		{
#if VALIDATE_MEMORY_PROTECTION
			BlocksToProtect.Emplace(DestAddr, UserPayload, Size, CurrentSyncIndex);
			BlocksToProtect.Emplace(Source, UserPayload, Size, CurrentSyncIndex);
#endif
			PlatformRelocate(DestAddr, Source, Size, UserPayload);
		}
		int64 RelocateSize = bOverlappedMove ? (Size * Settings.MaxDefragRelocations) : Size;
		Dest->UserPayload = UserPayload;
		Stats.NumBytesRelocated += RelocateSize;
		Stats.NumRelocations++;		
	}

	/**
	* Returns the sync index to be completed by the next call to FinishAllRelocations().
	*/
	uint32			GetCurrentSyncIndex() const
	{
		return CurrentSyncIndex;
	}

	/**
	* Performs a partial defrag doing fast checks only. Adjacency and freelist walk.
	*
	* @param Stats			[out] Stats
	* @param StartTime		Start time, used for limiting the Tick() time
	*/
	void			PartialDefragmentationFast(FRelocationStats& Stats, double StartTime);

	/**
	* Performs a partial defrag doing slow all chunk search to find used chunks to move that are surrounded by other used chunks
	* That a freechunk walk won't find.
	*
	* @param Stats			[out] Stats
	* @param StartTime		Start time, used for limiting the Tick() time
	*/
	void			PartialDefragmentationSlow(FRelocationStats& Stats, double StartTime);

	/**
	* Performs a partial defrag by shifting down memory to fill holes, in a brute-force manner.
	* Takes consideration to async reallocations, but processes the all memory in order.
	*
	* @param Stats			[out] Stats
	* @param StartTime		Start time, used for limiting the Tick() time
	*/
	void			PartialDefragmentationDownshift(FRelocationStats& Stats, double StartTime);

	/**
	* Performs a full defrag and ignores all reallocation requests.
	*
	* @param Stats			[out] Stats
	*/
	void			FullDefragmentation(FRelocationStats& Stats);

	/**
	* Tries to immediately grow a memory chunk by moving the base address, without relocating any memory.
	*
	* @param Chunk			Chunk to grow
	* @param GrowAmount	Number of uint8s to grow by
	* @return				nullptr if it failed, otherwise the new grown chunk
	*/
	FMemoryChunk*	Grow(FMemoryChunk* Chunk, int64 GrowAmount);

	/**
	* Immediately shrinks a memory chunk by moving the base address, without relocating any memory.
	* Always succeeds.
	*
	* @param Chunk			Chunk to shrink
	* @param ShrinkAmount	Number of uint8s to shrink by
	* @return				The new shrunken chunk
	*/
	FMemoryChunk*	Shrink(FMemoryChunk* Chunk, int64 ShrinkAmount);

	/**
	* Checks the int32ernal state for errors. (Slow)
	*
	* @param bCheckSortedFreeList	If true, also checks that the freelist is sorted
	*/
	void			CheckForErrors(bool bCheckSortedFreeList);

	/**
	* Returns true if the specified chunk is allowed to relocate at this time.
	* Will also call PlatformCanRelocate().
	*
	* @param Chunk		Chunk to check
	* @return			true if the allocation can be relocated at this time
	*/
	bool			CanRelocate(const FMemoryChunk* Chunk) const;

	/**
	* Inserts a platform fence and updates the allocator sync index to match.
	*/
	void			InsertFence();

	/**
	* Blocks the calling thread until the current sync fence has been completed.
	*/
	void			BlockOnFence();

	/**
	* Blocks the calling thread until the specified sync index has been completed.
	*
	* @param SyncIndex		Sync index to wait for
	*/
	void			BlockOnSyncIndex(uint32 SyncIndex);

	/**
	* Split allocation into two, first chunk being used and second being available.
	* Maint32ains the free-list order if bSortedFreeList is true.
	*
	* @param BaseChunk			Chunk to split
	* @param FirstSize			New size of first chunk
	* @param bSortedFreeList	If true, maint32ains the free-list order
	*/
	void			Split(FMemoryChunk* BaseChunk, int64 FirstSize)
	{
		check(BaseChunk);
		check(FirstSize < BaseChunk->Size);
		check(FirstSize > 0);

		// Don't make any assumptions on the following chunk. Because Reallocate() will make the 1st chunk free
		// and the 2nd chunk used, so it's ok to have the following chunk free. Note, this only happens when Reallocate()
		// is splitting the very first chunk in the pool.
		// Don't make any assumptions for the previous chunk either...
		// 		check( !BaseChunk->NextChunk || !BaseChunk->NextChunk->bIsAvailable );
		// 		check( !BaseChunk->PreviousChunk || !BaseChunk->PreviousChunk->bIsAvailable || !BaseChunk->bIsAvailable );

		// Calculate size of second chunk...
		int64 SecondSize = BaseChunk->Size - FirstSize;
		// ... and create it.

		//todo: fix stats
		//ensureMsgf(BaseChunk->Stat.IsNone(), TEXT("Free chunk has stat"));
		FMemoryChunk* NewFreeChunk = new FMemoryChunk(BaseChunk->Base + FirstSize, SecondSize, *this, BaseChunk, TStatId());

		// Keep the original sync index for the new chunk if necessary.
		if (BaseChunk->IsRelocating() && BaseChunk->SyncSize > FirstSize)
		{
			int64 SecondSyncSize = BaseChunk->SyncSize - FirstSize;
			NewFreeChunk->SetSyncIndex(BaseChunk->SyncIndex, SecondSyncSize);
		}
		BaseChunk->SetSyncIndex(BaseChunk->SyncIndex, FMath::Min((int64)FirstSize, BaseChunk->SyncSize));

		// Resize base chunk.
		BaseChunk->Size = FirstSize;
	} //-V773

	/**
	* Marks the specified chunk as 'allocated' and updates tracking variables.
	* Splits the chunk if only a portion of it is allocated.
	*
	* @param FreeChunk			Chunk to allocate
	* @param AllocationSize	Number of uint8s to allocate
	* @param bAsync			If true, allows allocating from relocating chunks and maint32ains the free-list sort order.
	* @return					The memory chunk that was allocated (the original chunk could've been split).
	*/
	FMemoryChunk*	AllocateChunk(FMemoryChunk* FreeChunk, int64 AllocationSize, bool bAsync, bool bDoValidation = true);

	/**
	* Marks the specified chunk as 'free' and updates tracking variables.
	* Calls LinkFreeChunk() to coalesce adjacent free memory.
	*
	* @param Chunk						Chunk to free	
	*/
	void			FreeChunk(FMemoryChunk* Chunk);

	/**
	* Frees the passed in chunk and coalesces adjacent free chunks into 'Chunk' if possible.
	* Maint32ains the free-list order if bSortedFreeList is true.
	*
	* @param Chunk				Chunk to mark as available.
	* @param bSortedFreeList	If true, maintains the free-list sort order
	*/
	void			LinkFreeChunk(FMemoryChunk* Chunk)
	{
		check(Chunk);
		// Mark chunk as available.
		Chunk->LinkFree(nullptr);
		// Kick of merge pass.
		Coalesce(Chunk);
	}

	/**
	* Merges any adjacent free chunks into the specified free chunk.
	* Doesn't affect the free-list sort order.
	*
	* @param FreedChunk	Chunk that just became available.
	*/
	void			Coalesce(FMemoryChunk* FreedChunk);

	/**
	* Sorts the freelist based on increasing base address.
	*
	* @param NumFreeChunks		[out] Number of free chunks
	* @param LargestFreeChunk	[out] Size of the largest free chunk, in uint8s
	*/
	void			SortFreeList(int32& NumFreeChunks, int64& LargestFreeChunk);

	/**
	* Defrag helper function. Checks if the specified allocation fits within
	* the adjacent free chunk(s).
	*
	* @param UsedChunk		Allocated chunk to check for a fit
	* @param bAnyChunkType	If false, only succeeds if 'UsedChunk' has a reallocation request and fits
	* @return				Returns 'UsedChunk' if it fits the criteria, otherwise nullptr
	*/
	FMemoryChunk*	FindAdjacent(FMemoryChunk* UsedChunk, bool bAnyChunkType);	

	/**
	* Searches for an allocated chunk that would fit within the specified free chunk.
	* The allocated chunk must be adjacent to a free chunk and have a larger
	* base address than 'FreeChunk'.
	* Starts searching from the end of the texture pool.
	*
	* @param FreeChunk		Free chunk we're trying to fill up
	* @return				Pointer to a suitable chunk, or nullptr
	*/
	FMemoryChunk*	FindAdjacentToHole(FMemoryChunk* FreeChunk);

	/**
	* Searches for an allocated chunk that would fit within the specified free chunk.
	* Any chunk that fits and has a larger base address than 'FreeChunk' is accepted.
	* Starts searching from the end of the texture pool.
	*
	* @param FreeChunk		Free chunk we're trying to fill up
	* @return				Pointer to a suitable chunk, or nullptr
	*/
	FMemoryChunk*	FindAny(FMemoryChunk* FreeChunk);

	/**
	* Initiates an async relocation of an allocated chunk into a free chunk.
	* Takes potential reallocation request into account.
	*
	* @param Stats			[out] Stats
	* @param FreeChunk		Destination chunk (free memory)
	* @param UsedChunk		Source chunk (allocated memory)
	* @return				Next Free chunk to try to fill up
	*/
	FMemoryChunk*	RelocateIntoFreeChunk(FRelocationStats& Stats, FMemoryChunk* FreeChunk, FMemoryChunk* UsedChunk);

	FMemoryChunk* RelocateAllowed(FMemoryChunk* FreeChunk, FMemoryChunk* UsedChunk);	

	FCriticalSection	SynchronizationObject;

	/** Total size of memory pool, in uint8s.						*/
	uint64				MemorySize;
	/** Base of memory pool.										*/
	uint8*			MemoryBase;
	/** Allocation alignment requirements.							*/
	int32				AllocationAlignment;
	/** Head of linked list of chunks. Sorted by memory address.	*/
	FMemoryChunk*	FirstChunk;
	/** Last chunk in the linked list of chunks (see FirstChunk).	*/
	FMemoryChunk*	LastChunk;
	/** Head of linked list of free chunks.	Unsorted.				*/
	FMemoryChunk*	FirstFreeChunk;
	/** Cumulative time spent in allocator.							*/
	double			TimeSpentInAllocator;
	/** Allocated memory in uint8s.									*/
	typedef int64 memsize_t;

	volatile memsize_t	PaddingWasteSize;

	volatile memsize_t	AllocatedMemorySize;
	/** Available memory in uint8s.									*/
	volatile memsize_t	AvailableMemorySize;
	/** Adjustment to allocated memory, pending all reallocations.	*/
	volatile memsize_t	PendingMemoryAdjustment;

	/** Mapping from Pointer to chunk for fast removal.				*/
	TMap<void*, FMemoryChunk*>	PointerToChunkMap;

	/** Allocator settings that affect its behavior.				*/
	FSettings		Settings;

	/** Ever-increasing index to synchronize all relocations initiated by Tick(). */
	uint64			CurrentSyncIndex;
	/** Sync index that has been completed, so far.					*/
	uint64			CompletedSyncIndex;

	/** Number of async relocations that are currently in progress.	*/
	int32				NumRelocationsInProgress;
	/** Platform-specific (GPU) fence, used for synchronizing the Sync Index. */
	uint64			PlatformSyncFence;

	/** Chunks that couldn't be freed immediately because they were being relocated. */
	TDoubleLinkedList<FMemoryChunk*>	PendingFreeChunks;

	uint64 CurrentLargestHole;
	int32 CurrentNumHoles;

	// Stats
	/** Total number of relocations performed so far.				*/
	uint64			TotalNumRelocations;
	/** Total number of uint8s relocated so far.						*/
	uint64			TotalNumBytesRelocated;
	/** Smallest consecutive free memory region we've had.			*/
	int64				MinLargestHole;
	/** Maximum number of disjoint32 free memory regions we've had.	*/
	int32				MaxNumHoles;	
	/** Total number of async reallocations successfully completed so far.	*/
	int32				NumFinishedAsyncReallocations;
	/** Total number of async allocations successfully completed so far.	*/
	int32				NumFinishedAsyncAllocations;
	/** Total number of async requests that has been canceled so far.		*/
	int32				NumCanceledAsyncRequests;
	/** Amount of time blocked by a platform fence since the beginning of the last call to Tick(), in appCycles.	*/
	uint32			BlockedCycles;

	int32 NumLockedChunks;

#if VALIDATE_MEMORY_PROTECTION
	double TimeInMemProtect;
#endif

	/** When in benchmark mode, don't call any Platform functions.	*/
	bool			bBenchmarkMode;

	friend FScopedGPUDefragLock;
};

//FScopedGPUDefragLock can't cover any scope that will add dcb commands or we might deadlock.
class FScopedGPUDefragLock
{
public:
	FScopedGPUDefragLock(FGPUDefragAllocator& InDefragAllocator)
		: DefragAllocator(InDefragAllocator)
	{
		DefragAllocator.SynchronizationObject.Lock();
	}

	~FScopedGPUDefragLock()
	{
		DefragAllocator.SynchronizationObject.Unlock();
	}
private:
	FGPUDefragAllocator& DefragAllocator;
};

/**
* Asynchronous reallocation request.
* Requests are created and deleted by the user, but it must stick around until the allocator is done with it.
* Requests may be fulfilled immediately, check HasCompleted() after making the request.
*/
class FAsyncReallocationRequest
{
public:
	/**
	* Creates a new reallocation request.
	*
	* @param InCurrentBaseAddress	Current base address
	* @param InNewSize				Requested new size, in uint8s
	* @param InRequestStatus		Will be decremented by one when the request has been completed. Can be nullptr.
	*/
	FAsyncReallocationRequest(void* InCurrentBaseAddress, int32 InNewSize, FThreadSafeCounter* InRequestStatus)
		: OldAddress(InCurrentBaseAddress)
		, NewAddress(nullptr)
		, OldSize(0)	// Set by AsyncReallocate()
		, NewSize(InNewSize)
		, int32ernalRequestStatus(1)
		, ExternalRequestStatus(InRequestStatus)
		, bIsCanceled(false)
		, MemoryChunk(nullptr)
	{
	}

	/** Destructor. */
	~FAsyncReallocationRequest()
	{
		check(!HasStarted() || IsCanceled() || HasCompleted());
	}

	/** Returns true if the request is for a new allocation. */
	bool	IsAllocation() const
	{
		return OldAddress == nullptr && OldSize == 0;
	}

	/** Returns true if the request is for a reallocation. */
	bool	IsReallocation() const
	{
		return OldAddress != nullptr;
	}

	/** Returns true if the request has been canceled. */
	bool	IsCanceled() const
	{
		return bIsCanceled;
	}

	/** Returns true if the request has been completed. */
	bool	HasCompleted() const
	{
		bool bHasCompleted = int32ernalRequestStatus.GetValue() == 0;
		check(!bHasCompleted || NewAddress || bIsCanceled);
		return bHasCompleted;
	}

	/** Returns true if the allocator has started processing the request (true for completed requests as well). */
	bool	HasStarted() const
	{
		return NewAddress ? true : false;
	}

	/** Returns the original base address. */
	void*	GetOldBaseAddress() const
	{
		return OldAddress;
	}

	/** Returns the new base address, or nullptr if the request hasn't started yet. */
	void*	GetNewBaseAddress() const
	{
		return NewAddress;
	}

	/** Returns the requested new memory size (in uint8s). */
	int32		GetNewSize() const
	{
		return NewSize;
	}

private:
	// Hidden on purpose since outside usage isn't necessarily thread-safe.
	FAsyncReallocationRequest(const FAsyncReallocationRequest& Other)	{ FMemory::Memcpy(this, &Other, sizeof(FAsyncReallocationRequest)); }
	void operator=(const FAsyncReallocationRequest& Other) { FMemory::Memcpy(this, &Other, sizeof(FAsyncReallocationRequest)); }

	/**
	* Marks the request as completed. Also decrements the external request status, if it wasn't nullptr.
	*/
	void	MarkCompleted()
	{
		check(int32ernalRequestStatus.GetValue() == 1);
		int32ernalRequestStatus.Decrement();
		if (ExternalRequestStatus)
		{
			ExternalRequestStatus->Decrement();
		}
	}

	/** Original base address. */
	void*				OldAddress;
	/** New base address, or nullptr if the request hasn't started yet. */
	void*				NewAddress;
	/** Original memory size, in uint8s. Set by AsyncReallocate(). */
	int32					OldSize;
	/** Requested new memory size, in uint8s. */
	int32					NewSize;
	/** Thread-safe counter that will be decremented by one when the request has been completed. */
	FThreadSafeCounter	int32ernalRequestStatus;
	/** External counter that will be decremented by one when the request has been completed. */
	FThreadSafeCounter*	ExternalRequestStatus;
	/** true if the request has been canceled. */
	uint32				bIsCanceled : 1;

	/**
	* Corresponding memory chunk. Starts out as the chunk that contains the original memory block,
	* but is changed to the destination chunk once the allocator starts processing the request.
	*/
	class FGPUDefragAllocator::FMemoryChunk*		MemoryChunk;

	friend class FGPUDefragAllocator;
};


/**
*	Returns the current size (in uint8s), or the final size if it has a reallocating request.
*/
inline int64 FGPUDefragAllocator::FMemoryChunk::GetFinalSize() const
{
	return Size;
}

/**
* Returns true if the specified chunk is allowed to relocate at this time.
* Will also call PlatformCanRelocate().
*
* @param Chunk		Chunk to check
* @return			true if the allocation can be relocated at this time
*/
inline bool FGPUDefragAllocator::CanRelocate(const FMemoryChunk* Chunk) const
{
	if (Chunk->IsLocked())
	{
		return false;
	}

	if (!bBenchmarkMode)
	{
		return PlatformCanRelocate(Chunk->Base, Chunk->UserPayload);
	}
	else
	{
		return true;
	}
}

/**
* Blocks the calling thread until the specified request has been completed.
*
* @param Request	Request to wait for. Must be a valid request.
*/
inline void FGPUDefragAllocator::BlockOnAsyncReallocation(FAsyncReallocationRequest* Request)
{
	check(Request->HasStarted());
	if (!Request->HasCompleted())
	{
		BlockOnSyncIndex(Request->MemoryChunk->SyncIndex);
	}
}
