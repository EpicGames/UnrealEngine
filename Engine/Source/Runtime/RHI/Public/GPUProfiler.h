// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUProfiler.h: Hierarchical GPU Profiler.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/TVariant.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include "RHIBreadcrumbs.h"
#include "RHIStats.h"

#include "Containers/AnsiString.h"
#include "Containers/SpscQueue.h"
#include "Containers/StaticArray.h"

struct FRHIEndFrameArgs;

namespace UE::RHI::GPUProfiler
{
	struct FQueue
	{
		enum class EType : uint8
		{
			Graphics,
			Compute,
			Copy,
			SwapChain
		};

		union
		{
			struct
			{
				EType Type;
				uint8 GPU;
				uint8 Index;
				uint8 Padding;
			};
			uint32 Value = 0;
		};

		FQueue() = default;

		constexpr FQueue(EType Type, uint8 GPU, uint8 Index)
			: Type   (Type)
			, GPU    (GPU)
			, Index  (Index)
			, Padding(0)
		{}

		constexpr bool operator == (FQueue const& RHS) const
		{
			return Value == RHS.Value;
		}

		constexpr bool operator != (FQueue const& RHS) const
		{
			return !(*this == RHS);
		}

		friend uint32 GetTypeHash(FQueue const& Queue)
		{
			return GetTypeHash(Queue.Value);
		}

		TCHAR const* GetTypeString() const
		{
			switch (Type)
			{
			case EType::Graphics:  return TEXT("Graphics");
			case EType::Compute:   return TEXT("Compute");
			case EType::Copy:      return TEXT("Copy");
			case EType::SwapChain: return TEXT("Swapchain");
			default:               return TEXT("<unknown>");
			}
		}
	};

	struct FEvent
	{
		//
		// All timestamps are relative to FPlatformTime::Cycles64().
		// TOP = Top of Pipe. Timestamps written by the GPU's command processor before work begins.
		// BOP = Bottom of Pipe. Timestamps written after the GPU completes work.
		//
		
		// Inserted on each call to RHIEndFrame. Marks the end of a profiler frame.
		struct FFrameBoundary
		{
			// CPU timestamp from the platform RHI's submission thread where the frame boundary occured.
			uint64 CPUTimestamp;

			// The index of the frame that just ended.
			// Very first frame of the engine is frame 0 (from boot to first call to RHIEndFrame).
			uint32 const FrameNumber;

			// When true, triggers the profiling of the next frame
			bool const bProfileNextFrame;

		#if STATS
			// Should be TOptional<int64> but it is not trivially destructible
			bool const bStatsFrameSet;
			int64 const StatsFrame;
		#endif

		#if WITH_RHI_BREADCRUMBS
			// The RHI breadcrumb currently at the top of the stack at the frame boundary.
			FRHIBreadcrumbNode* const Breadcrumb;
		#endif

			RHI_API FFrameBoundary(ERHIPipeline Pipeline, FRHIEndFrameArgs const& Args, uint64 CPUTimestamp);
		};

		// When present in the stream, overrides the total GPU time stat with the value it contains.
		// Used for platform RHIs which don't support accurate GPU timing.
		struct FFrameTime
		{
			// Same frequency as FPlatformTime::Cycles64()
			uint64 TotalGPUTime;

			FFrameTime(uint64 InTotalGPUTime)
				: TotalGPUTime(InTotalGPUTime)
			{}
		};

	#if WITH_RHI_BREADCRUMBS
		struct FBeginBreadcrumb
		{
			FRHIBreadcrumbNode* const Breadcrumb;
			uint64 GPUTimestampTOP;

			FBeginBreadcrumb(FRHIBreadcrumbNode* Breadcrumb, uint64 GPUTimestampTOP = 0)
				: Breadcrumb(Breadcrumb)
				, GPUTimestampTOP(GPUTimestampTOP)
			{}
		};

		struct FEndBreadcrumb
		{
			FRHIBreadcrumbNode* const Breadcrumb;
			uint64 GPUTimestampBOP = 0;

			FEndBreadcrumb(FRHIBreadcrumbNode* Breadcrumb, uint64 GPUTimestampBOP = 0)
				: Breadcrumb(Breadcrumb)
				, GPUTimestampBOP(GPUTimestampBOP)
			{}
		};
	#endif

		// Inserted when the GPU starts work on a queue.
		struct FBeginWork
		{
			// CPU timestamp of when the work was submitted to the driver for execution on the GPU.
			uint64 CPUTimestamp;

			// TOP timestamp of when the work actually started on the GPU.
			uint64 GPUTimestampTOP;

			FBeginWork(uint64 CPUTimestamp, uint64 GPUTimestampTOP = 0)
				: CPUTimestamp(CPUTimestamp)
				, GPUTimestampTOP(GPUTimestampTOP)
			{}
		};

		// Inserted when the GPU completes work on a queue and goes idle.
		struct FEndWork
		{
			uint64 GPUTimestampBOP;

			FEndWork(uint64 GPUTimestampBOP = 0)
				: GPUTimestampBOP(GPUTimestampBOP)
			{}
		};

		struct FStats
		{
			uint32 NumDraws      = 0;
			uint32 NumDispatches = 0;
			uint32 NumPrimitives = 0;
			uint32 NumVertices   = 0;

			FStats& operator += (FStats const& RHS)
			{
				NumDraws      += RHS.NumDraws;
				NumDispatches += RHS.NumDispatches;
				NumPrimitives += RHS.NumPrimitives;
				NumVertices   += RHS.NumVertices;
				return *this;
			}

			operator bool() const
			{
				return NumDraws > 0
					|| NumDispatches > 0
					|| NumPrimitives > 0
					|| NumVertices > 0;
			}
		};

		// Can only be inserted when the GPU is marked "idle", i.e. after an FEndWork event.
		struct FSignalFence
		{
			//
			// Timestamp when the fence signal was enqueued to the GPU/driver.
			// 
			// The signal on the GPU doesn't happen until after the previous FEndWork
			// event's BOP timestamp, or this CPU timestamp, whichever is later.
			//
			uint64 CPUTimestamp;

			// The fence value signaled.
			uint64 Value;

			FSignalFence(uint64 CPUTimestamp, uint64 Value)
				: CPUTimestamp(CPUTimestamp)
				, Value(Value)
			{}
		};

		// Can only be inserted when the GPU is marked "idle", i.e. after an FEndWork event.
		struct FWaitFence
		{
			// Timestamp when the fence wait was enqueued to the GPU/driver.
			uint64 CPUTimestamp;

			// The fence value awaited.
			uint64 Value;

			// The queue the GPU is waiting for a fence signal from.
			FQueue Queue;

			FWaitFence(uint64 CPUTimestamp, uint64 Value, FQueue Queue)
				: CPUTimestamp(CPUTimestamp)
				, Value(Value)
				, Queue(Queue)
			{}
		};

		struct FFlip
		{
			uint64 GPUTimestamp;
		};

		struct FVsync
		{
			uint64 GPUTimestamp;
		};
		
		using FStorage = TVariant<
			  FFrameBoundary
			, FFrameTime
		#if WITH_RHI_BREADCRUMBS
			, FBeginBreadcrumb
			, FEndBreadcrumb
		#endif
			, FBeginWork
			, FEndWork
			, FStats
			, FSignalFence
			, FWaitFence
			, FFlip
			, FVsync
		>;

		enum class EType
		{
			FrameBoundary   = FStorage::IndexOfType<FFrameBoundary  >(),
			FrameTime       = FStorage::IndexOfType<FFrameTime      >(),
		#if WITH_RHI_BREADCRUMBS
			BeginBreadcrumb = FStorage::IndexOfType<FBeginBreadcrumb>(),
			EndBreadcrumb   = FStorage::IndexOfType<FEndBreadcrumb  >(),
		#endif
			BeginWork       = FStorage::IndexOfType<FBeginWork      >(),
			EndWork         = FStorage::IndexOfType<FEndWork        >(),
			Stats           = FStorage::IndexOfType<FStats          >(),
			SignalFence     = FStorage::IndexOfType<FSignalFence    >(),
			WaitFence       = FStorage::IndexOfType<FWaitFence      >(),
			Flip            = FStorage::IndexOfType<FFlip           >(),
			VSync		    = FStorage::IndexOfType<FVsync          >()
		};

		FStorage Value;

		EType GetType() const
		{
			return static_cast<EType>(Value.GetIndex());
		}

		template <typename T>
		FEvent(T const& Value)
			: Value(TInPlaceType<T>(), Value)
		{}

		FEvent(FEvent const&) = delete;
		FEvent(FEvent&&) = delete;
	};

	class FEventStream
	{
	private:
		struct FChunk
		{
			struct FHeader
			{
				FChunk* Next = nullptr;
				uint32 Num = 0;
				uint32 Iter = 0;

			#if WITH_RHI_BREADCRUMBS
				FRHIBreadcrumbAllocatorArray BreadcrumbAllocators;
			#endif
			} Header;

			static constexpr uint32 ChunkSizeInBytes = 16 * 1024;
			static constexpr uint32 RemainingBytes = ChunkSizeInBytes - Align<uint32>(sizeof(FHeader), alignof(FHeader));
			static constexpr uint32 MaxEventsPerChunk = RemainingBytes / Align<uint32>(sizeof(FEvent), alignof(FEvent));

			TStaticArray<TTypeCompatibleBytes<FEvent>, MaxEventsPerChunk> Elements;

			static RHI_API TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> MemoryPool;

			void* operator new(size_t Size)
			{
				check(Size == sizeof(FChunk));

				void* Memory = MemoryPool.Pop();
				if (!Memory)
				{
					Memory = FMemory::Malloc(sizeof(FChunk), alignof(FChunk));
				}

				// UE-295331 Investigation : Fill memory with garbage on allocation to catch use-after-free etc.
				FMemory::Memset(Memory, 0xf7, sizeof(FChunk));

				return Memory;
			}

			void operator delete(void* Pointer)
			{
				// UE-295331 Investigation : Fill memory with garbage on deallocation to catch use-after-free etc.
				FMemory::Memset(Pointer, 0xe5, sizeof(FChunk));

				MemoryPool.Push(Pointer);
			}

			FEvent* GetElement(uint32 Index)
			{
				return Elements[Index].GetTypedPtr();
			}
		};

		static_assert(sizeof(FChunk) <= FChunk::ChunkSizeInBytes, "Incorrect FChunk size.");

		FChunk* First = nullptr;
		FChunk* Current = nullptr;

	public:
		FQueue const Queue;

		FEventStream(FQueue const Queue)
			: Queue(Queue)
		{}

		FEventStream(FEventStream const&) = delete;

		FEventStream(FEventStream&& Other)
			: First  (Other.First)
			, Current(Other.Current)
			, Queue  (Other.Queue)
		{
			Other.First = nullptr;
			Other.Current = nullptr;
		}

		~FEventStream()
		{
			while (First)
			{
				FChunk* Next = First->Header.Next;
				delete First;
				First = Next;
			}
		}

		template <typename TEventType, typename... TArgs>
		TEventType& Emplace(TArgs&&... Args)
		{
			static_assert(std::is_trivially_destructible_v<TEventType>, "Destructors are not called on GPU profiler events, so the types must be trivially destructible.");

			if (!Current)
			{
				Current = new FChunk;
				if (!First)
				{
					First = Current;
				}
			}

			if (Current->Header.Num >= FChunk::MaxEventsPerChunk)
			{
				FChunk* NewChunk = new FChunk;
				Current->Header.Next = NewChunk;
				Current = NewChunk;
			}

			FEvent* Event = Current->GetElement(Current->Header.Num++);
			new (Event) FEvent(TEventType(Forward<TArgs>(Args)...));

			TEventType& Data = Event->Value.Get<TEventType>();

		#if WITH_RHI_BREADCRUMBS
			if constexpr (
				std::is_same_v<UE::RHI::GPUProfiler::FEvent::FBeginBreadcrumb, TEventType> ||
				std::is_same_v<UE::RHI::GPUProfiler::FEvent::FEndBreadcrumb  , TEventType> ||
				std::is_same_v<UE::RHI::GPUProfiler::FEvent::FFrameBoundary  , TEventType>
				)
			{
				if (Data.Breadcrumb)
				{
					// Attach the breadcrumb allocator for begin/end breadcrumb events.
					// This keeps the breadcrumbs alive until the events have been consumed by the profilers.
					Current->Header.BreadcrumbAllocators.AddUnique(Data.Breadcrumb->Allocator);
				}
			}
		#endif

			return Data;
		}

		bool IsEmpty() const
		{
			return First == nullptr;
		}

		void Append(FEventStream&& Other)
		{
			check(Queue == Other.Queue);

			if (IsEmpty())
			{
				Current = Other.Current;
				First = Other.First;
			}
			else if (!Other.IsEmpty())
			{
				Current->Header.Next = Other.First;
				Current = Other.Current;
			}

			Other.Current = nullptr;
			Other.First = nullptr;
		}

	private:
		friend struct FGPUProfiler;

		FEvent const* Peek() const;
		void Pop();
	};

	RHI_API void ProcessEvents(TArrayView<FEventStream> EventStreams);
	RHI_API void InitializeQueues(TConstArrayView<FQueue> Queues);

	struct FGPUStat
	{
		enum class EType
		{
			Busy,
			Wait,
			Idle
		};

		TCHAR const* const StatName;
		TCHAR const* const DisplayName;
		FRHIDrawStatsCategory const* const DrawStatsCategory;

	#if CSV_PROFILER_STATS
		TOptional<FCsvDeclaredStat> CsvStat;
	#endif

	private:
	#if STATS
		static FString GetIDString(FQueue Queue, bool bFriendly);
		static TCHAR const* GetTypeString(EType Type);

		struct FStatCategory
		{
			FAnsiString const GroupName;
			FString     const GroupDesc;

			FStatCategory(FQueue Queue);

			static TMap<FQueue, TUniquePtr<FStatCategory>> Categories;
		};

		struct FStatInstance
		{
			struct FInner
			{
			#if STATS
				FName StatName;
				TUniquePtr<FDynamicStat> Stat;
			#endif
			};

			FInner Busy, Wait, Idle;
		};

		TMap<FQueue, FStatInstance> Instances;

		FStatInstance::FInner& GetStatInstance(FQueue Queue, EType Type);
	#endif

	public:
		bool bEmitToEngineStats = true;

		FGPUStat(TCHAR const* StatName, TCHAR const* DisplayName, FRHIDrawStatsCategory const* DrawStatsCategory)
			: StatName         (StatName)
			, DisplayName      (DisplayName)
			, DrawStatsCategory(DrawStatsCategory)
		{}

		virtual ~FGPUStat() = default;

		enum class EOnTimingResultsAction : uint8
		{
			Keep,   // Stat object lifetime is managed by the caller (static/global stats).
			Delete, // Profiler will delete this object after all per-queue OnTimingResults calls for the frame have fired.
		};

		virtual EOnTimingResultsAction OnTimingResults(FQueue Queue, double BusyMs, double IdleMs, double WaitMs) { return EOnTimingResultsAction::Keep; }

	#if STATS
		TStatId GetStatId(FQueue Queue, EType Type);
	#endif
	};

	template <typename TNameProvider>
	struct TGPUStat : public FGPUStat
	{
		TGPUStat(FRHIDrawStatsCategory const* DrawStatsCategory = nullptr)
			: FGPUStat(TNameProvider::GetStatName(), TNameProvider::GetDisplayName(), DrawStatsCategory)
		{}
	};

	template <typename TNameProvider>
	struct TGPUStatWithDrawcallCategory : public TGPUStat<TNameProvider>
	{
	#if HAS_GPU_STATS
		FRHIDrawStatsCategory DrawStatsCategory;

		TGPUStatWithDrawcallCategory()
			: TGPUStat<TNameProvider>(&DrawStatsCategory)
			, DrawStatsCategory(FName(TNameProvider::GetStatName()))
		{}
	#endif
	};

	RHI_API bool IsProfiling();

	RHI_API bool ShouldProfileNextFrame();
}

//
// Type used to pipe GPU frame timings from the end-of-pipe / RHI threads up to the game / render threads.
// Stores a history of GPU frame timings, which can be retrieved by engine code via:
//
//       static FRHIGPUFrameTimeHistory::FState GPUFrameTimeState;
//       uint64 GPUFrameTimeCycles64;
//       while (GPUFrameTimeState.PopFrameCycles(GPUFrameTimeCycles64) != FRHIGPUFrameTimeHistory::EResult::Empty)
//       {
//           ...
//       }
//
class FRHIGPUFrameTimeHistory
{
public:
	enum class EResult
	{
		// The next frame timing has been retrieved
		Ok,

		// The next frame timing has been retrieved, but the client has also missed some frames.
		Disjoint,

		// No new frame timing data available.
		Empty
	};

	class FState
	{
		friend FRHIGPUFrameTimeHistory;
		uint64 NextIndex = 0;
	public:
		RHI_API EResult PopFrameCycles(uint64& OutCycles64);
	};

private:
	// Total number of GPU frame timings to store
	static constexpr uint32 MaxLength = 16;

	uint64 NextIndex = 0;
	TStaticArray<uint64, MaxLength> History { InPlace, 0 };

	FCriticalSection CS;

	EResult PopFrameCycles(FState& State, uint64& OutCycles64);

public:
	// Called by platform RHIs to submit new GPU timing data
	RHI_API void PushFrameCycles(double GPUFrequency, uint64 GPUCycles);
};

extern RHI_API FRHIGPUFrameTimeHistory GRHIGPUFrameTimeHistory;
