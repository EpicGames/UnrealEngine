// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"
#include "RHIDefinitions.h"
#include "Trace/Config.h"
#include "UObject/NameTypes.h"

// Tracing for the new GPU Profiler
#ifndef UE_TRACE_GPU_PROFILER_ENABLED
	#define UE_TRACE_GPU_PROFILER_ENABLED UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#endif

class FCborWriter;
class FDebugName;
class FMemoryWriter;

namespace UE::RHI::GPUProfiler
{
class FMetadataSerializer
{
public:
	RHI_API FMetadataSerializer();
	RHI_API ~FMetadataSerializer();

	RHI_API void AppendValue(const ANSICHAR* Value);
	RHI_API void AppendValue(const WIDECHAR* Value);
	RHI_API void AppendValue(const UTF8CHAR* Value);
	RHI_API void AppendValue(uint64 Value);
	RHI_API void AppendValue(int64 Value);
	RHI_API void AppendValue(bool Value);
	RHI_API void AppendValue(const FName& Value);
	RHI_API void AppendValue(const FString& Value);
	RHI_API void AppendValue(const FDebugName& Value);
	RHI_API void AppendValue(double Value);

	void AppendValue(int32 Value)
	{
		AppendValue((int64)Value);
	}
	void AppendValue(int16 Value)
	{
		AppendValue((int64)Value);
	}
	void AppendValue(int8 Value)
	{
		AppendValue((int64)Value);
	}
	void AppendValue(uint32 Value)
	{
		AppendValue((uint64)Value);
	}
	void AppendValue(uint16 Value)
	{
		AppendValue((uint64)Value);
	}
	void AppendValue(uint8 Value)
	{
		AppendValue((uint64)Value);
	}

	void AppendValue(float Value)
	{
		AppendValue((double)Value);
	}

	const TArray<uint8>& GetData() const { return CborData; }

private:
	FCborWriter* CborWriter;
	FMemoryWriter* MemoryWriter;
	TArray<uint8> CborData;
};

struct FGpuProfilerTrace
{
#if UE_TRACE_GPU_PROFILER_ENABLED
private:
	
	RHI_API static uint32 InternalBreadcrumbSpec(const TCHAR* StaticName, const TCHAR* NameFormat, const TArray<uint8>& FieldNames);

	static std::atomic<uint32> NextSpecId;

public:

	/**
	 * Returns true if the Gpu channel is enabled.
	 */
	RHI_API static bool IsAvailable();

	/**
	 * Initialize Gpu Profiling Trace. Must only be called once.
	 */
	RHI_API static void Initialize();

	/**
	 * Trace an initialization event for a Gpu Queue.
	 */
	RHI_API static void InitializeQueue(uint32 QueueId, const TCHAR* Name);

	/**
	 * Trace a frame boundary for a Gpu Queue. 
	 */
	RHI_API static void FrameBoundary(uint32 QueueId, uint32 FrameId);

	/**
	 * Traces a breadcrumb spec and returns an id that can be used in BeginBreadcrumb. 
	 */
	template<size_t Size>
	static uint32 BreadcrumbSpec(const TCHAR* StaticName, const TCHAR* NameFormat, const std::array<const TCHAR*, Size>& FieldNames)
	{
			FMetadataSerializer Serializer;
			for (const TCHAR* FieldName : FieldNames)
			{
				Serializer.AppendValue(FieldName);
			}

			return InternalBreadcrumbSpec(StaticName, NameFormat, Serializer.GetData());
	}

	RHI_API static void BeginBreadcrumb(uint32 SpecId, uint32 QueueId, uint64 GPUTimestampTOP, const TArray<uint8>& CborData);

	/**
	 * Trace the end of a breadcrumb on a Gpu Queue. 
	 */
	RHI_API static void EndBreadcrumb(uint32 QueueId, uint64 GPUTimestampBOP);
	
	/**
	 * Trace the start of a work event on a Gpu Queue. 
	 */
	RHI_API static void BeginWork(uint32 QueueId, uint64 GPUTimestampTOP, uint64 CPUTimestamp);

	/**
	 * Trace the end of a work event on a Gpu Queue. 
	 */
	RHI_API static void EndWork(uint32 QueueId, uint64 GPUTimestampBOP);

	/**
	 * Trace a wait event on a Gpu Queue. 
	 */
	RHI_API static void TraceWait(uint32 QueueId, uint64 StartTime, uint64 EndTime);

	/**
	 * Trace Gpu stats. 
	 */
	RHI_API static void Stats(uint32 QueueId, uint32 NumDraws, uint32 NumPrimitives);

	/**
	 * Trace a signal fence event on a Gpu Queue. 
	 */
	RHI_API static void SignalFence(uint32 QueueId, uint64 ResolvedTimestamp, uint64 Value);

	/**
	 * Trace a wait fence event on a Gpu Queue. 
	 */
	RHI_API static void WaitFence(uint32 QueueId, uint64 ResolvedTimestamp, uint32 QueueToWaitForId, uint64 Value);

#else
	static bool IsAvailable() { return false; }
	static void Initialize() {}
	static void InitializeQueue(uint32 QueueId, const TCHAR* Name) {}
	static void FrameBoundary(uint32 QueueId, uint32 FrameId) {}
	static void BeginBreadcrumb(uint32 SpecId, uint32 QueueId, uint64 GPUTimestampTOP, const TArray<uint8>& CborData) {}
	static void EndBreadcrumb(uint32 QueueId, uint64 GPUTimestampBOP) {}
	static void BeginWork(uint32 QueueId, uint64 GPUTimestampTOP, uint64 CPUTimestamp) {}
	static void EndWork(uint32 QueueId, uint64 GPUTimestampBOP) {}
	static void TraceWait(uint32 QueueId, uint64 StartTime, uint64 EndTime) {}
	static void Stats(uint32 QueueId, uint32 NumDraws, uint32 NumPrimitives) {}
	static void SignalFence(uint32 QueueId, uint64 ResolvedTimestamp, uint64 Value) {}
	static void WaitFence(uint32 QueueId, uint64 ResolvedTimestamp, uint32 QueueToWaitForId, uint64 Value) {}
	
	template<size_t Size>
	static uint32 BreadcrumbSpec(const TCHAR* StaticName, const TCHAR* NameFormat, const std::array<const TCHAR*, Size>& FieldNames) { return 0; }
#endif
};

}

