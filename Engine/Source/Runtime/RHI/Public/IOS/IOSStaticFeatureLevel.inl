// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(UE_IOS_SHADER_PLATFORM_METAL_MRT)
	#define UE_IOS_STATIC_FEATURE_LEVEL ERHIFeatureLevel::SM5
#else
	#define UE_IOS_STATIC_FEATURE_LEVEL ERHIFeatureLevel::ES3_1
#endif

struct FStaticFeatureLevel
{
	inline FStaticFeatureLevel(const ERHIFeatureLevel::Type InFeatureLevel)
	{
		checkSlow(InFeatureLevel == UE_IOS_STATIC_FEATURE_LEVEL);
	}

	inline FStaticFeatureLevel(const TEnumAsByte<ERHIFeatureLevel::Type> InFeatureLevel)
	{
		checkSlow(InFeatureLevel.GetValue() == UE_IOS_STATIC_FEATURE_LEVEL);
	}

	inline operator ERHIFeatureLevel::Type() const
	{
		return UE_IOS_STATIC_FEATURE_LEVEL;
	}

	inline bool operator == (const ERHIFeatureLevel::Type Other) const
	{
		return Other == UE_IOS_STATIC_FEATURE_LEVEL;
	}

	inline bool operator != (const ERHIFeatureLevel::Type Other) const
	{
		return Other != UE_IOS_STATIC_FEATURE_LEVEL;
	}

	inline bool operator <= (const ERHIFeatureLevel::Type Other) const
	{
		return UE_IOS_STATIC_FEATURE_LEVEL <= Other;
	}

	inline bool operator < (const ERHIFeatureLevel::Type Other) const
	{
		return UE_IOS_STATIC_FEATURE_LEVEL < Other;
	}

	inline bool operator >= (const ERHIFeatureLevel::Type Other) const
	{
		return UE_IOS_STATIC_FEATURE_LEVEL >= Other;
	}

	inline bool operator > (const ERHIFeatureLevel::Type Other) const
	{
		return UE_IOS_STATIC_FEATURE_LEVEL > Other;
	}
};
