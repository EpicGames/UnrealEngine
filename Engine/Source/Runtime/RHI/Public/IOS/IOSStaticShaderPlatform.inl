// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"

#if defined(UE_IOS_SHADER_PLATFORM_METAL_ES3_1_IOS)
	#define UE_IOS_STATIC_SHADER_PLATFORM SP_METAL_ES3_1_IOS
#elif defined(UE_IOS_SHADER_PLATFORM_METAL_SIM)
	#define UE_IOS_STATIC_SHADER_PLATFORM SP_METAL_SIM
#elif defined(UE_IOS_SHADER_PLATFORM_METAL_SM5_IOS)
	#define UE_IOS_STATIC_SHADER_PLATFORM SP_METAL_SM5_IOS
#elif defined(UE_IOS_SHADER_PLATFORM_METAL_SM6_IOS)
	#define UE_IOS_STATIC_SHADER_PLATFORM SP_METAL_SM6_IOS
#else
	#error "Unknown IOS static shader platform"
#endif

struct FStaticShaderPlatform
{
	inline FStaticShaderPlatform(const EShaderPlatform InPlatform)
	{
		checkSlow(UE_IOS_STATIC_SHADER_PLATFORM == InPlatform);
	}

	inline operator EShaderPlatform() const
	{
		return UE_IOS_STATIC_SHADER_PLATFORM;
	}

	inline bool operator == (const EShaderPlatform Other) const
	{
		return Other == UE_IOS_STATIC_SHADER_PLATFORM;
	}
	
	inline bool operator != (const EShaderPlatform Other) const
	{
		return Other != UE_IOS_STATIC_SHADER_PLATFORM;
	}
};
