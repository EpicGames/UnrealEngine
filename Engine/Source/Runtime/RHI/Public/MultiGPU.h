// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MultiGPU.h: Multi-GPU support
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"

#if DO_GUARD_SLOW
	#define GPUMASK_CONSTEXPR
#else
	#define GPUMASK_CONSTEXPR constexpr
#endif

#if WITH_MGPU
	#define MAX_NUM_GPUS 8
	extern RHI_API uint32 GNumExplicitGPUsForRendering;
	extern RHI_API uint32 GVirtualMGPU;
	#define SGPU_CONSTEXPR
#else
	#define MAX_NUM_GPUS 1
	#define GNumExplicitGPUsForRendering 1
	#define GVirtualMGPU 0
	#define SGPU_CONSTEXPR GPUMASK_CONSTEXPR
#endif

/** A mask where each bit is a GPU index. Can not be empty so that non SLI platforms can optimize it to be always 1.  */
struct FRHIGPUMask
{
private:
#if WITH_MGPU
	uint32 GPUMask;
	inline GPUMASK_CONSTEXPR uint32 GetMask() const
	{
		return GPUMask;
	}
#else
	inline constexpr uint32 GetMask() const
	{
		return 1;
	}
#endif


#if WITH_MGPU
	inline explicit GPUMASK_CONSTEXPR FRHIGPUMask(uint32 InGPUMask)
		: GPUMask(InGPUMask)
	{
		checkSlow(InGPUMask != 0);
	}
#else
	inline explicit GPUMASK_CONSTEXPR FRHIGPUMask(uint32 InGPUMask)
	{
		checkSlow(InGPUMask == 1);
	}
#endif

public:
	inline GPUMASK_CONSTEXPR FRHIGPUMask()
		: FRHIGPUMask(FRHIGPUMask::GPU0())
	{
	}

#if WITH_MGPU
	inline uint32 ToIndex() const
	{
		checkSlow(HasSingleIndex());
		return FMath::CountTrailingZeros(GetMask());
	}

	inline SGPU_CONSTEXPR bool HasSingleIndex() const
	{
		return FMath::IsPowerOfTwo(GetMask());
	}

	inline SGPU_CONSTEXPR uint32 GetNumActive() const
	{
		return FPlatformMath::CountBits(GetMask());
	}

	inline SGPU_CONSTEXPR uint32 GetLastIndex() const
	{
		return FPlatformMath::FloorLog2(GetMask());
	}

	inline SGPU_CONSTEXPR uint32 GetFirstIndex() const
	{
		return FPlatformMath::CountTrailingZeros(GetMask());
	}
#else
	inline constexpr uint32 ToIndex() const
	{
		return 0;
	}

	inline constexpr bool HasSingleIndex() const
	{
		return true;
	}

	inline constexpr uint32 GetNumActive() const
	{
		return 1;
	}

	inline constexpr uint32 GetLastIndex() const
	{
		return 0;
	}

	inline constexpr uint32 GetFirstIndex() const
	{
		return 0;
	}
#endif

	inline SGPU_CONSTEXPR bool Contains(uint32 GPUIndex) const
	{
		return (GetMask() & (1 << GPUIndex)) != 0;
	}

	inline SGPU_CONSTEXPR bool ContainsAll(FRHIGPUMask Rhs) const
	{
		return (GetMask() & Rhs.GetMask()) == Rhs.GetMask();
	}

	inline SGPU_CONSTEXPR bool Intersects(FRHIGPUMask Rhs) const
	{
		return (GetMask() & Rhs.GetMask()) != 0;
	}

	inline SGPU_CONSTEXPR bool operator ==(FRHIGPUMask Rhs) const
	{
		return GetMask() == Rhs.GetMask();
	}

	inline SGPU_CONSTEXPR bool operator !=(FRHIGPUMask Rhs) const
	{
		return GetMask() != Rhs.GetMask();
	}

	inline SGPU_CONSTEXPR void operator |=(FRHIGPUMask Rhs)
	{
#if WITH_MGPU
		GPUMask |= Rhs.GetMask();
#endif
	}

	inline SGPU_CONSTEXPR void operator &=(FRHIGPUMask Rhs)
	{
#if WITH_MGPU
		GPUMask &= Rhs.GetMask();
#endif
	}

	inline SGPU_CONSTEXPR uint32 GetNative() const
	{
		return GVirtualMGPU ? 1 : GetMask();
	}

	// Direct use of the internal mask is discouraged, but it can be useful for debugging to display
	inline SGPU_CONSTEXPR uint32 GetForDisplay() const
	{
		return GetMask();
	}

	inline SGPU_CONSTEXPR FRHIGPUMask operator &(FRHIGPUMask Rhs) const
	{
		return FRHIGPUMask(GetMask() & Rhs.GetMask());
	}

	inline SGPU_CONSTEXPR FRHIGPUMask operator |(FRHIGPUMask Rhs) const
	{
		return FRHIGPUMask(GetMask() | Rhs.GetMask());
	}

	inline static GPUMASK_CONSTEXPR FRHIGPUMask FromIndex(uint32 GPUIndex)
	{
		return FRHIGPUMask(1 << GPUIndex);
	}

	inline static GPUMASK_CONSTEXPR FRHIGPUMask GPU0()
	{
		return FRHIGPUMask(1);
	}
	
	inline static SGPU_CONSTEXPR FRHIGPUMask All()
	{
		return FRHIGPUMask((1 << GNumExplicitGPUsForRendering) - 1);
	}
	
	inline static SGPU_CONSTEXPR FRHIGPUMask FilterGPUsBefore(uint32 GPUIndex)
	{
		return FRHIGPUMask(~((1u << GPUIndex) - 1)) & All();
	}

	// Inverts a GPU mask, returning true if the inverse succeeded.  If it fails, OutInverse is arbitrarily set to GPU0.
	inline bool Invert(FRHIGPUMask& OutInverse) const
	{
		if (*this == All())
		{
			OutInverse = FRHIGPUMask::GPU0();
			return false;
		}
		else
		{
			OutInverse = FRHIGPUMask(~GetMask()) & All();
			return true;
		}
	}

	struct FIterator
	{
		inline explicit FIterator(const uint32 InGPUMask)
			: GPUMask(InGPUMask)
#if WITH_MGPU
			, FirstGPUIndexInMask(FPlatformMath::CountTrailingZeros(InGPUMask))
#endif
		{
		}

		inline explicit FIterator(FRHIGPUMask InGPUMask)
			: FIterator(InGPUMask.GetMask())
		{
		}

		inline FIterator& operator++()
		{
#if WITH_MGPU
			GPUMask &= ~(1 << FirstGPUIndexInMask);
			FirstGPUIndexInMask = FPlatformMath::CountTrailingZeros(GPUMask);
#else
			GPUMask = 0;
#endif
			return *this;
		}

		inline FIterator operator++(int)
		{
			FIterator Copy(*this);
			++*this;
			return Copy;
		}

		inline uint32 operator*() const
		{
			return GetFirstIndexInMask();
		}

		inline bool operator !=(const FIterator& Rhs) const
		{
			return GetMask() != Rhs.GetMask();
		}

		inline explicit operator bool() const
		{
			return GetMask() != 0;
		}

		inline bool operator !() const
		{
			return !(bool)*this;
		}

	private:
		// NOTE: we cannot remove this in single GPU mode since we need to actually iterate once.
		uint32 GPUMask;

		inline uint32 GetMask() const
		{
			return GPUMask;
		}

#if WITH_MGPU
		uint32 FirstGPUIndexInMask;
		inline uint32 GetFirstIndexInMask() const
		{
			return FirstGPUIndexInMask;
		}
#else
		inline constexpr uint32 GetFirstIndexInMask() const
		{
			return 0;
		}
#endif
	};

	inline friend FRHIGPUMask::FIterator begin(FRHIGPUMask NodeMask)
	{
		return FRHIGPUMask::FIterator(NodeMask.GetMask());
	}

	inline friend FRHIGPUMask::FIterator end(FRHIGPUMask NodeMask)
	{
		return FRHIGPUMask::FIterator(0);
	}
};
