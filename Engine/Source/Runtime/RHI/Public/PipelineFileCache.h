// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Containers/List.h"
#include "Containers/MpscQueue.h"
#include "Containers/StringView.h"
#include "RHI.h"
#include "RHIShaderBindingLayout.h"
#include "PipelineStateCache.h"

DECLARE_STATS_GROUP(TEXT("ShaderPipelineCache"),STATGROUP_PipelineStateCache, STATCAT_Advanced);

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Total Graphics Pipeline State Count"), STAT_TotalGraphicsPipelineStateCount, STATGROUP_PipelineStateCache, RHI_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Total Compute Pipeline State Count"), STAT_TotalComputePipelineStateCount, STATGROUP_PipelineStateCache, RHI_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Total RayTracing Pipeline State Count"), STAT_TotalRayTracingPipelineStateCount, STATGROUP_PipelineStateCache, RHI_API);

#define PIPELINE_CACHE_DEFAULT_ENABLED (!WITH_EDITOR)

enum class EPSOPrecacheResult;

/**
 * PSO_COOKONLY_DATA
 * - Is a transitory data area that should only be used during the cook and stable pipeline cache file generation processes.
 * - If def'ing it out in GAME builds helps to reduce confusion as to where the actual data resides
 * - Should not be serialized or used in comparsion operations (e.g. UsageMask: PSO need to be able to compare equal with different Masks during cook).
 */
#define PSO_COOKONLY_DATA (WITH_EDITOR || IS_PROGRAM)

struct FPipelineFileCacheRasterizerState
{
	FPipelineFileCacheRasterizerState() { FMemory::Memzero(*this); }
	FPipelineFileCacheRasterizerState(FRasterizerStateInitializerRHI const& Other) { operator=(Other); }
	
	float DepthBias;
	float SlopeScaleDepthBias;
	TEnumAsByte<ERasterizerFillMode> FillMode;
	TEnumAsByte<ERasterizerCullMode> CullMode;
	ERasterizerDepthClipMode DepthClipMode;
	bool bAllowMSAA;
	
	FPipelineFileCacheRasterizerState& operator=(FRasterizerStateInitializerRHI const& Other)
	{
		DepthBias = Other.DepthBias;
		SlopeScaleDepthBias = Other.SlopeScaleDepthBias;
		FillMode = Other.FillMode;
		CullMode = Other.CullMode;
		DepthClipMode = Other.DepthClipMode;
		bAllowMSAA = Other.bAllowMSAA;
		return *this;
	}
	
	operator FRasterizerStateInitializerRHI() const
	{
		FRasterizerStateInitializerRHI Initializer(FillMode, CullMode, DepthBias, SlopeScaleDepthBias, DepthClipMode, bAllowMSAA);
		return Initializer;
	}
	
	friend RHI_API FArchive& operator<<(FArchive& Ar, FPipelineFileCacheRasterizerState& RasterizerStateInitializer);

	friend uint32 GetTypeHash(const FPipelineFileCacheRasterizerState &Key)
	{
		uint32 KeyHash = (*((uint32*)&Key.DepthBias) ^ *((uint32*)&Key.SlopeScaleDepthBias));
		KeyHash ^= (Key.FillMode << 8);
		KeyHash ^= Key.CullMode;
		KeyHash ^= Key.DepthClipMode == ERasterizerDepthClipMode::DepthClamp ? 0x951f4c3b : 0; // crc32 "DepthClamp"
		KeyHash ^= Key.bAllowMSAA ? 0x694ea601 : 0; // crc32 "bAllowMSAA"
		return KeyHash;
	}
	RHI_API FString ToString() const;
	RHI_API void FromString(const FStringView& Src);
};

class FRayTracingPipelineStateInitializer;
class FRHIRayTracingShader;
enum class ERayTracingPipelineCacheFlags : uint8;

/**
 * Tracks stats for the current session between opening & closing the file-cache.
 */
struct FPipelineStateStats
{
	FPipelineStateStats()
	: FirstFrameUsed(-1)
	, LastFrameUsed(-1)
	, CreateCount(0)
	, TotalBindCount(0)
	, PSOHash(0)
	{
	}
	
	~FPipelineStateStats() = default;

	RHI_API static void UpdateStats(FPipelineStateStats* Stats);
	
	friend FArchive& operator<<( FArchive& Ar, FPipelineStateStats& Info );

	int64 FirstFrameUsed;
	int64 LastFrameUsed;
	uint64 CreateCount;
	int64 TotalBindCount;
	uint32 PSOHash;
};

struct FPipelineCacheFileFormatPSO
{
	using TReadableStringBuilder = TStringBuilder<1024>;

	struct ComputeDescriptor
	{
		FShaderHash ComputeShader;

		RHI_API FString ToString() const;
		void AddToReadableString(TReadableStringBuilder& OutBuilder) const;
		static FString HeaderLine();
		RHI_API void FromString(const FStringView& Src);
	};
	struct GraphicsDescriptor
	{
		FShaderHash VertexShader;
		FShaderHash FragmentShader;
		FShaderHash GeometryShader;
		FShaderHash MeshShader;
		FShaderHash AmplificationShader;

		FVertexDeclarationElementList VertexDescriptor;
		FBlendStateInitializerRHI BlendState;
		FPipelineFileCacheRasterizerState RasterizerState;
		FDepthStencilStateInitializerRHI DepthStencilState;
		
		EPixelFormat RenderTargetFormats[MaxSimultaneousRenderTargets];
		ETextureCreateFlags RenderTargetFlags[MaxSimultaneousRenderTargets];
		uint32 RenderTargetsActive;
		uint32 MSAASamples;
		
		EPixelFormat DepthStencilFormat;
		ETextureCreateFlags DepthStencilFlags;
		ERenderTargetLoadAction DepthLoad;
		ERenderTargetLoadAction StencilLoad;
		ERenderTargetStoreAction DepthStore;
		ERenderTargetStoreAction StencilStore;
		
		EPrimitiveType PrimitiveType;
		
		uint8 SubpassHint;	
		uint8 SubpassIndex;

		uint8	MultiViewCount;
		bool	bHasFragmentDensityAttachment;

		bool	bAllowVariableRateShading;
		EVRSShadingRate ShadingRate;

		bool	bDepthBounds;
		
		RHI_API FString ToString() const;
		RHI_API void AddToReadableString(TReadableStringBuilder& OutBuilder) const;
		RHI_API static FString HeaderLine();
		RHI_API bool FromString(const FStringView& Src);

		RHI_API FString ShadersToString() const;
		RHI_API void AddShadersToReadableString(TReadableStringBuilder& OutBuilder) const;

		RHI_API static FString ShaderHeaderLine();
		RHI_API void ShadersFromString(const FStringView& Src);

		RHI_API FString StateToString() const;
		RHI_API void AddStateToReadableString(TReadableStringBuilder& OutBuilder) const;
		RHI_API static FString StateHeaderLine();
		RHI_API bool StateFromString(const FStringView& Src);

		/** Not all RT flags make sense for the replayed PSO, only those that can influence the RT formats */
		static ETextureCreateFlags ReduceRTFlags(ETextureCreateFlags InFlags);

		/** Not all DepthStencil flags make sense for the replayed PSO, only those that can influence the actual format */
		static ETextureCreateFlags ReduceDSFlags(ETextureCreateFlags InFlags);
	};
	struct FPipelineFileCacheRayTracingDesc
	{
		FShaderHash ShaderHash;
		uint32 DeprecatedMaxPayloadSizeInBytes = 0;
		EShaderFrequency Frequency = SF_RayGen;
		FRHIShaderBindingLayout ShaderBindingLayout;
		
		FPipelineFileCacheRayTracingDesc() = default;
		FPipelineFileCacheRayTracingDesc(const FRayTracingPipelineStateInitializer& Initializer, const FRHIRayTracingShader* ShaderRHI);

		RHI_API FString ToString() const;
		void AddToReadableString(TReadableStringBuilder& OutBuilder) const;
		FString HeaderLine() const;
		RHI_API void FromString(const FString& Src);

		friend uint32 GetTypeHash(const FPipelineFileCacheRayTracingDesc& Desc)
		{
			return HashCombineFast(GetTypeHash(Desc.ShaderHash), HashCombineFast(GetTypeHash(Desc.Frequency), GetTypeHash(Desc.ShaderBindingLayout)));
		}

		bool operator == (const FPipelineFileCacheRayTracingDesc& Other) const
		{
			return ShaderHash == Other.ShaderHash &&
				Frequency == Other.Frequency &&
				ShaderBindingLayout == Other.ShaderBindingLayout;
		}
	};
	enum class DescriptorType : uint32
	{
		Compute = 0,
		Graphics = 1,
		RayTracing = 2,
	};
	
	DescriptorType Type;
	ComputeDescriptor ComputeDesc;
	GraphicsDescriptor GraphicsDesc;
	FPipelineFileCacheRayTracingDesc RayTracingDesc;
	EPSOPrecacheResult PSOPrecacheResult = EPSOPrecacheResult::Unknown;

#if PSO_COOKONLY_DATA
	uint64 UsageMask;
	int64 BindCount;
#endif
	
	RHI_API FPipelineCacheFileFormatPSO();
	RHI_API ~FPipelineCacheFileFormatPSO();
	
	RHI_API FPipelineCacheFileFormatPSO& operator=(const FPipelineCacheFileFormatPSO& Other);
	RHI_API FPipelineCacheFileFormatPSO(const FPipelineCacheFileFormatPSO& Other);
	
	RHI_API bool operator==(const FPipelineCacheFileFormatPSO& Other) const;

	friend RHI_API uint32 GetTypeHash(const FPipelineCacheFileFormatPSO &Key);
	friend RHI_API FArchive& operator<<( FArchive& Ar, FPipelineCacheFileFormatPSO& Info );
	
	static bool Init(FPipelineCacheFileFormatPSO& PSO, FComputePipelineStateInitializer const& Init, EPSOPrecacheResult PSOPrecacheResult);
	static bool Init(FPipelineCacheFileFormatPSO& PSO, FGraphicsPipelineStateInitializer const& Init, EPSOPrecacheResult PSOPrecacheResult);
	static bool Init(FPipelineCacheFileFormatPSO & PSO, FPipelineFileCacheRayTracingDesc const& Desc);

	RHI_API static void SortVertexElements(FVertexDeclarationElementList& VertexElements);

	RHI_API FString CommonToString() const;
	RHI_API static FString CommonHeaderLine();
	RHI_API void CommonFromString(const FStringView& Src);

	/** Prints out human-readable representation of the PSO, for any type */
	RHI_API FString ToStringReadable() const;
	
	// Potential cases for seperating verify logic if requiired: RunTime-Logging, RunTime-UserCaching, RunTime-PreCompile, CommandLet-Cooking
	RHI_API bool Verify() const;
};

struct FPipelineCacheFileFormatPSORead
{	
	FPipelineCacheFileFormatPSORead()
	: Ar(nullptr)
	, Hash(0)
	, bReadCompleted(false)
	, bValid(false)
	{}
	
	~FPipelineCacheFileFormatPSORead()
	{
		if(Ar != nullptr)
		{
			delete Ar;
			Ar = nullptr;
		}
	}
	
	TArray<uint8> Data;
	FArchive* Ar;
	
	uint32 Hash;
	bool bReadCompleted;
	bool bValid;
	
	// Note that the contract of IAsyncReadFileHandle and IAsyncReadRequest requires that we delete the ReadRequest before deleting its ParentFileHandle. 
	// We therefore require that ParentFileHandle is declared before ReadRequest, so that the class destructor tears down first ReadRequest then ParentFileHandle.
	TSharedPtr<class IAsyncReadFileHandle, ESPMode::ThreadSafe> ParentFileHandle;
	TSharedPtr<class IAsyncReadRequest, ESPMode::ThreadSafe> ReadRequest;
};

struct FPipelineCachePSOHeader
{
	TSet<FShaderHash> Shaders;
	uint32 Hash;
};

extern RHI_API const uint32 FPipelineCacheFileFormatCurrentVersion;

/*
 * User definable Mask Comparsion function:
 * @param ReferenceMask is the Current Bitmask set via SetGameUsageMask
 * @param PSOMask is the PSO UsageMask
 * @return Function should return true if this PSO is to be precompiled or false otherwise
 */
typedef bool(*FPSOMaskComparisonFn)(uint64 ReferenceMask, uint64 PSOMask);

/**
 * FPipelineFileCacheManager:
 * The RHI-level backend for FShaderPipelineCache, responsible for tracking PSOs and their usage stats as well as dealing with the pipeline cache files.
 * It is not expected that games or end-users invoke this directly, they should be calling FShaderPipelineCache which exposes this functionality in a usable form. 
 */

struct FPSOUsageData
{
	FPSOUsageData(): UsageMask(0), PSOHash(0), EngineFlags(0) {}
	FPSOUsageData(uint32 InPSOHash, uint64 InUsageMask, uint16 InEngineFlags): UsageMask(InUsageMask), PSOHash(InPSOHash), EngineFlags(InEngineFlags) {}
	uint64 UsageMask;
	uint32 PSOHash;
	uint16 EngineFlags;
};

class FPipelineFileCacheManager
{
    friend class FPipelineCacheFile;
public:
	enum class SaveMode : uint32
	{
		Incremental = 0, // Fast(er) approach which saves new entries incrementally at the end of the file, replacing the table-of-contents, but leaves everything else alone.
        BoundPSOsOnly = 1, // Slower approach which consolidates and saves all PSOs used in this run of the program, removing any entry that wasn't seen, and sorted by the desired sort-mode.
	};
	
	enum class PSOOrder : uint32
	{
		Default = 0, // Whatever order they are already in.
		FirstToLatestUsed = 1, // Start with the PSOs with the lowest first-frame used and work toward those with the highest.
		MostToLeastUsed = 2 // Start with the most often used PSOs working toward the least.
	};

public:
	
	RHI_API static void Initialize(uint32 GameVersion);
	RHI_API static void Shutdown();
	
	RHI_API static bool LoadPipelineFileCacheInto(FString const& Path, TSet<FPipelineCacheFileFormatPSO>& PSOs);
	RHI_API static bool SavePipelineFileCacheFrom(uint32 GameVersion, EShaderPlatform Platform, FString const& Path, const TSet<FPipelineCacheFileFormatPSO>& PSOs);
	RHI_API static bool MergePipelineFileCaches(FString const& PathA, FString const& PathB, FPipelineFileCacheManager::PSOOrder Order, FString const& OutputPath);
																				
	/* Open the pipeline file cache for the specfied name and platform. If successful, the GUID of the game file will be returned in OutGameFileGuid */
	RHI_API static bool OpenPipelineFileCache(const FString& Key, const FString& CacheName, EShaderPlatform Platform, FGuid& OutGameFileGuid);

	/* Open the user pipeline file cache for the specified name and platform. The user cache is always created even if the file was not present when opened.
	* Name is the name used when opening the file, the key value for the user cache is held within UserCacheName.
	* returns true if the file was opened.
	*/
	RHI_API static bool OpenUserPipelineFileCache(const FString& Key, const FString& CacheName, EShaderPlatform Platform, FGuid& OutGameFileGuid);

	RHI_API static bool SavePipelineFileCache(SaveMode Mode);

	RHI_API static void CloseUserPipelineFileCache();

	RHI_API static void CacheGraphicsPSO(uint32 RunTimeHash, FGraphicsPipelineStateInitializer const& Initializer, EPSOPrecacheResult PSOPrecacheResult, FPipelineStateStats** OutStats = nullptr);
	RHI_API static void CacheComputePSO(uint32 RunTimeHash, FComputePipelineStateInitializer const& Initializer, EPSOPrecacheResult PSOPrecacheResult, FPipelineStateStats** OutStats = nullptr);
	RHI_API static void CacheRayTracingPSO(const FRayTracingPipelineStateInitializer& Initializer, ERayTracingPipelineCacheFlags Flags);

	// true if the named PSOFC is currently open.
	RHI_API static bool HasPipelineFileCache(const FString& PSOCacheKey);

	/*
	 * This PSO has failed compile and is invalid - this cache should not return this invalid PSO from subsequent calls for PreCompile requests.
	 * Note: Not implementated for Compute that has no flag to say it came from this cache - don't want to consume failures that didn't propagate from this cache.
	 */
	RHI_API static void RegisterPSOCompileFailure(uint32 RunTimeHash, FGraphicsPipelineStateInitializer const& Initializer);
	
	/**
	 * Event signature for being notified that a new PSO has been logged
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FPipelineStateLoggedEvent, const FPipelineCacheFileFormatPSO&);

	/**
	 * Gets the event delegate to register for pipeline state logging events.
	 */
	RHI_API static FPipelineStateLoggedEvent& OnPipelineStateLogged();

	/*
	 * If the delegate is set, broadcasts any new PSOs that were encountered since the last time the delegate was broadcast.
	 * The broadcast is scheduled to be run on the game thread.
	 */
	RHI_API static void BroadcastNewPSOsDelegate();
	
	RHI_API static void GetOrderedPSOHashes(const FString& PSOCacheKey, TArray<FPipelineCachePSOHeader>& PSOHashes, PSOOrder Order, int64 MinBindCount, TSet<uint32> const& AlreadyCompiledHashes);
	RHI_API static void FetchPSODescriptors(const FString& PSOCacheKey, TDoubleLinkedList<FPipelineCacheFileFormatPSORead*>& LoadedBatch);

	RHI_API static int32 GetTotalPSOCount(const FString& PSOCacheKey);

	RHI_API static uint32 NumPSOsLogged();
	
	RHI_API static bool IsPipelineFileCacheEnabled();
	RHI_API static bool LogPSOtoFileCache();
	RHI_API static bool ReportNewPSOs();
	/* Report additional data about new PSOs to the log. */
	RHI_API static bool LogPSODetails();
	
	/**
	 * Define the Current Game Usage Mask and a comparison function to compare this mask against the recorded mask in each PSO
	 * @param GameUsageMask Current Game Usage Mask to set, typically from user quality settings
	 * @param InComparisonFnPtr Pointer to the comparsion function - see above FPSOMaskComparisonFn definition for details
	 * @returns the old mask
	 */
	RHI_API static uint64 SetGameUsageMaskWithComparison(uint64 GameUsageMask, FPSOMaskComparisonFn InComparisonFnPtr);
	static uint64 GetGameUsageMask() { return GameUsageMask; }
	static bool IsGameUsageMaskSet() { return GameUsageMaskSet; }
	
	RHI_API static void PreCompileComplete();

	/*
	 * Enable or disable the logging of new PSOs (PSOs that were needed but not found in the file cache) to console and CSV.
	 */
	static void SetNewPSOConsoleAndCSVLogging(bool bEnabled) { LogNewPSOsToConsoleAndCSV = bEnabled; }
private:
	
	static void RegisterPSOUsageDataUpdateForNextSave(FPSOUsageData& UsageData);
	static void ClearOSPipelineCache();
	static bool ShouldEnableFileCache();
	
	static bool IsBSSEquivalentPSOEntryCached(FPipelineCacheFileFormatPSO const& NewEntry);
	static bool IsPSOEntryCached(FPipelineCacheFileFormatPSO const& NewEntry, FPSOUsageData* EntryData = nullptr);

	static void LogNewGraphicsPSOToConsoleAndCSV(FPipelineCacheFileFormatPSO& PSO, uint32 PSOHash, EPSOPrecacheResult PSOPrecacheResult);
	static void LogNewComputePSOToConsoleAndCSV(FPipelineCacheFileFormatPSO& PSO, uint32 PSOHash, EPSOPrecacheResult PSOPrecacheResult);
	static void LogNewRaytracingPSOToConsole(FPipelineCacheFileFormatPSO& PSO, uint32 PSOHash, bool bIsNonBlockingPSO);

	static FPipelineStateStats* RegisterPSOStatsInternal(class FRWScopeLock& Lock, uint32 RunTimeHash);
	
private:
	static FRWLock FileCacheLock;

	// Containers for the multiple bundled PSOFCs
	// Name to PipelineCacheFile
	static TMap<FString, TUniquePtr<class FPipelineCacheFile>> FileCacheMap;
	// PipelineCacheFile GUID to Name
	static TMap<FGuid, FString> GameGuidToCacheKey;
	// User cache's key within FileCacheMap
	static FString UserCacheKey;
	// Helper for retrieving a file cache from the name.
	static class FPipelineCacheFile* GetPipelineCacheFileFromKey(const FString& PSOCacheKey)
	{
		TUniquePtr<FPipelineCacheFile>* FileCacheFound = FileCacheMap.Find(PSOCacheKey);
		return FileCacheFound ? FileCacheFound->Get() : nullptr;
	}

	// PSO recording
	static TMap<uint32, FPSOUsageData> RunTimeToPSOUsage;		// Fast check structure - Not saved (External state cache runtime hash to seen usage data)
	static TMap<uint32, FPSOUsageData> NewPSOUsage;				// For mask or engine updates - Merged + Saved (Our internal PSO hash to latest usage data) - temp working scratch, only holds updates since last "save" so is not the authority on state
	static TMap<uint32, FPipelineStateStats*> Stats;
	static TMap<uint32, FPipelineCacheFileFormatPSO> NewPSOs;
	static TMpscQueue<FPipelineCacheFileFormatPSO> NewPSOsToReport; // New PSOs that will be broadcast via a delegate (if bound). Cleared when the delegate is broadcast.
	static uint32 NumNewPSOs;
	static PSOOrder RequestedOrder;
	static bool FileCacheEnabled;
	static FPipelineStateLoggedEvent PSOLoggedEvent;
	RHI_API static uint64 GameUsageMask;
	RHI_API static bool GameUsageMaskSet;
	static FPSOMaskComparisonFn MaskComparisonFn;	

	RHI_API static bool LogNewPSOsToConsoleAndCSV; // Whether to log new PSOs to the log file and CSV.
};
