// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PipelineStateCache.h: Pipeline state cache definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "Misc/Timeout.h"
#include "Misc/EnumClassFlags.h"

class FComputePipelineState;
class FGraphicsPipelineState;
class FRayTracingPipelineState;
class FWorkGraphPipelineState;

// Utility flags for modifying render target behavior on a PSO
enum class EApplyRendertargetOption : int
{
	DoNothing = 0,			// Just use the PSO from initializer's values, no checking and no modifying (used for PSO precompilation only)
	CheckApply = 1 << 0,	// Verify that the PSO's RT formats match the last Render Target formats set into the CmdList
	ForceApply = CheckApply,// Deprecated. Do not use
};

ENUM_CLASS_FLAGS(EApplyRendertargetOption);

/**
 * PSO Precache request priority
 */
enum class EPSOPrecachePriority : uint8
{
	Low = 0,
	Medium,
	High,
	Highest,
	Num,
	NotSet = Num,
};

/**
 * PSO Precache InMemory mode
 */
enum class EPSOPrecacheInMemoryMode : uint8
{
	Cached,		//< Keep precached PSO in memory
	Uncached,	//< Don't keep precached PSO in memory (release after async compile)
};

/**
* PSO Precache parameters provided to the graphics and compute precache function
*/
struct FPSOPrecacheRequestParams
{
	EPSOPrecachePriority PrecachePriority = EPSOPrecachePriority::Medium;
	EPSOPrecacheInMemoryMode InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;	
	FString ResourceName;
	const TCHAR* CollectorName = nullptr;
	bool bForcePrecache = false;
};

enum class ERayTracingPipelineCacheFlags : uint8
{
	// Query the pipeline cache, create pipeline if necessary.
	// Compilation may happen on a task, but RHIThread will block on it before translating the RHICmdList.
	// Therefore the RHIThread may stall when creating large / complex pipelines.
	Default = 0,

	// Query the pipeline cache, create a background task to create the pipeline if necessary.
	// GetAndOrCreateRayTracingPipelineState() may return NULL if pipeline is not ready.
	// Caller must use an alternative fallback PSO to render current frame and may retry next frame.
	// Pipeline creation task will not block RenderThread or RHIThread, allowing hitch-free rendering.
	NonBlocking = 1 << 0,
};
ENUM_CLASS_FLAGS(ERayTracingPipelineCacheFlags);

enum class EPSOPrecacheResult
{
	Unknown,			//< No known pre cache state
	Active,				//< PSO is currently precaching
	Complete,			//< PSO has been precached successfully
	Missed,				//< PSO precache miss, needs to be compiled at draw time
	TooLate,			//< PSO precache request still compiling when needed
	NotSupported,		//< PSO precache not supported (VertexFactory or MeshPassProcessor doesn't support/implement precaching)
	Untracked,			//< PSO is not tracked at all (Global shader or not coming from MeshDrawCommands)
};

extern RHI_API const TCHAR* LexToString(EPSOPrecacheResult Result);

// Unique request ID of PSOPrecache which can be used to boost the priority of a PSO precache requests if it's needed for rendering
struct FPSOPrecacheRequestID
{
	enum class EType
	{
		Invalid = 0,
		Graphics,
		Compute
	};

	FPSOPrecacheRequestID() : Type((uint32)EType::Invalid), RequestID(0) { }

	EType GetType() const { return (EType)Type; }
	bool IsValid() const 
	{ 		
		return Type != (uint32)EType::Invalid; 
	}
	bool operator==(const FPSOPrecacheRequestID& Other) const
	{
		return Type == Other.Type && RequestID == Other.RequestID;
	}
	bool operator!=(const FPSOPrecacheRequestID& rhs) const
	{
		return !(*this == rhs);
	}

	uint32 Type : 2;			//< PSO request type
	uint32 RequestID : 30;		//< Unique request ID
};

// Result data of a precache pipeline state request
struct FPSOPrecacheRequestResult
{
	bool IsValid() const { return RequestID.IsValid(); }
	bool operator==(const FPSOPrecacheRequestResult& Other) const
	{
		return RequestID == Other.RequestID && AsyncCompileEvent == Other.AsyncCompileEvent;
	}
	bool operator!=(const FPSOPrecacheRequestResult& rhs) const
	{
		return !(*this == rhs);
	}

	FPSOPrecacheRequestID RequestID;
	FGraphEventRef AsyncCompileEvent;
};

// Statistics about PSOs created at runtime, including stats for PSOs which took over a pre-defined threshold to be created. 
// These only track PSOs requested by the renderer on the critical path (not from background precaching).
// Note that while slow PSO creation times usually result in a hitch at runtime, it's not a 1-to-1 correlation.
// There are cases where multiple PSOs are slow in a single frame and therefore cause a single hitch, and cases
// where a PSO might not be slow enough to cause a visible hitch for a certain frame (depending on the threshold).
struct FPSORuntimeCreationStats
{
	// The total number of PSOs created after a request by the renderer. Does not include PSOs created from background precaching.
	uint32 TotalPSOCreations = 0;

	// The number of compute and graphics PSOs that took too long to create.
	uint32 ComputePSOHitches = 0;
	uint32 GraphicsPSOHitches = 0;

	// How many of the PSOs that took too long to create were previously precached in the background.
	uint32 PreviouslyPrecachedPSOHitches = 0;

	// How many of the PSOs that too long to create were previously precached in the background and are suspected to have hit a full compilation again.
	// These use a higher threshold and usually indicate the driver cache is being missed or is not working.
	uint32 SuspectedUnhealthyDriverCachePSOHitches = 0;

	// Whether the driver cache is suspected to be unhealthy due to the number of very long creation times for precached PSO (controlled by a configurable threshold).
	bool bDriverCacheSuspectedUnhealthy = false;
};

extern RHI_API void SetComputePipelineState(FRHIComputeCommandList& RHICmdList, const FComputePipelineStateInitializer& Initializer);
extern RHI_API void SetGraphicsPipelineStateCheckApply(FRHICommandList& RHICmdList, const FGraphicsPipelineStateInitializer& Initializer, uint32 StencilRef, bool bApplyAdditionalState = true);
extern RHI_API void SetGraphicsPipelineState(FRHICommandList& RHICmdList, const FGraphicsPipelineStateInitializer& Initializer, uint32 StencilRef, EApplyRendertargetOption ApplyFlags = EApplyRendertargetOption::CheckApply, bool bApplyAdditionalState = true);

//UE_DEPRECATED(5.8, "Identifying compute pipelines specifying only a shader is deprecated, use a FComputePipelineStateInitializer.")
inline void SetComputePipelineState(FRHIComputeCommandList& RHICmdList, FRHIComputeShader* ComputeShader)
{
	SetComputePipelineState(RHICmdList, FComputePipelineStateInitializer(ComputeShader));
}

namespace PipelineStateCache
{
	extern RHI_API uint64					RetrieveGraphicsPipelineStateSortKey(const FGraphicsPipelineState* GraphicsPipelineState);

	extern RHI_API FComputePipelineState*	GetAndOrCreateComputePipelineState(FRHIComputeCommandList& RHICmdList, const FComputePipelineStateInitializer& Initializer);

	//UE_DEPRECATED(5.8, "Identifying compute pipelines specifying only a shader is deprecated, use a FComputePipelineStateInitializer.")
	inline FComputePipelineState* GetAndOrCreateComputePipelineState(FRHIComputeCommandList& RHICmdList, FRHIComputeShader* ComputeShader, bool bFromFileCache)
	{
		FComputePipelineStateInitializer ComputeInitializer(ComputeShader);
		ComputeInitializer.bFromPSOFileCache = bFromFileCache;
		return GetAndOrCreateComputePipelineState(RHICmdList, ComputeInitializer);
	}

	extern RHI_API FWorkGraphPipelineState* GetAndOrCreateWorkGraphPipelineState(FRHIComputeCommandList& RHICmdList, const FWorkGraphPipelineStateInitializer& Initializer);

	extern RHI_API FGraphicsPipelineState*	GetAndOrCreateGraphicsPipelineState(FRHICommandList& RHICmdList, const FGraphicsPipelineStateInitializer& OriginalInitializer, EApplyRendertargetOption ApplyFlags);

	extern RHI_API FComputePipelineState*	FindComputePipelineState(const FComputePipelineStateInitializer& Initializer, bool bVerifyUse = true);

	//UE_DEPRECATED(5.8, "Identifying compute pipelines specifying only a shader is deprecated, use a FComputePipelineStateInitializer.")
	inline FComputePipelineState* FindComputePipelineState(FRHIComputeShader* ComputeShader, bool bVerifyUse)
	{
		return PipelineStateCache::FindComputePipelineState(FComputePipelineStateInitializer(ComputeShader), bVerifyUse);
	}

	extern RHI_API FWorkGraphPipelineState*	FindWorkGraphPipelineState(const FWorkGraphPipelineStateInitializer& Initializer, bool bVerifyUse = true);

	extern RHI_API FGraphicsPipelineState*	FindGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer, bool bVerifyUse = true);

	extern RHI_API void                     GetPipelineStates(TArray<TRefCountPtr<FRHIResource>>& Out, bool bConsolidateCaches = true, UE::FTimeout ConsolidationTimeout = UE::FTimeout::Never());

	extern RHI_API FRHIVertexDeclaration*	GetOrCreateVertexDeclaration(const FVertexDeclarationElementList& Elements);

	// Retrieves RTPSO object from cache or adds a task to create it, which will be waited on by RHI thread.
	// May return NULL in non-blocking mode if pipeline is not already in cache.
	extern RHI_API FRayTracingPipelineState* GetAndOrCreateRayTracingPipelineState(
		FRHICommandList& RHICmdList,
		const FRayTracingPipelineStateInitializer& Initializer,
		ERayTracingPipelineCacheFlags Flags = ERayTracingPipelineCacheFlags::Default);

	// Retrieves RTPSO object from cache or returns NULL if it's not found.
	extern RHI_API FRayTracingPipelineState* GetRayTracingPipelineState(const FRayTracingPipelineStateSignature& Signature);

	/* Evicts unused state entries based on r.pso.evictiontime time. Called in RHICommandList::BeginFrame */
	extern RHI_API void FlushResources();

	extern RHI_API void ReportFrameHitchToCSV();

	// Waits for any pending tasks to complete.
	extern RHI_API void WaitForAllTasks();

	// Initializes any required component.
	extern RHI_API void Init();

	/* Clears all pipeline cached state. Called on shutdown, calling GetAndOrCreate after this will recreate state */
	extern RHI_API void Shutdown();

	/* Called when PSO precompile has completed */
	extern RHI_API void PreCompileComplete();

	/* Returns the number of PSO precompiles currently in progress */
	extern RHI_API int32 GetNumActivePipelinePrecompileTasks();
	
	/* Is precaching currently enabled - can help to skip certain time critical code when precaching is disabled */
	extern RHI_API bool						IsPSOPrecachingEnabled();

	/* Precache the compute shader and return a request ID if precached async */
	extern RHI_API FPSOPrecacheRequestResult PrecacheComputePipelineState(const FComputePipelineStateInitializer& ComputeInitializer, const FPSOPrecacheRequestParams& RequestParams);

	/* Precache the graphics PSO and return an optional graph event if precached async */
	extern RHI_API FPSOPrecacheRequestResult PrecacheGraphicsPipelineState(const FGraphicsPipelineStateInitializer& PipelineStateInitializer, const FPSOPrecacheRequestParams& RequestParams);

	/* Retrieve the current PSO precache result state (slightly slower than IsPrecaching) */
	extern RHI_API EPSOPrecacheResult		CheckPipelineStateInCache(const FGraphicsPipelineStateInitializer& PipelineStateInitializer, FGraphEventRef& OutCompletionEvent, FPSOPrecacheRequestID& OutPSOPrecacheRequestID);	
	inline EPSOPrecacheResult CheckPipelineStateInCache(const FGraphicsPipelineStateInitializer& PipelineStateInitializer)
	{
		FGraphEventRef CompletionEvent;
		FPSOPrecacheRequestID PSOPrecacheRequestID;
		return CheckPipelineStateInCache(PipelineStateInitializer, CompletionEvent, PSOPrecacheRequestID);
	}

	/* Retrieve the current PSO precache result state (slightly slower than IsPrecaching) */
	extern RHI_API EPSOPrecacheResult		CheckPipelineStateInCache(FRHIComputeShader* ComputeShader, FGraphEventRef& OutCompletionEvent, FPSOPrecacheRequestID& OutPSOPrecacheRequestID);
	inline EPSOPrecacheResult CheckPipelineStateInCache(FRHIComputeShader* ComputeShader)
	{
		FGraphEventRef CompletionEvent;
		FPSOPrecacheRequestID PSOPrecacheRequestID;
		return CheckPipelineStateInCache(ComputeShader, CompletionEvent, PSOPrecacheRequestID);
	}
		
	/* Release the precached PSOs - returns true when PSO is queued for deletion, otherwise only the refcount was decremented */
	extern RHI_API bool						ReleasePrecachedPSO(const FPSOPrecacheRequestID& PSORequest);
		
	/* Retrieve the the time it took to compile this precache PSO */
	extern RHI_API uint64					GetPrecachePSOCompilationTime(bool bGraphics, uint64 PSOPrecacheHash);

	/* Is the given PSO still precaching? */
	extern RHI_API bool						IsPrecaching(const FPSOPrecacheRequestID& PSOPrecacheRequestID);

	/* Is the given PSO initializer still precaching? */
	extern RHI_API bool						IsPrecaching(const FGraphicsPipelineStateInitializer& PipelineStateInitializer);

	/* Is the given PSO initializer still precaching? */
	extern RHI_API bool						IsPrecaching(FRHIComputeShader* ComputeShader);

	/* Any async precaching operations still busy */
	extern RHI_API bool						IsPrecaching();

	/* Boost the priority of the given PSO request ID */
	extern RHI_API void						BoostPrecachePriority(EPSOPrecachePriority PSOPrecachePriority, const FPSOPrecacheRequestID& PSOPrecacheRequestID);

	/* Return number of active or pending PSO precache requests */
	extern RHI_API uint32					NumActivePrecacheRequests();

	/* Set all subsequent high priority requests to highest priority, useful in non-interactive scenarios where maximum PSO throughput is preferable. */
	extern RHI_API void						PrecachePSOsBoostToHighestPriority(bool bForceHighest);

	/* Return stats on PSOs created at runtime, including slow PSO creation stats. */
	extern RHI_API FPSORuntimeCreationStats	GetPSORuntimeCreationStats();

	/* Reset the PSO hitch tracking counters */
	extern RHI_API void						ResetPSOHitchTrackingStats();
}

// Returns the shader index within the ray tracing pipeline or INDEX_NONE if given shader does not exist.
// Asserts if shader is not found but bRequired is true.
extern RHI_API int32 FindRayTracingHitGroupIndex(FRayTracingPipelineState* Pipeline, FRHIRayTracingShader* HitGroupShader, bool bRequired = true);
extern RHI_API int32 FindRayTracingCallableShaderIndex(FRayTracingPipelineState* Pipeline, FRHIRayTracingShader* CallableShader, bool bRequired = true);
extern RHI_API int32 FindRayTracingMissShaderIndex(FRayTracingPipelineState* Pipeline, FRHIRayTracingShader* MissShader, bool bRequired = true);

