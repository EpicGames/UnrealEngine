// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Set.h"
#include "Misc/AssertionMacros.h"

/* Implements a Least Recently Used (LRU) cache.
 *
 * @param KeyType The type of cache entry keys.
 * @param ValueType The type of cache entry values.
 */
template<typename KeyType, typename ValueType>
class TPsoLruCache
{
	/** An entry in the LRU cache. */
	struct FCacheEntry
	{
		/** The entry's lookup key. */
		KeyType Key;

		/** The less recent entry in the linked list. */
		FCacheEntry* LessRecent;

		/** The more recent entry in the linked list. */
		FCacheEntry* MoreRecent;

		/** The entry's value. */
		ValueType Value;

		/**
		 * Create and initialize a new instance.
		 *
		 * @param InKey The entry's key.
		 * @param InValue The entry's value.
		 */
		FCacheEntry(const KeyType& InKey, const ValueType& InValue)
			: Key(InKey)
			, LessRecent(nullptr)
			, MoreRecent(nullptr)
			, Value(InValue)
		{ }

		/** Add this entry before the given one. */
		inline void LinkBefore(FCacheEntry* Other)
		{
			LessRecent = Other;

			if (Other != nullptr)
			{
				Other->MoreRecent = this;
			}
		}

		/** Remove this entry from the list. */
		inline void Unlink()
		{
			if (LessRecent != nullptr)
			{
				LessRecent->MoreRecent = MoreRecent;
			}

			if (MoreRecent != nullptr)
			{
				MoreRecent->LessRecent = LessRecent;
			}

			LessRecent = nullptr;
			MoreRecent = nullptr;
		}
	};

	/** Lookup set key functions. */
	struct FKeyFuncs : public BaseKeyFuncs<FCacheEntry*, KeyType>
	{
		inline static const KeyType& GetSetKey(const FCacheEntry* Entry)
		{
			return Entry->Key;
		}

		inline static bool Matches(KeyType A, KeyType B)
		{
			return A == B;
		}

		inline static uint32 GetKeyHash(KeyType Key)
		{
			return GetTypeHash(Key);
		}
	};

public:

	/** Default constructor (empty cache that cannot hold any values). */
	TPsoLruCache()
		: LeastRecent(nullptr)
		, MostRecent(nullptr)
		, MaxNumElements(0)
	{ }

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InMaxNumElements The maximum number of elements this cache can hold.
	 */
	TPsoLruCache(int32 InMaxNumElements)
		: LeastRecent(nullptr)
		, MostRecent(nullptr)
		, MaxNumElements(InMaxNumElements)
	{
		Empty(InMaxNumElements);
	}

	/** Destructor. */
	~TPsoLruCache()
	{
		Empty();
	}

public:

	/**
	 * Add an entry to the cache.
	 *
	 * The new entry must not exist in the cache,
	 * there must be space within the LRU for the new entry.
	 * The new entry will be marked as the most recently used one.
	 *
	 * @param Key The entry's lookup key.
	 * @param Value The entry's value.
	 * @return FSetElementId of the entry, update recent status without requiring a find operation.

	 * @see Empty, Find, GetKeys, Remove
	 */
	FSetElementId Add(const KeyType& Key, const ValueType& Value)
	{
		check(MaxNumElements > 0 && "Cannot add values to zero size FOpenGLLruCache");
		check(!LookupSet.Contains(Key));

		check(LookupSet.Num() < MaxNumElements);
		// add new entry
		FCacheEntry* NewEntry = new FCacheEntry(Key, Value);
		NewEntry->LinkBefore(MostRecent);
		MostRecent = NewEntry;

		if (LeastRecent == nullptr)
		{
			LeastRecent = NewEntry;
		}
		return LookupSet.Add(NewEntry);
	}

	/**
	 * Check whether an entry with the specified key is in the cache.
	 *
	 * @param Key The key of the entry to check.
	 * @return true if the entry is in the cache, false otherwise.
	 * @see Add, ContainsByPredicate, Empty, FilterByPredicate, Find, GetKeys, Remove
	 */
	inline bool Contains(const KeyType& Key) const
	{
		return LookupSet.Contains(Key);
	}

	/**
	 * Check whether an entry for which a predicate returns true is in the cache.
	 *
	 * @param Pred The predicate functor to apply to each entry.
	 * @return true if at least one matching entry is in the cache, false otherwise.
	 * @see Contains, FilterByPredicate, FindByPredicate, RemoveByPredicate
	 */
	template<typename Predicate>
	inline bool ContainsByPredicate(Predicate Pred) const
	{
		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				return true;
			}
		}

		return false;
	}

	/**
	 * Empty the cache.
	 *
	 * @param InMaxNumElements The maximum number of elements this cache can hold (default = 0).
	 * @see Add, Find, GetKeys, Max, Num, Remove
	 */
	void Empty(int32 InMaxNumElements = 0)
	{
		check(InMaxNumElements >= 0);

		for (FCacheEntry* Entry : LookupSet)
		{
			delete Entry;
		}

		MaxNumElements = InMaxNumElements;
		LookupSet.Empty(MaxNumElements);

		MostRecent = nullptr;
		LeastRecent = nullptr;
	}

	/**
	 * Filter the entries in the cache using a predicate.
	 *
	 * @param Pred The predicate functor to apply to each entry.
	 * @return Collection of values for which the predicate returned true.
	 * @see ContainsByPredicate, FindByPredicate, Find, RemoveByPredicate
	 */
	template<typename Predicate>
	TArray<ValueType> FilterByPredicate(Predicate Pred) const
	{
		TArray<ValueType> Result;

		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				Result.Add(Entry->Value);
			}
		}

		return Result;
	}

	/**
	 * Find the value of the entry with the specified key.
	 *
	 * @param Key The key of the entry to get.
	 * @return Pointer to the value, or nullptr if not found.
	 * @see Add, Contains, Empty, FindAndTouch, GetKeys, Remove
	 */
	inline const ValueType* Find(const KeyType& Key) const
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			return &(*EntryPtr)->Value;
		}

		return nullptr;
	}

	/**
	 * Find the value of the entry with the specified key and mark it as the most recently used.
	 *
	 * @param Key The key of the entry to get.
	 * @return Pointer to the value, or nullptr if not found.
	 * @see Add, Contains, Empty, Find, GetKeys, Remove
	 */
	const ValueType* FindAndTouch(const KeyType& Key)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr == nullptr)
		{
			return nullptr;
		}

		MarkAsRecent(**EntryPtr);

		return &(*EntryPtr)->Value;
	}

	/**
	 * Find the value of an entry using a predicate.
	 *
	 * @param Pred The predicate functor to apply to each entry.
	 * @return Pointer to value for which the predicate returned true, or nullptr if not found.
	 * @see ContainsByPredicate, FilterByPredicate, RemoveByPredicate
	 */
	template<typename Predicate>
	const ValueType* FindByPredicate(Predicate Pred) const
	{
		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				return &Entry->Value;
			}
		}

		return nullptr;
	}

	/**
	 * Find the keys of all cached entries.
	 *
	 * @param OutKeys Will contain the collection of keys.
	 * @see Add, Empty, Find
	 */
	void GetKeys(TArray<KeyType>& OutKeys) const
	{
		for (const FCacheEntry* Entry : LookupSet)
		{
			OutKeys.Add(Entry->Key);
		}
	}

	/**
	 * Get the maximum number of entries in the cache.
	 *
	 * @return Maximum number of entries.
	 * @see Empty, Num
	 */
	inline int32 Max() const
	{
		return MaxNumElements;
	}

	/**
	 * Get the number of entries in the cache.
	 *
	 * @return Number of entries.
	 * @see Empty, Max
	 */
	inline int32 Num() const
	{
		return LookupSet.Num();
	}

	/**
	 * Remove all entries with the specified key from the cache.
	 *
	 * @param Key The key of the entries to remove.
	 * @see Add, Empty, Find, RemoveByPredicate
	 */
	void Remove(const KeyType& Key)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			Remove(*EntryPtr);
		}
	}

	bool Remove(const KeyType& Key, ValueType& RemovedValue)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			RemovedValue = MoveTemp((*EntryPtr)->Value);
			Remove(*EntryPtr);
			return true;
		}
		return false;
	}

	/**
	 * Remove all entries using a predicate.
	 *
	 * @param Pred The predicate function to apply to each entry.
	 * @return Number of removed entries.
	 * @see ContainsByPredicate, FilterByPredicate, FindByPredicate, Remove
	 */
	template<typename Predicate>
	int32 RemoveByPredicate(Predicate Pred)
	{
		int32 NumRemoved = 0;

		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				Remove(Entry);
				++NumRemoved;
			}
		}

		return NumRemoved;
	}

	/**
	 * Remove and return the least recent element from the cache.
	 *
	 * @return Copy of removed value.
	 */
	inline ValueType RemoveLeastRecent()
	{
		check(LeastRecent);
		ValueType LeastRecentElement = MoveTemp(LeastRecent->Value);
		Remove(LeastRecent);
		return LeastRecentElement;
	}

	/**
	 * Return the least recent element from the cache.
	 *
	 * @return Copy of the least recent value.
	 */
	inline const ValueType GetLeastRecent() const
	{
		check(LeastRecent);
		return LeastRecent->Value;
	}

	/**
	* Remove and return the most recent element from the cache.
	*
	* @return Copy of removed value.
	*/
	inline ValueType RemoveMostRecent()
	{
		check(MostRecent);
		ValueType MostRecentElement = MoveTemp(MostRecent->Value);
		Remove(MostRecent);
		return MostRecentElement;
	}

	inline void MarkAsRecent(const FSetElementId& LRUNode)
	{
		MarkAsRecent(*LookupSet[LRUNode]);
	}

public:

	/**
	 * Base class for cache iterators.
	 *
	 * Iteration begins at the most recent entry.
	 */
	template<bool Const>
	class TBaseIterator
	{
	public:

		inline TBaseIterator()
			: CurrentEntry(nullptr)
		{ }

		inline TBaseIterator(const TPsoLruCache& Cache)
			: CurrentEntry(Cache.MostRecent)
		{ }

	public:

		inline TBaseIterator& operator++()
		{
			Increment();
			return *this;
		}

		inline friend bool operator==(const TBaseIterator& Lhs, const TBaseIterator& Rhs)
		{
			return Lhs.CurrentEntry == Rhs.CurrentEntry;
		}

		inline friend bool operator!=(const TBaseIterator& Lhs, const TBaseIterator& Rhs)
		{
			return Lhs.CurrentEntry != Rhs.CurrentEntry;
		}

		ValueType& operator->() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Value;
		}

		ValueType& operator*() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Value;
		}

		inline explicit operator bool() const
		{
			return (CurrentEntry != nullptr);
		}

		inline bool operator!() const
		{
			return !(bool)*this;
		}

	public:

		inline KeyType& Key() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Key;
		}

		inline ValueType& Value() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Value;
		}

	protected:

		FCacheEntry* GetCurrentEntry()
		{
			return CurrentEntry;
		}

		void Increment()
		{
			check(CurrentEntry != nullptr);
			CurrentEntry = CurrentEntry->LessRecent;
		}

	private:

		FCacheEntry* CurrentEntry;
	};


	/**
	 * Cache iterator (const).
	 */
	class TConstIterator
		: public TBaseIterator<true>
	{
	public:

		inline TConstIterator()
			: TBaseIterator<true>()
		{ }

		inline TConstIterator(const TPsoLruCache& Cache)
			: TBaseIterator<true>(Cache)
		{ }
	};

	
	/**
	 * Cache iterator.
	 */
	class TIterator
		: public TBaseIterator<false>
	{
	public:

		inline TIterator()
			: TBaseIterator<false>()
			, Cache(nullptr)
		{ }

		inline TIterator(TPsoLruCache& InCache)
			: TBaseIterator<false>(InCache)
			, Cache(&InCache)
		{ }

		/** Removes the current element from the cache and increments the iterator. */
		inline void RemoveCurrentAndIncrement()
		{
			check(Cache != nullptr);

			FCacheEntry* MoreRecentEntry = this->GetCurrentEntry();
			this->Increment();
			Cache->Remove(MoreRecentEntry);
		}

	private:
		
		TPsoLruCache* Cache;
	};

protected:

	/**
	 * Mark the given entry as recently used.
	 *
	 * @param Entry The entry to mark.
	 */
	inline void MarkAsRecent(FCacheEntry& Entry)
	{
		check(LeastRecent != nullptr);
		check(MostRecent != nullptr);

		// if entry is least recent and not the only item in the list, make it not least recent
		if ((&Entry == LeastRecent) && (LeastRecent->MoreRecent != nullptr))
		{
			LeastRecent = LeastRecent->MoreRecent;
		}

		// relink if not already the most recent item
		if (&Entry != MostRecent)
		{
			Entry.Unlink();
			Entry.LinkBefore(MostRecent);
			MostRecent = &Entry;
		}
	}

	/**
	 * Remove the specified entry from the cache.
	 *
	 * @param Entry The entry to remove.
	 */
	inline void Remove(FCacheEntry* Entry)
	{
		if (Entry == nullptr)
		{
			return;
		}

		LookupSet.Remove(Entry->Key);

		if (Entry == LeastRecent)
		{
			LeastRecent = Entry->MoreRecent;
		}

		if (Entry == MostRecent)
		{
			MostRecent = Entry->LessRecent;
		}

		Entry->Unlink();
		delete Entry;
	}

private:

	friend TIterator begin(TPsoLruCache& Cache) { return TIterator(Cache); }
	friend TConstIterator begin(const TPsoLruCache& Cache) { return TConstIterator(Cache); }
	friend TIterator end(TPsoLruCache& Cache) { return TIterator(); }
	friend TConstIterator end(const TPsoLruCache& Cache) { return TConstIterator(); }

private:

	/** Set of entries for fast lookup. */
	TSparseSet<FCacheEntry*, FKeyFuncs> LookupSet;

	/** Least recent item in the cache. */
	FCacheEntry* LeastRecent;

	/** Most recent item in the cache. */
	FCacheEntry* MostRecent;

	/** Maximum number of elements in the cache. */
	int32 MaxNumElements;
};
