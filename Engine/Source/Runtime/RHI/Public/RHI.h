// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHI.h: Render Hardware Interface definitions.
=============================================================================*/

#pragma once

#include "Containers/ContainersFwd.h"
#include "RHIDefinitions.h"
#include "Templates/TypeHash.h" // This must be before StaticArray to resolve adl compile error in nopch unity builds
#include "Containers/StaticArray.h"
#include "Containers/StringFwd.h"
#include "Math/IntRect.h"
#include "Math/PerspectiveMatrix.h"
#include "Math/ScaleMatrix.h"
#include "Math/TranslationMatrix.h"
#include "PixelFormat.h"
#include "GpuProfilerTrace.h"
#include "RHIShaderPlatform.h"
#include "RHIFeatureLevel.h"
#include "RHIAccess.h"
#include "RHIGlobals.h"

class FResourceArrayInterface;
class FResourceBulkDataInterface;
class FRHICommandList;

/**
 * RHI capabilities.
 */

// to trigger GPU specific optimizations and fallbacks
RHI_API bool IsRHIDeviceAMD();

// to trigger GPU specific optimizations and fallbacks
RHI_API bool IsRHIDeviceIntel();

// to trigger GPU specific optimizations and fallbacks
RHI_API bool IsRHIDeviceNVIDIA();

// to trigger GPU specific optimizations and fallbacks
RHI_API bool IsRHIDeviceQualcomm();

// to trigger GPU specific optimizations and fallbacks
RHI_API bool IsRHIDeviceApple();

RHI_API EGpuVendorId GetRHIDeviceVendorId();

namespace UE::RHI
{
	// Returns true when GPU crash debugging is enabled.
	RHI_API bool UseGPUCrashDebugging();

	// True when using engine-managed GPU breadcrumb tracking
	RHI_API bool UseGPUCrashBreadcrumbs();

	// Used to determine whether specific GPU crash debugging features are enabled given the combination of command line switches and console variable settings
	RHI_API bool ShouldEnableGPUCrashFeature(IConsoleVariable& CVar, TCHAR const* CommandLineSwitch);
}

// helper to return the shader language version for Metal shader.
RHI_API uint32 RHIGetMetalShaderLanguageVersion(const FStaticShaderPlatform Platform);

// helper to check if a preview feature level has been requested.
RHI_API bool RHIGetPreviewFeatureLevel(ERHIFeatureLevel::Type& PreviewFeatureLevelOUT);

// helper to check if preferred EPixelFormat is supported, return one if it is not
UE_DEPRECATED(5.8, "RHIPreferredPixelFormatHint is deprecated. Viewports should be created or resized with an explicit pixel format, not with PF_Unknown.")
RHI_API EPixelFormat RHIPreferredPixelFormatHint(EPixelFormat PreferredPixelFormat);

// helper to check which resource type should be used for clear (UAV) replacement shaders.
RHI_API int32 RHIGetPreferredClearUAVRectPSResourceType(const FStaticShaderPlatform Platform);

// helper to force dump all RHI resource to CSV file
RHI_API void RHIDumpResourceMemoryToCSV();

// helper to check whether rendering to volume textures is supported. This takes both GSupportsVolumeTextureRendering and DDPI entries into account as this is dependent on the preview platform.
RHI_API bool RHISupportsVolumeTextureRendering(const FStaticShaderPlatform Platform);

struct FRHIResourceStats
{
	FName Name;
	FName OwnerName;
	FString Type;
	FString Flags;
	uint64	SizeInBytes = 0;
	bool	bResident = false;
	bool	bMarkedForDelete = false;
	bool	bTransient = false;
	bool	bStreaming = false;
	bool	bRenderTarget = false;
	bool	bDepthStencil = false;
	bool	bUnorderedAccessView = false;
	bool	bRayTracingAccelerationStructure = false;
	bool	bHasFlags = false;

	FRHIResourceStats(const FName& InName, const FName& InOwnerName, const FString& InType, const FString& InFlags, const uint64& InSizeInBytes,
						bool bInResident, bool bInMarkedForDelete, bool bInTransient, bool bInStreaming, bool bInRT, bool bInDS, bool bInUAV, bool bInRTAS, bool bInHasFlags)
		: Name(InName)
		, OwnerName(InOwnerName)
		, Type(InType)
		, Flags(InFlags)
		, SizeInBytes(InSizeInBytes)
		, bResident(bInResident)
		, bMarkedForDelete(bInMarkedForDelete)
		, bTransient(bInTransient)
		, bStreaming(bInStreaming)
		, bRenderTarget(bInRT)
		, bDepthStencil(bInDS)
		, bUnorderedAccessView(bInUAV)
		, bRayTracingAccelerationStructure(bInRTAS)
		, bHasFlags(bInHasFlags)
	{ }
};

RHI_API void RHIGetTrackedResourceStats(TArray<TSharedPtr<FRHIResourceStats>>& OutResourceStats);

#include "MultiGPU.h"

// Calculate the index of the sample in GRHIDefaultMSAASampleOffsets
extern RHI_API int32 CalculateMSAASampleArrayIndex(int32 NumSamples, int32 SampleIndex);

// Gets the MSAA sample's offset from the center of the pixel coordinate.
inline FVector2f GetMSAASampleOffsets(int32 NumSamples, int32 SampleIndex)
{
	return GRHIDefaultMSAASampleOffsets[CalculateMSAASampleArrayIndex(NumSamples, SampleIndex)];
}

/** Initialize the 'best guess' pixel format capabilities. Platform formats and support must be filled out before calling this. */
extern RHI_API void RHIInitDefaultPixelFormatCapabilities();

inline bool RHIPixelFormatHasCapabilities(EPixelFormat InFormat, EPixelFormatCapabilities InCapabilities)
{
	return UE::PixelFormat::HasCapabilities(InFormat, InCapabilities);
}

inline bool RHIIsTypedUAVLoadSupported(EPixelFormat InFormat)
{
	return UE::PixelFormat::HasCapabilities(InFormat, EPixelFormatCapabilities::TypedUAVLoad);
}

inline bool RHIIsTypedUAVStoreSupported(EPixelFormat InFormat)
{
	return UE::PixelFormat::HasCapabilities(InFormat, EPixelFormatCapabilities::TypedUAVStore);
}

/**
* Returns the memory required to store an image in the given pixel format (EPixelFormat). Use
* GPixelFormats[Format].Get2D/3DImageSizeInBytes instead, unless you need PF_A1.
*/
extern RHI_API SIZE_T CalculateImageBytes(uint32 SizeX,uint32 SizeY,uint32 SizeZ,uint8 Format);


/**
 * Adjusts a projection matrix to output in the correct clip space for the
 * current RHI. Unreal projection matrices follow certain conventions and
 * need to be patched for some RHIs. All projection matrices should be adjusted
 * before being used for rendering!
 */
inline FMatrix AdjustProjectionMatrixForRHI(const FMatrix& InProjectionMatrix)
{
	FScaleMatrix ClipSpaceFixScale(FVector(1.0f, GProjectionSignY, 1.0f - GMinClipZ));
	FTranslationMatrix ClipSpaceFixTranslate(FVector(0.0f, 0.0f, GMinClipZ));	
	return InProjectionMatrix * ClipSpaceFixScale * ClipSpaceFixTranslate;
}

/** Set runtime selection of mobile feature level preview. */
RHI_API void RHISetMobilePreviewFeatureLevel(ERHIFeatureLevel::Type MobilePreviewFeatureLevel);

struct FVertexElement
{
	uint8 StreamIndex;
	uint8 Offset;
	TEnumAsByte<EVertexElementType> Type;
	uint8 AttributeIndex;
	uint16 Stride;
	/**
	 * Whether to use instance index or vertex index to consume the element.  
	 * eg if bUseInstanceIndex is 0, the element will be repeated for every instance.
	 */
	uint16 bUseInstanceIndex;

	FVertexElement() {}
	FVertexElement(uint8 InStreamIndex,uint8 InOffset,EVertexElementType InType,uint8 InAttributeIndex,uint16 InStride,bool bInUseInstanceIndex = false):
		StreamIndex(InStreamIndex),
		Offset(InOffset),
		Type(InType),
		AttributeIndex(InAttributeIndex),
		Stride(InStride),
		bUseInstanceIndex(bInUseInstanceIndex)
	{}

	bool operator==(const FVertexElement& Other) const
	{
		return (StreamIndex		== Other.StreamIndex &&
				Offset			== Other.Offset &&
				Type			== Other.Type &&
				AttributeIndex	== Other.AttributeIndex &&
				Stride			== Other.Stride &&
				bUseInstanceIndex == Other.bUseInstanceIndex);
	}

	friend FArchive& operator<<(FArchive& Ar,FVertexElement& Element)
	{
		Ar << Element.StreamIndex;
		Ar << Element.Offset;
		Ar << Element.Type;
		Ar << Element.AttributeIndex;
		Ar << Element.Stride;
		Ar << Element.bUseInstanceIndex;
		return Ar;
	}
	RHI_API FString ToString() const;
	RHI_API void FromString(const FString& Src);
	RHI_API void FromString(const FStringView& Src);
};

typedef TArray<FVertexElement,TFixedAllocator<MaxVertexElementCount> > FVertexDeclarationElementList;

struct FSamplerStateInitializerRHI
{
	FSamplerStateInitializerRHI() {}
	FSamplerStateInitializerRHI(
		ESamplerFilter InFilter,
		ESamplerAddressMode InAddressU = AM_Wrap,
		ESamplerAddressMode InAddressV = AM_Wrap,
		ESamplerAddressMode InAddressW = AM_Wrap,
		float InMipBias = 0,
		int32 InMaxAnisotropy = 0,
		float InMinMipLevel = 0,
		float InMaxMipLevel = FLT_MAX,
		uint32 InBorderColor = 0,
		/** Only supported in D3D11 */
		ESamplerCompareFunction InSamplerComparisonFunction = SCF_Never
		)
	:	Filter(InFilter)
	,	AddressU(InAddressU)
	,	AddressV(InAddressV)
	,	AddressW(InAddressW)
	,	MipBias(InMipBias)
	,	MinMipLevel(InMinMipLevel)
	,	MaxMipLevel(InMaxMipLevel)
	,	MaxAnisotropy(InMaxAnisotropy)
	,	BorderColor(InBorderColor)
	,	SamplerComparisonFunction(InSamplerComparisonFunction)
	{
	}
	TEnumAsByte<ESamplerFilter> Filter = SF_Point;
	TEnumAsByte<ESamplerAddressMode> AddressU = AM_Wrap;
	TEnumAsByte<ESamplerAddressMode> AddressV = AM_Wrap;
	TEnumAsByte<ESamplerAddressMode> AddressW = AM_Wrap;
	float MipBias = 0.0f;
	/** Smallest mip map level that will be used, where 0 is the highest resolution mip level. */
	float MinMipLevel = 0.0f;
	/** Largest mip map level that will be used, where 0 is the highest resolution mip level. */
	float MaxMipLevel = FLT_MAX;
	int32 MaxAnisotropy = 0;
	uint32 BorderColor = 0;
	TEnumAsByte<ESamplerCompareFunction> SamplerComparisonFunction = SCF_Never;


	RHI_API friend uint32 GetTypeHash(const FSamplerStateInitializerRHI& Initializer);
	RHI_API friend bool operator== (const FSamplerStateInitializerRHI& A, const FSamplerStateInitializerRHI& B);
};

struct FRasterizerStateInitializerRHI
{
	TEnumAsByte<ERasterizerFillMode> FillMode = FM_Point;
	TEnumAsByte<ERasterizerCullMode> CullMode = CM_None;
	float DepthBias = 0.0f;
	float SlopeScaleDepthBias = 0.0f;
	ERasterizerDepthClipMode DepthClipMode = ERasterizerDepthClipMode::DepthClip;
	bool bAllowMSAA = false;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRasterizerStateInitializerRHI() = default;
	FRasterizerStateInitializerRHI(const FRasterizerStateInitializerRHI&) = default;
	FRasterizerStateInitializerRHI(FRasterizerStateInitializerRHI&&) = default;
	FRasterizerStateInitializerRHI& operator=(const FRasterizerStateInitializerRHI&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FRasterizerStateInitializerRHI(ERasterizerFillMode InFillMode, ERasterizerCullMode InCullMode, bool bInAllowMSAA)
		: FillMode(InFillMode)
		, CullMode(InCullMode)
		, bAllowMSAA(bInAllowMSAA)
	{
	}

	FRasterizerStateInitializerRHI(ERasterizerFillMode InFillMode, ERasterizerCullMode InCullMode, float InDepthBias, float InSlopeScaleDepthBias, ERasterizerDepthClipMode InDepthClipMode, bool bInAllowMSAA)
		: FillMode(InFillMode)
		, CullMode(InCullMode)
		, DepthBias(InDepthBias)
		, SlopeScaleDepthBias(InSlopeScaleDepthBias)
		, DepthClipMode(InDepthClipMode)
		, bAllowMSAA(bInAllowMSAA)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FRasterizerStateInitializerRHI& RasterizerStateInitializer)
	{
		Ar << RasterizerStateInitializer.FillMode;
		Ar << RasterizerStateInitializer.CullMode;
		Ar << RasterizerStateInitializer.DepthBias;
		Ar << RasterizerStateInitializer.SlopeScaleDepthBias;
		Ar << RasterizerStateInitializer.DepthClipMode;
		Ar << RasterizerStateInitializer.bAllowMSAA;
		return Ar;
	}

	RHI_API friend uint32 GetTypeHash(const FRasterizerStateInitializerRHI& Initializer);
	RHI_API friend bool operator== (const FRasterizerStateInitializerRHI& A, const FRasterizerStateInitializerRHI& B);
};

struct FDepthStencilStateInitializerRHI
{
	bool bEnableDepthWrite;
	TEnumAsByte<ECompareFunction> DepthTest;

	bool bEnableFrontFaceStencil;
	TEnumAsByte<ECompareFunction> FrontFaceStencilTest;
	TEnumAsByte<EStencilOp> FrontFaceStencilFailStencilOp;
	TEnumAsByte<EStencilOp> FrontFaceDepthFailStencilOp;
	TEnumAsByte<EStencilOp> FrontFacePassStencilOp;
	bool bEnableBackFaceStencil;
	TEnumAsByte<ECompareFunction> BackFaceStencilTest;
	TEnumAsByte<EStencilOp> BackFaceStencilFailStencilOp;
	TEnumAsByte<EStencilOp> BackFaceDepthFailStencilOp;
	TEnumAsByte<EStencilOp> BackFacePassStencilOp;
	uint8 StencilReadMask;
	uint8 StencilWriteMask;

	FDepthStencilStateInitializerRHI(
		bool bInEnableDepthWrite = true,
		ECompareFunction InDepthTest = CF_LessEqual,
		bool bInEnableFrontFaceStencil = false,
		ECompareFunction InFrontFaceStencilTest = CF_Always,
		EStencilOp InFrontFaceStencilFailStencilOp = SO_Keep,
		EStencilOp InFrontFaceDepthFailStencilOp = SO_Keep,
		EStencilOp InFrontFacePassStencilOp = SO_Keep,
		bool bInEnableBackFaceStencil = false,
		ECompareFunction InBackFaceStencilTest = CF_Always,
		EStencilOp InBackFaceStencilFailStencilOp = SO_Keep,
		EStencilOp InBackFaceDepthFailStencilOp = SO_Keep,
		EStencilOp InBackFacePassStencilOp = SO_Keep,
		uint8 InStencilReadMask = 0xFF,
		uint8 InStencilWriteMask = 0xFF
		)
	: bEnableDepthWrite(bInEnableDepthWrite)
	, DepthTest(InDepthTest)
	, bEnableFrontFaceStencil(bInEnableFrontFaceStencil)
	, FrontFaceStencilTest(InFrontFaceStencilTest)
	, FrontFaceStencilFailStencilOp(InFrontFaceStencilFailStencilOp)
	, FrontFaceDepthFailStencilOp(InFrontFaceDepthFailStencilOp)
	, FrontFacePassStencilOp(InFrontFacePassStencilOp)
	, bEnableBackFaceStencil(bInEnableBackFaceStencil)
	, BackFaceStencilTest(InBackFaceStencilTest)
	, BackFaceStencilFailStencilOp(InBackFaceStencilFailStencilOp)
	, BackFaceDepthFailStencilOp(InBackFaceDepthFailStencilOp)
	, BackFacePassStencilOp(InBackFacePassStencilOp)
	, StencilReadMask(InStencilReadMask)
	, StencilWriteMask(InStencilWriteMask)
	{}
	
	friend FArchive& operator<<(FArchive& Ar,FDepthStencilStateInitializerRHI& DepthStencilStateInitializer)
	{
		Ar << DepthStencilStateInitializer.bEnableDepthWrite;
		Ar << DepthStencilStateInitializer.DepthTest;
		Ar << DepthStencilStateInitializer.bEnableFrontFaceStencil;
		Ar << DepthStencilStateInitializer.FrontFaceStencilTest;
		Ar << DepthStencilStateInitializer.FrontFaceStencilFailStencilOp;
		Ar << DepthStencilStateInitializer.FrontFaceDepthFailStencilOp;
		Ar << DepthStencilStateInitializer.FrontFacePassStencilOp;
		Ar << DepthStencilStateInitializer.bEnableBackFaceStencil;
		Ar << DepthStencilStateInitializer.BackFaceStencilTest;
		Ar << DepthStencilStateInitializer.BackFaceStencilFailStencilOp;
		Ar << DepthStencilStateInitializer.BackFaceDepthFailStencilOp;
		Ar << DepthStencilStateInitializer.BackFacePassStencilOp;
		Ar << DepthStencilStateInitializer.StencilReadMask;
		Ar << DepthStencilStateInitializer.StencilWriteMask;
		return Ar;
	}

	RHI_API friend uint32 GetTypeHash(const FDepthStencilStateInitializerRHI& Initializer);
	RHI_API friend bool operator== (const FDepthStencilStateInitializerRHI& A, const FDepthStencilStateInitializerRHI& B);
	
	RHI_API FString ToString() const;
	RHI_API void FromString(const FString& Src);
	RHI_API void FromString(const FStringView& Src);
};

class FBlendStateInitializerRHI
{
public:

	struct FRenderTarget
	{
		enum
		{
			NUM_STRING_FIELDS = 7
		};
		TEnumAsByte<EBlendOperation> ColorBlendOp;
		TEnumAsByte<EBlendFactor> ColorSrcBlend;
		TEnumAsByte<EBlendFactor> ColorDestBlend;
		TEnumAsByte<EBlendOperation> AlphaBlendOp;
		TEnumAsByte<EBlendFactor> AlphaSrcBlend;
		TEnumAsByte<EBlendFactor> AlphaDestBlend;
		TEnumAsByte<EColorWriteMask> ColorWriteMask;
		
		FRenderTarget(
			EBlendOperation InColorBlendOp = BO_Add,
			EBlendFactor InColorSrcBlend = BF_One,
			EBlendFactor InColorDestBlend = BF_Zero,
			EBlendOperation InAlphaBlendOp = BO_Add,
			EBlendFactor InAlphaSrcBlend = BF_One,
			EBlendFactor InAlphaDestBlend = BF_Zero,
			EColorWriteMask InColorWriteMask = CW_RGBA
			)
		: ColorBlendOp(InColorBlendOp)
		, ColorSrcBlend(InColorSrcBlend)
		, ColorDestBlend(InColorDestBlend)
		, AlphaBlendOp(InAlphaBlendOp)
		, AlphaSrcBlend(InAlphaSrcBlend)
		, AlphaDestBlend(InAlphaDestBlend)
		, ColorWriteMask(InColorWriteMask)
		{}
		
		friend FArchive& operator<<(FArchive& Ar,FRenderTarget& RenderTarget)
		{
			Ar << RenderTarget.ColorBlendOp;
			Ar << RenderTarget.ColorSrcBlend;
			Ar << RenderTarget.ColorDestBlend;
			Ar << RenderTarget.AlphaBlendOp;
			Ar << RenderTarget.AlphaSrcBlend;
			Ar << RenderTarget.AlphaDestBlend;
			Ar << RenderTarget.ColorWriteMask;
			return Ar;
		}
		
		RHI_API FString ToString() const;
		RHI_API void FromString(const TArray<FString>& Parts, int32 Index);
		RHI_API void FromString(TArrayView<const FStringView> Parts);
	};

	FBlendStateInitializerRHI() {}

	FBlendStateInitializerRHI(const FRenderTarget& InRenderTargetBlendState, bool bInUseAlphaToCoverage = false)
	:	bUseIndependentRenderTargetBlendStates(false)
	,	bUseAlphaToCoverage(bInUseAlphaToCoverage)
	{
		RenderTargets[0] = InRenderTargetBlendState;
	}

	template<uint32 NumRenderTargets>
	FBlendStateInitializerRHI(const TStaticArray<FRenderTarget,NumRenderTargets>& InRenderTargetBlendStates, bool bInUseAlphaToCoverage = false)
	:	bUseIndependentRenderTargetBlendStates(NumRenderTargets > 1)
	,	bUseAlphaToCoverage(bInUseAlphaToCoverage)
	{
		static_assert(NumRenderTargets <= MaxSimultaneousRenderTargets, "Too many render target blend states.");

		for(uint32 RenderTargetIndex = 0;RenderTargetIndex < NumRenderTargets;++RenderTargetIndex)
		{
			RenderTargets[RenderTargetIndex] = InRenderTargetBlendStates[RenderTargetIndex];
		}
	}

	TStaticArray<FRenderTarget,MaxSimultaneousRenderTargets> RenderTargets;
	bool bUseIndependentRenderTargetBlendStates;
	bool bUseAlphaToCoverage;
	
	friend FArchive& operator<<(FArchive& Ar,FBlendStateInitializerRHI& BlendStateInitializer)
	{
		Ar << BlendStateInitializer.RenderTargets;
		Ar << BlendStateInitializer.bUseIndependentRenderTargetBlendStates;
		Ar << BlendStateInitializer.bUseAlphaToCoverage;
		return Ar;
	}

	RHI_API friend uint32 GetTypeHash(const FBlendStateInitializerRHI::FRenderTarget& RenderTarget);
	RHI_API friend bool operator== (const FBlendStateInitializerRHI::FRenderTarget& A, const FBlendStateInitializerRHI::FRenderTarget& B);
	
	RHI_API friend uint32 GetTypeHash(const FBlendStateInitializerRHI& Initializer);
	RHI_API friend bool operator== (const FBlendStateInitializerRHI& A, const FBlendStateInitializerRHI& B);
	
	RHI_API FString ToString() const;
	RHI_API void FromString(const FString& Src);
	RHI_API void FromString(const FStringView& Src);
};



/**
 *	Viewport bounds structure to set multiple view ports for the geometry shader
 *  (needs to be 1:1 to the D3D11 structure)
 */
struct FViewportBounds
{
	float	TopLeftX;
	float	TopLeftY;
	float	Width;
	float	Height;
	float	MinDepth;
	float	MaxDepth;

	FViewportBounds() {}

	FViewportBounds(float InTopLeftX, float InTopLeftY, float InWidth, float InHeight, float InMinDepth = 0.0f, float InMaxDepth = 1.0f)
		:TopLeftX(InTopLeftX), TopLeftY(InTopLeftY), Width(InWidth), Height(InHeight), MinDepth(InMinDepth), MaxDepth(InMaxDepth)
	{
	}
};



struct FVRamAllocation
{
	FVRamAllocation() = default;
	FVRamAllocation(uint64 InAllocationStart, uint64 InAllocationSize)
		: AllocationStart(InAllocationStart)
		, AllocationSize(InAllocationSize)
	{
	}

	bool IsValid() const { return AllocationSize > 0; }

	// in bytes
	uint64 AllocationStart{};
	// in bytes
	uint64 AllocationSize{};
};

struct FRHIResourceInfo
{
	FName Name;
	ERHIResourceType Type{ RRT_None };
	FVRamAllocation VRamAllocation;
	bool IsTransient{ false };
	bool bValid{ true };
	bool bResident{ true };
};

struct FRHIDispatchIndirectParametersNoPadding
{
	uint32 ThreadGroupCountX;
	uint32 ThreadGroupCountY;
	uint32 ThreadGroupCountZ;
};

struct FRHIDispatchIndirectParameters : public FRHIDispatchIndirectParametersNoPadding
{
#if PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE == 64
	uint32 Padding;			// pad to 32 bytes to prevent crossing of 64 byte boundary in ExecuteIndirect calls
#elif PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE == 128
	uint32 Padding[5];		// pad to 64 bytes to prevent crossing of 128 byte boundary in ExecuteIndirect calls
#elif PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE != 0
	#error FRHIDispatchIndirectParameters does not account for PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE.
#endif
};
static_assert(PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE == 0 || PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE % sizeof(FRHIDispatchIndirectParameters) == 0);

struct FRHIDrawIndirectParameters
{
	uint32 VertexCountPerInstance;
	uint32 InstanceCount;
	uint32 StartVertexLocation;
	uint32 StartInstanceLocation;
};

struct FRHIDrawIndexedIndirectParameters
{
	uint32 IndexCountPerInstance;
	uint32 InstanceCount;
	uint32 StartIndexLocation;
	int32 BaseVertexLocation;
	uint32 StartInstanceLocation;
};

// RHI base resource types.
#include "RHIResources.h"
#include "DynamicRHI.h"

/** Initializes the RHI. */
extern RHI_API void RHIInit(bool bHasEditorToken);

/** Performs additional RHI initialization before the render thread starts. */
extern RHI_API void RHIPostInit(const TArray<uint32>& InPixelFormatByteWidth);

/** Shuts down the RHI. */
extern RHI_API void RHIExit();

// Detect whether the current driver is denylisted and show a message box
// prompting to update it if necessary.
extern RHI_API void RHIDetectAndWarnOfBadDrivers(bool bHasEditorToken);

// Panic delegate is called when when a fatal condition is encountered within RHI function.
DECLARE_DELEGATE_OneParam(FRHIPanicEvent, const FName&);
extern RHI_API FRHIPanicEvent& RHIGetPanicDelegate();

// Return what the expected number of samplers will be supported by a feature level
// Note that since the Feature Level is pretty orthogonal to the RHI/HW, this is not going to be perfect
// If should only be used for a guess at the limit, the real limit will not be known until runtime
inline uint32 GetExpectedFeatureLevelMaxTextureSamplers(const FStaticFeatureLevel FeatureLevel)
{
	return 16;
}

// Get the bindless configuration specified via the command line
RHI_API TOptional<ERHIBindlessConfiguration> RHIGetForcedBindlessConfiguration();

UE_DEPRECATED(5.8, "RHIGetShaderPlatformConfigurationInt is no longer implemented.")
inline bool RHIGetShaderPlatformConfigurationInt(int32& OutSetting, FConfigCacheIni* Config, EShaderPlatform Platform, const TCHAR* SettingName)
{
	return false;
}

UE_DEPRECATED(5.8, "RHIGetShaderPlatformConfigurationInt is no longer implemented.")
inline bool RHIGetShaderPlatformConfigurationString(FString& OutSetting, FConfigCacheIni* Config, EShaderPlatform Platform, const TCHAR* SettingName)
{
	return false;
}

UE_DEPRECATED(5.8, "RHIParseBindlessConfiguration is no longer implemented. Please use FShaderPlatformConfig::GetBindlessConfiguration")
inline ERHIBindlessConfiguration RHIParseBindlessConfiguration(EShaderPlatform Platform, const FString& ConfigSetting, const FString& CvarSetting)
{
	return ERHIBindlessConfiguration::Disabled;
}

UE_DEPRECATED(5.8, "RHIGetRuntimeBindlessConfiguration is no longer implemented. Please use FShaderPlatformConfig::GetBindlessConfiguration")
inline ERHIBindlessConfiguration RHIGetRuntimeBindlessConfiguration(EShaderPlatform Platform)
{
	return ERHIBindlessConfiguration::Disabled;
}

UE_DEPRECATED(5.7, "RHIGetRuntimeBindlessResourcesConfiguration is no longer implemented. Please use FShaderPlatformConfig::GetBindlessConfiguration")
inline ERHIBindlessConfiguration RHIGetRuntimeBindlessResourcesConfiguration(EShaderPlatform Platform)
{
	return ERHIBindlessConfiguration::Disabled;
}

UE_DEPRECATED(5.7, "RHIGetRuntimeBindlessSamplersConfiguration is no longer implemented. Please use FShaderPlatformConfig::GetBindlessConfiguration")
inline ERHIBindlessConfiguration RHIGetRuntimeBindlessSamplersConfiguration(EShaderPlatform Platform)
{
	return ERHIBindlessConfiguration::Disabled;
}