// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"

class FString;

enum class ERHIAccess : uint32
{
	// Used when the previous state of a resource is not known,
	// which implies we have to flush all GPU caches etc.
	Unknown = 0,

	// Read states
	CPURead                 = 1 <<  0,
	Present                 = 1 <<  1,
	IndirectArgs            = 1 <<  2,
	VertexOrIndexBuffer     = 1 <<  3,
	SRVCompute              = 1 <<  4,
	SRVGraphicsPixel        = 1 <<  5,
	SRVGraphicsNonPixel     = 1 <<  6,
	CopySrc                 = 1 <<  7,
	ResolveSrc              = 1 <<  8,
	DSVRead                 = 1 <<  9,

	// Read-write states
	UAVCompute              = 1 << 10,
	UAVGraphics             = 1 << 11,
	RTV                     = 1 << 12,
	CopyDest                = 1 << 13,
	ResolveDst              = 1 << 14,
	DSVWrite                = 1 << 15,

	// Ray tracing acceleration structure states.
	// Buffer that contains an AS must always be in either of these states.
	// BVHRead -- required for AS inputs to build/update/copy/trace commands.
	// BVHWrite -- required for AS outputs of build/update/copy commands.
	BVHRead                  = 1 << 16,
	BVHWrite                 = 1 << 17,

	// Invalid released state (transient resources)
	Discard					= 1 << 18,

	// Shading Rate Source
	ShadingRateSource	= 1 << 19,

	Last = ShadingRateSource,
	None = Unknown,
	Mask = (Last << 1) - 1,

	// Graphics is a combination of pixel and non-pixel
	SRVGraphics = SRVGraphicsPixel | SRVGraphicsNonPixel,

	// A mask of the two possible SRV states
	SRVMask = SRVCompute | SRVGraphics,

	// A mask of the two possible UAV states
	UAVMask = UAVCompute | UAVGraphics,

	// A mask of all bits representing read-only states which cannot be combined with other write states.
	ReadOnlyExclusiveMask = CPURead | Present | IndirectArgs | VertexOrIndexBuffer | SRVGraphics | SRVCompute | CopySrc | ResolveSrc | BVHRead | ShadingRateSource,

	// A mask of all bits representing read-only states on the compute pipe which cannot be combined with other write states.
	ReadOnlyExclusiveComputeMask = CPURead | IndirectArgs | SRVCompute | CopySrc | BVHRead,

	// A mask of all bits representing read-only states which may be combined with other write states.
	ReadOnlyMask = ReadOnlyExclusiveMask | DSVRead | ShadingRateSource,

	// A mask of all bits representing readable states which may also include writable states.
	ReadableMask = ReadOnlyMask | UAVMask,

	// A mask of all bits representing write-only states which cannot be combined with other read states.
	WriteOnlyExclusiveMask = RTV | CopyDest | ResolveDst,

	// A mask of all bits representing write-only states which may be combined with other read states.
	WriteOnlyMask = WriteOnlyExclusiveMask | DSVWrite,

	// A mask of all bits representing writable states which may also include readable states.
	WritableMask = WriteOnlyMask | UAVMask | BVHWrite,

	// A mask of all bits readable states exclusive to the graphics queue.
	GraphicsReadableMask = VertexOrIndexBuffer | DSVRead | SRVGraphics | ResolveSrc | ShadingRateSource
};
ENUM_CLASS_FLAGS(ERHIAccess)

inline constexpr bool IsReadOnlyExclusiveAccess(ERHIAccess Access)
{
	return EnumHasAnyFlags(Access, ERHIAccess::ReadOnlyExclusiveMask) && !EnumHasAnyFlags(Access, ~ERHIAccess::ReadOnlyExclusiveMask);
}

inline constexpr bool IsReadOnlyAccess(ERHIAccess Access)
{
	return EnumHasAnyFlags(Access, ERHIAccess::ReadOnlyMask) && !EnumHasAnyFlags(Access, ~ERHIAccess::ReadOnlyMask);
}

inline constexpr bool IsWriteOnlyAccess(ERHIAccess Access)
{
	return EnumHasAnyFlags(Access, ERHIAccess::WriteOnlyMask) && !EnumHasAnyFlags(Access, ~ERHIAccess::WriteOnlyMask);
}

inline constexpr bool IsWritableAccess(ERHIAccess Access)
{
	return EnumHasAnyFlags(Access, ERHIAccess::WritableMask);
}

inline constexpr bool IsReadableAccess(ERHIAccess Access)
{
	return EnumHasAnyFlags(Access, ERHIAccess::ReadableMask);
}

inline constexpr bool IsInvalidAccess(ERHIAccess Access)
{
	return
		((EnumHasAnyFlags(Access, ERHIAccess::ReadOnlyExclusiveMask) && EnumHasAnyFlags(Access, ERHIAccess::WritableMask)) ||
			(EnumHasAnyFlags(Access, ERHIAccess::WriteOnlyExclusiveMask) && EnumHasAnyFlags(Access, ERHIAccess::ReadableMask)));
}

inline constexpr bool IsValidAccess(ERHIAccess Access)
{
	return !IsInvalidAccess(Access);
}

/** Mask of states which are allowed to be considered for state merging. */
extern RHI_API ERHIAccess GRHIMergeableAccessMask;

/** Mask of states which are allowed to be considered for multi-pipeline state merging. This should be a subset of GRHIMergeableAccessMask. */
extern RHI_API ERHIAccess GRHIMultiPipelineMergeableAccessMask;

/** [NeedsTransientDiscardStateTracking] State to use as an intermediate state when transitioning to discard and the resource has differing prior states per subresource. */
extern RHI_API ERHIAccess GRHIMultiSubresourceDiscardIntermediateAccess;