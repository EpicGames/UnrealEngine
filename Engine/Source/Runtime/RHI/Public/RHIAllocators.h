// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHIAllocators.h: Render Hardware Interface allocators
=============================================================================*/

#pragma once

#include "Experimental/ConcurrentLinearAllocator.h"

// Same of FDefaultBlockAllocationTag but custom TagName
struct FCommandListBaseBlockAllocationTag
{
	static constexpr uint32 BlockSize = 64 * 1024;		// Blocksize used to allocate from
	static constexpr bool AllowOversizedBlocks = true;  // The allocator supports oversized Blocks and will store them in a separate Block with counter 1
	static constexpr bool RequiresAccurateSize = true;  // GetAllocationSize returning the accurate size of the allocation otherwise it could be relaxed to return the size to the end of the Block
	static constexpr bool InlineBlockAllocation = false;  // Inline or Noinline the BlockAllocation which can have an impact on Performance
	static constexpr const char* TagName = "RHICommandListBaseAllocator";

	using Allocator = TBlockAllocationLockFreeCache<BlockSize, FAlignedAllocator>;
};

using FRHICmdListBaseArrayAllocator = TConcurrentLinearArrayAllocator<FCommandListBaseBlockAllocationTag>;
using FRHICmdListBaseSetAllocator = TConcurrentLinearSetAllocator<FCommandListBaseBlockAllocationTag>;
using FRHICmdListBaseLinearAllocator = TConcurrentLinearAllocator<FCommandListBaseBlockAllocationTag>;