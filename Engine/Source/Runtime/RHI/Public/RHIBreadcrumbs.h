// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "Misc/MemStack.h"
#include "Hash/Fnv.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

#include "GpuProfilerTrace.h"
#include "RHIFwd.h"
#include "RHIPipeline.h"
#include "MultiGPU.h"

#include <array>

//
// Controls whether the infrastructure for the RHI breadcrumbs system is included in the build.
// (RHI breadcrumb allocators, platform RHI implemntation etc).
//
#ifndef WITH_RHI_BREADCRUMBS
#define WITH_RHI_BREADCRUMBS (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT || WITH_PROFILEGPU || HAS_GPU_STATS)
#endif

// Enables all RDG/RHI breadcrumb scopes, and features such as Insights markers.
#define WITH_RHI_BREADCRUMBS_FULL    (WITH_RHI_BREADCRUMBS && (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT || WITH_PROFILEGPU))

// Enables only the necessary scopes for GPU stats to work
#define WITH_RHI_BREADCRUMBS_MINIMAL (WITH_RHI_BREADCRUMBS && (!WITH_RHI_BREADCRUMBS_FULL))

// Whether to emit Unreal Insights breadcrumb events on threads involved in RHI command list recording and execution.
#define RHI_BREADCRUMBS_EMIT_CPU (WITH_RHI_BREADCRUMBS_FULL && CPUPROFILERTRACE_ENABLED && 1)

// Whether to store the filename and line number of each RHI breadcrumb and emit this data to Insights.
#define RHI_BREADCRUMBS_EMIT_LOCATION (WITH_RHI_BREADCRUMBS_FULL && CPUPROFILERTRACE_ENABLED && 1)

namespace UE::RHI::GPUProfiler
{
	struct FGPUStat;
}

#if WITH_RHI_BREADCRUMBS

	//
	// Holds the filename and line number location of the RHI breadcrumb in source.
	//
	struct FRHIBreadcrumbData_Location
	{
#if RHI_BREADCRUMBS_EMIT_LOCATION
		ANSICHAR const* File;
		uint32 Line;
#endif

		FRHIBreadcrumbData_Location(ANSICHAR const* File, uint32 Line)
#if RHI_BREADCRUMBS_EMIT_LOCATION
			: File(File)
			, Line(Line)
#endif
		{}
	};


	//
	// Container for extra profiling-related data for each RHI breadcrumb.
	//
	class FRHIBreadcrumbData
		// Use inheritance for empty-base-optimization.
		: public FRHIBreadcrumbData_Location
	{
	public:
		TCHAR const* const StaticName;

		// Random assigned color, based on static name
		FColor const Color;

		template <size_t N>
		FRHIBreadcrumbData(TCHAR const(&StaticName)[N], ANSICHAR const* File, uint32 Line)
			: FRHIBreadcrumbData_Location(File, Line)
			, StaticName(StaticName)
			, Color(FLinearColor::IntToDistinctColor(UE::HashStringFNV1a<uint32>(MakeStringView(StaticName, N - 1))).ToFColor(true))
		{}
	};

	class FRHIBreadcrumbAllocator;
	struct FRHIBreadcrumbRange;

	struct FRHIBreadcrumbState
	{
		struct FPipeline
		{
			uint32 MarkerIn = 0, MarkerOut = 0;
		};

		struct FDevice
		{
			TRHIPipelineArray<FPipeline> Pipelines;
		};

		TStaticArray<FDevice, MAX_NUM_GPUS> Devices{ InPlace };

		struct FQueueID
		{
			uint32 DeviceIndex;
			ERHIPipeline Pipeline;

			bool operator == (FQueueID const& RHS) const
			{
				return DeviceIndex == RHS.DeviceIndex && Pipeline == RHS.Pipeline;
			}

			bool operator != (FQueueID const& RHS) const
			{
				return !(*this == RHS);
			}

			friend uint32 GetTypeHash(FQueueID const& ID)
			{
				return HashCombineFast(GetTypeHash(ID.DeviceIndex), GetTypeHash(ID.Pipeline));
			}
		};

		enum class EVerbosity
		{
			Log,
			Warning,
			Error
		};

		RHI_API void DumpActiveBreadcrumbs(TMap<FQueueID, TArray<FRHIBreadcrumbRange>> const& QueueRanges, EVerbosity Verbosity = EVerbosity::Error) const;
	};

	struct FRHIBreadcrumbNode
	{
		RHI_API static std::atomic<uint32> NextID;

	private:
		FRHIBreadcrumbNode* Parent = Sentinel;
		FRHIBreadcrumbNode* ListLink = nullptr;
		TStaticArray<FRHIBreadcrumbNode*, uint32(ERHIPipeline::Num)> NextPtrs { InPlace, nullptr };

	public:
		FRHIBreadcrumbAllocator* const Allocator = nullptr;

		FRHIBreadcrumbData const& Data;
#if HAS_GPU_STATS
		UE::RHI::GPUProfiler::FGPUStat* GPUStat = nullptr;
#endif
	#if RHI_BREADCRUMBS_EMIT_CPU
		uint32 TraceCpuSpecId = 0;
		uint32 TraceCpuMetadataId = 0;
	#endif

		uint32 const ID = 0;

	#if DO_CHECK
		// Used to track use of this breadcrumb on each GPU pipeline. Breadcrumbs can only be begun/ended once per pipe.
		std::atomic<std::underlying_type_t<ERHIPipeline>> BeginPipes = std::underlying_type_t<ERHIPipeline>(ERHIPipeline::None);
		std::atomic<std::underlying_type_t<ERHIPipeline>> EndPipes   = std::underlying_type_t<ERHIPipeline>(ERHIPipeline::None);
	#endif

		FRHIBreadcrumbNode(FRHIBreadcrumbData const& Data, FRHIBreadcrumbAllocator& Allocator, UE::RHI::GPUProfiler::FGPUStat* InGPUStat = nullptr)
			: Allocator(&Allocator)
			, Data(Data)
#if HAS_GPU_STATS
			, GPUStat(InGPUStat)
#endif
			, ID(NextID.fetch_add(1, std::memory_order_relaxed) | 0x80000000) // Set the top bit to avoid collision with zero (i.e. "no breadcrumb")
		{
#if !HAS_GPU_STATS
			(void)InGPUStat;
#endif
		}

		FRHIBreadcrumbNode*&       GetNextPtr(ERHIPipeline Pipeline)       { return NextPtrs[GetRHIPipelineIndex(Pipeline)]; }
		FRHIBreadcrumbNode* const& GetNextPtr(ERHIPipeline Pipeline) const { return NextPtrs[GetRHIPipelineIndex(Pipeline)]; }

		FRHIBreadcrumbNode* GetParent() const { return Parent; }
		inline void SetParent(FRHIBreadcrumbNode* Node);

		virtual void TraceBeginGPU(uint32 QueueId, uint64 GPUTimestampTOP) const = 0;
		virtual void TraceEndGPU(uint32 QueueId, uint64 GPUTimestampTOP) const = 0;

		inline void TraceBeginCPU() const;
		inline void TraceEndCPU() const;

		// Calls BeginCPU() on all the breadcrumb nodes between the root and the specified node.
		// Only valid to call from the bottom-of-pipe, after the dispatch thread has fixed up the breadcrumb tree.
		static inline void WalkIn(FRHIBreadcrumbNode const* Node);

		// Same as WalkIn, but the root node is specified, allowing it to be called from the top-of-pipe.
		static inline void WalkInRange(FRHIBreadcrumbNode const* Leaf, FRHIBreadcrumbNode const* Root);

		// Calls EndCPU() on all the breadcrumb nodes between the specified node and the root.
		// Only valid to call from the bottom-of-pipe, after the dispatch thread has fixed up the breadcrumb tree.
		static inline void WalkOut(FRHIBreadcrumbNode const* Node);

		// Same as WalkOut, but the root node is specified, allowing it to be called from the top-of-pipe.
		static inline void WalkOutRange(FRHIBreadcrumbNode const* Leaf, FRHIBreadcrumbNode const* Root);

		// ----------------------------------------------------
		// Debug logging / crash reporting
		// ----------------------------------------------------

	#if WITH_ADDITIONAL_CRASH_CONTEXTS
		// Logs the stack of breadcrumbs to the crash context, starting from the current node.
		RHI_API void WriteCrashData(struct FCrashContextExtendedWriter& Writer, const TCHAR* ThreadName) const;
	#endif

		RHI_API FString GetFullPath() const;

		static RHI_API FRHIBreadcrumbNode const* FindCommonAncestor(FRHIBreadcrumbNode const* Node0, FRHIBreadcrumbNode const* Node1);
		static RHI_API uint32 GetLevel(FRHIBreadcrumbNode const* Node);

		static RHI_API FRHIBreadcrumbNode const* GetNonNullRoot(FRHIBreadcrumbNode const* Node);

		// A constant pointer value representing an undefined node. Used as the parent pointer for nodes in sub-trees
		// that haven't been attached to the root yet, specifically to be distinct from nullptr which is the root.
		static RHI_API FRHIBreadcrumbNode* const Sentinel;

		// The maximum length of a breadcrumb string, including the null terminator.
		static constexpr uint32 MaxLength = 128;

		struct FBuffer
		{
			TCHAR Data[MaxLength];
		};

		virtual TCHAR const* GetTCHAR(FBuffer& Buffer) const = 0;

		TCHAR const* GetTCHARNoFormat() const
		{
			return Data.StaticName;
		}

	protected:
		// Constructor for the sentinel value
		FRHIBreadcrumbNode(FRHIBreadcrumbData const& Data);
		friend struct FRHIBreadcrumbList;
	};

	// Typedef for backwards compatibility. The FRHIBreadcrumb and FRHIBreadcrumbNode have been merged into one type.
	using FRHIBreadcrumb = FRHIBreadcrumbNode;

	template <typename TDesc, typename... TValues>
	using TRHIBreadcrumbInitializer = std::tuple<TDesc const*, std::tuple<TValues...>, UE::RHI::GPUProfiler::FGPUStat*>;

	class FRHIBreadcrumbAllocatorArray : public TArray<TSharedRef<class FRHIBreadcrumbAllocator>, TInlineAllocator<2>>
	{
	public:
		inline void AddUnique(FRHIBreadcrumbAllocator* Allocator);
		inline bool Contains(FRHIBreadcrumbAllocator* Allocator);
	};

	class FRHIBreadcrumbAllocator : public TSharedFromThis<FRHIBreadcrumbAllocator>
	{
		static thread_local FRHIBreadcrumbAllocatorArray PendingDestruction;
		static thread_local int32 DestructionDepth;

		friend FRHIBreadcrumbNode;

		FMemStackBase Inner;
		FRHIBreadcrumbAllocatorArray Parents;

	public:
		RHI_API ~FRHIBreadcrumbAllocator();

		FRHIBreadcrumbAllocatorArray const& GetParents() const { return Parents; }

		template <typename TDesc, typename... TValues>
		inline FRHIBreadcrumbNode* AllocBreadcrumb(TRHIBreadcrumbInitializer<TDesc, TValues...> const& Args);

	#if ENABLE_RHI_VALIDATION
		// Used by RHI validation for circular reference detection.
		bool bVisited = false;
	#endif
	};

	inline void FRHIBreadcrumbAllocatorArray::AddUnique(FRHIBreadcrumbAllocator* Allocator)
	{
		if (!Contains(Allocator))
		{
			Add(Allocator->AsShared());
		}
	}

	inline bool FRHIBreadcrumbAllocatorArray::Contains(FRHIBreadcrumbAllocator* Allocator)
	{
		for (TSharedRef<FRHIBreadcrumbAllocator> const& Existing : *this)
		{
			if (Allocator == &Existing.Get())
			{
				return true;
			}
		}

		return false;
	}

	//
	// A linked list of breadcrumb nodes.
	// Nodes may only be attached to one list at a time.
	//
	struct FRHIBreadcrumbList
	{
		FRHIBreadcrumbNode* First = nullptr;
		FRHIBreadcrumbNode* Last = nullptr;

		void Append(FRHIBreadcrumbNode* Node)
		{
			check(Node && Node != FRHIBreadcrumbNode::Sentinel);
			check(!Node->ListLink);

			if (!First)
			{
				First = Node;
			}

			if (Last)
			{
				Last->ListLink = Node;
			}
			Last = Node;
		}

		[[nodiscard]] auto IterateAndUnlink()
		{
			struct FResult
			{
				FRHIBreadcrumbNode* First;

				auto begin() const
				{
					struct FIterator
					{
						FRHIBreadcrumbNode* Current;
						FRHIBreadcrumbNode* Next;

						FIterator& operator++()
						{
							Current = Next;
							if (Current)
							{
								Next = Current->ListLink;
								Current->ListLink = nullptr;
							}
							else
							{
								Next = nullptr;
							}

							return *this;
						}

						bool operator != (std::nullptr_t) const
						{
							return Current != nullptr;
						}

						FRHIBreadcrumbNode* operator*() const
						{
							return Current;
						}
					};

					FIterator Iterator { nullptr, First };
					++Iterator;
					return Iterator;
				}

				std::nullptr_t end() const { return nullptr; }
			};

			FResult Result { First };
			First = nullptr;
			Last = nullptr;
			return Result;
		}
		
	};

	//
	// A range of breadcrumb nodes for a given GPU pipeline.
	//
	struct FRHIBreadcrumbRange
	{
		FRHIBreadcrumbNode* First;
		FRHIBreadcrumbNode* Last;

		FRHIBreadcrumbRange() = default;

		FRHIBreadcrumbRange(FRHIBreadcrumbNode* SingleNode)
			: First(SingleNode)
			, Last(SingleNode)
		{}

		FRHIBreadcrumbRange(FRHIBreadcrumbNode* First, FRHIBreadcrumbNode* Last)
			: First(First)
			, Last(Last)
		{}

		//
		// Links the nodes in the 'Other' range into this range, after the node specified by 'Prev'.
		// If 'Prev' is nullptr, the other nodes will be inserted at the start of the range.
		//
		void InsertAfter(FRHIBreadcrumbRange const& Other, FRHIBreadcrumbNode* Prev, ERHIPipeline Pipeline)
		{
			// Either both are nullptr, or both are valid
			check(!Other.First == !Other.Last);
			check(!First == !Last);

			if (!Other.First)
			{
				// Other range has no nodes, nothing to do.
				return; 
			}

			// Other range should not already be linked beyond its end.
			check(!Other.Last->GetNextPtr(Pipeline));

			if (!Prev)
			{
				// Insert at the front of the range
				Other.Last->GetNextPtr(Pipeline) = First;
				First = Other.First;

				if (!Last)
				{
					Last = Other.Last;
				}
			}
			else
			{
				// Insert after 'Prev' node

				// We shouldn't have a 'Prev' node if the outer range is empty.
				check(First);

				FRHIBreadcrumbNode* Next = Prev->GetNextPtr(Pipeline);
				Prev->GetNextPtr(Pipeline) = Other.First;
				Other.Last->GetNextPtr(Pipeline) = Next;

				if (Last == Prev)
				{
					// Range was inserted after all other nodes. Update Last pointer.
					Last = Other.Last;
				}
			}
		}

		class FOuter;
		FOuter Enumerate(ERHIPipeline Pipeline) const;

		operator bool() const { return First != nullptr; }

		bool operator == (FRHIBreadcrumbRange const& RHS) const
		{
			return First == RHS.First && Last == RHS.Last;
		}

		bool operator != (FRHIBreadcrumbRange const& RHS) const
		{
			return !(*this == RHS);
		}

		friend uint32 GetTypeHash(FRHIBreadcrumbRange const& Range)
		{
			return HashCombineFast(GetTypeHash(Range.First), GetTypeHash(Range.Last));
		}
	};
	
	class FRHIBreadcrumbRange::FOuter
	{
		FRHIBreadcrumbRange const Range;
		ERHIPipeline const Pipeline;

	public:
		FOuter(FRHIBreadcrumbRange const& Range, ERHIPipeline Pipeline)
			: Range(Range)
			, Pipeline(Pipeline)
		{}

		auto begin() const
		{
			struct FIterator
			{
				FRHIBreadcrumbNode* Current;
				FRHIBreadcrumbNode* const Last;
#if DO_CHECK
				FRHIBreadcrumbNode* const First;
#endif
				ERHIPipeline const Pipeline;

				bool operator != (std::nullptr_t) const
				{
					return Current != nullptr;
				}

				FRHIBreadcrumbNode* operator*() const
				{
					return Current;
				}

				FIterator& operator++()
				{
					if (Current == Last)
					{
						Current = nullptr;
					}
					else
					{
						FRHIBreadcrumbNode* Next = Current->GetNextPtr(Pipeline);

						// Next should never be null here. When iterating a non-empty range, we should always expect to reach 'Last' rather than nullptr.
						checkf(Next, TEXT("Nullptr 'Next' breadcrumb found before reaching the 'Last' breadcrumb in the range. (First: 0x%p, Last: 0x%p, Current: 0x%p)"), First, Last, Current);

						Current = Next;
					}

					return *this;
				}
			};

			return FIterator
			{
					Range.First
				, Range.Last
			#if DO_CHECK
				, Range.First
			#endif
				, Pipeline
			};
		}

		constexpr std::nullptr_t end() const
		{
			return nullptr;
		}
	};

	inline FRHIBreadcrumbRange::FOuter FRHIBreadcrumbRange::Enumerate(ERHIPipeline Pipeline) const
	{
		// Either both must be null, or both must be non-null
		check(!First == !Last);

		return FOuter { *this, Pipeline };
	}

	inline void FRHIBreadcrumbNode::SetParent(FRHIBreadcrumbNode* Node)
	{
		check(Parent == nullptr || Parent == FRHIBreadcrumbNode::Sentinel);
		Parent = Node;

		if (Parent && Parent != FRHIBreadcrumbNode::Sentinel && Parent->Allocator != Allocator)
		{
			Allocator->Parents.AddUnique(Parent->Allocator);
		}
	}

	inline void FRHIBreadcrumbNode::TraceBeginCPU() const
	{
#if RHI_BREADCRUMBS_EMIT_CPU
		if (TraceCpuSpecId)
		{
			if (TraceCpuMetadataId > 0)
			{
				FCpuProfilerTrace::OutputBeginEventWithMetadata(TraceCpuMetadataId);
			}
			else
			{
				FCpuProfilerTrace::OutputBeginEvent(TraceCpuSpecId);
			}
		}
#endif
	}
	inline void FRHIBreadcrumbNode::TraceEndCPU() const
	{
#if RHI_BREADCRUMBS_EMIT_CPU
		if (TraceCpuSpecId)
		{
			if (TraceCpuMetadataId > 0)
			{
				FCpuProfilerTrace::OutputEndEventWithMetadata();
			}
			else
			{
				FCpuProfilerTrace::OutputEndEvent();
			}
		}
#endif
	}

	inline void FRHIBreadcrumbNode::WalkIn(FRHIBreadcrumbNode const* Node)
	{
	#if RHI_BREADCRUMBS_EMIT_CPU
		if (TRACE_CPUPROFILER_EVENT_MANUAL_IS_ENABLED())
		{
			auto Recurse = [](auto const& Recurse, FRHIBreadcrumbNode const* Current) -> void
			{
				if (!Current || Current == Sentinel)
					return;

				Recurse(Recurse, Current->GetParent());
				Current->TraceBeginCPU();
			};
			Recurse(Recurse, Node);
		}
	#endif
	}

	inline void FRHIBreadcrumbNode::WalkInRange(FRHIBreadcrumbNode const* Leaf, FRHIBreadcrumbNode const* Root)
	{
		check(Leaf && Root);

	#if RHI_BREADCRUMBS_EMIT_CPU
		if (TRACE_CPUPROFILER_EVENT_MANUAL_IS_ENABLED())
		{
			auto Recurse = [&](auto const& Recurse, FRHIBreadcrumbNode const* Current) -> void
			{
				if (Current != Root)
				{
					Recurse(Recurse, Current->GetParent());
				}

				Current->TraceBeginCPU();
			};
			Recurse(Recurse, Leaf);
		}
	#endif
	}

	inline void FRHIBreadcrumbNode::WalkOut(FRHIBreadcrumbNode const* Node)
	{
	#if RHI_BREADCRUMBS_EMIT_CPU
		if (TRACE_CPUPROFILER_EVENT_MANUAL_IS_ENABLED())
		{
			while (Node && Node != Sentinel)
			{
				Node->TraceEndCPU();
				Node = Node->GetParent();
			}
		}
	#endif
	}

	inline void FRHIBreadcrumbNode::WalkOutRange(FRHIBreadcrumbNode const* Leaf, FRHIBreadcrumbNode const* Root)
	{
		check(Leaf && Root);

	#if RHI_BREADCRUMBS_EMIT_CPU
		if (TRACE_CPUPROFILER_EVENT_MANUAL_IS_ENABLED())
		{
			while (true)
			{
				Leaf->TraceEndCPU();
				if (Leaf == Root)
					break;

				Leaf = Leaf->GetParent();
			}
		}
	#endif
	}

	namespace UE::RHI::Breadcrumbs::Private
	{
		// Replacement for std::remove_cvref, since this isn't available until C++20.
		template <typename T>
		using TRemoveCVRef = std::remove_cv_t<std::remove_reference_t<T>>;

		// Used to control the static_asserts below.
		template <typename...>
		struct TFalse { static constexpr bool Value = false; };

		//
		// The TValue types are used to capture and store vararg format values in RHI breadcrumbs. Capturing values like this means we can avoid the string
		// formatting cost until we actually need the final string (e.g. we're emitting breadcrumbs to an external profiler, we're running 'profilegpu' etc.).
		// 
		// The inner FConvert type is used to prepare the TValue when calling FCString::Snprintf to generate the final breadcrumb string.
		// 
		// This base definition catches all integer, float and enum values.
		//
		template <typename T>
		struct TValue
		{
			static constexpr bool bValidType = std::is_integral_v<T> || std::is_floating_point_v<T> || std::is_enum_v<T>;

			T const Value;

			template <typename TArg>
			TValue(TArg const& Value)
				: Value(Value)
			{
				static_assert(bValidType || TFalse<TArg>::Value, "Type is not compatible with RHI breadcrumbs.");
			}

			struct FConvert
			{
				T const& Inner;
				FConvert(TValue const& Value)
					: Inner(Value.Value)
				{}
			};

			void Serialize(UE::RHI::GPUProfiler::FMetadataSerializer& Serializer) const
			{
				Serializer.AppendValue(Value);
			}
		};

		// Disallow all pointer types. Provide a specific assert message for TCHAR pointers.
		template <typename T>
		struct TValue<T*>
		{
			static constexpr bool bValidType = false;

			template <typename... TArgs>
			TValue(TArgs&&...)
			{
				if constexpr (std::is_same_v<std::remove_const_t<T>, TCHAR>)
				{
					static_assert(TFalse<TArgs...>::Value,
						"Do not use raw TCHAR pointers with RHI breadcrumbs. Pass the FString, FName, or string literal instead. If you are certain your TCHAR pointer is a string literal "
						"(e.g. from a function returning a literal) and you cannot pass that literal directly to an RHI breadcrumb, use the RHI_BREADCRUMB_FORCE_STRING_LITERAL macro to silence "
						"this static assert. Incorrect use of RHI_BREADCRUMB_FORCE_STRING_LITERAL will lead to use-after-free, as only the raw pointer is retained by the breadcrumb."
					);
				}
				else
				{			
					static_assert(TFalse<TArgs...>::Value, "RHI breadcrumbs do not support arbitrary pointer types.");
				}
			}

			struct FConvert
			{
				uint32 Inner;
				template <typename... TArgs>
				FConvert(TArgs&&...)
					: Inner(0)
				{}
			};

			void Serialize(UE::RHI::GPUProfiler::FMetadataSerializer& Serializer) const {}
		};

		// String literal - keep the string pointer
		template <size_t N>
		struct TValue<TCHAR[N]>
		{
			static constexpr bool bValidType = true;

			TCHAR const(&Value)[N];

			TValue(TCHAR const(&Value)[N])
				: Value(Value)
			{}

			struct FConvert
			{
				TCHAR const(&Inner)[N];
				FConvert(TValue const& Value)
					: Inner(Value.Value)
				{}
			};

			void Serialize(UE::RHI::GPUProfiler::FMetadataSerializer& Serializer) const
			{
				Serializer.AppendValue(Value);
			}
		};

		// FName - keep the FName itself and defer resolving
		template <>
		struct TValue<FName>
		{
			static constexpr bool bValidType = true;

			FName Value;
			TValue(FName const& Value)
				: Value(Value)
			{}

			struct FConvert
			{
				TCHAR Inner[FRHIBreadcrumb::MaxLength];
				FConvert(TValue const& Name)
				{
					Name.Value.ToStringTruncate(Inner);
				}
			};

			void Serialize(UE::RHI::GPUProfiler::FMetadataSerializer& Serializer) const
			{
				Serializer.AppendValue(Value);
			}
		};

		// FDebugName - keep the FDebugName itself and defer resolving
		template <>
		struct TValue<FDebugName>
		{
			static constexpr bool bValidType = true;

			FDebugName Value;
			TValue(FDebugName const& Value)
				: Value(Value)
			{}

			struct FConvert
			{
				TCHAR Inner[FRHIBreadcrumb::MaxLength];
				FConvert(TValue const& Name)
				{
					Name.Value.ToStringTruncate(Inner);
				}
			};

			void Serialize(UE::RHI::GPUProfiler::FMetadataSerializer& Serializer) const
			{
				Serializer.AppendValue(Value);
			}
		};

		// FString - Take an immediate copy of the string. Total length is limited by fixed buffer size.
		template <>
		struct TValue<FString>
		{
			static constexpr bool bValidType = true;

			TCHAR Buffer[FRHIBreadcrumb::MaxLength];
			TValue(FString const& Value)
			{
				FCString::Strncpy(Buffer, *Value, UE_ARRAY_COUNT(Buffer));
			}

			struct FConvert
			{
				TCHAR const* Inner;
				FConvert(TValue const& String)
					: Inner(String.Buffer)
				{}
			};

			void Serialize(UE::RHI::GPUProfiler::FMetadataSerializer& Serializer) const
			{
				Serializer.AppendValue(Buffer);
			}
		};

		// Determines if a vararg type is a string literal / value tuple, as defined by RHI_BREADCRUMB_FIELD.
		template <          typename T> struct TIsField                                   { static constexpr bool Value = false; };
		template <size_t N, typename T> struct TIsField<std::tuple<TCHAR const(&)[N], T>> { static constexpr bool Value = true;  };

		// Determines the TValue type used to store a vararg value in a breadcrumb. Also unpacks the value type from RHI_BREADCRUMB_FIELD tuples.
		template <          typename T> struct TGetValueType                                   { using TType = TValue<TRemoveCVRef<T>>; };
		template <size_t N, typename T> struct TGetValueType<std::tuple<TCHAR const(&)[N], T>> { using TType = TValue<TRemoveCVRef<T>>; };

		// Helper to concatenate index sequences.
		template <size_t  , typename...  > struct TConcatIndexSequence;
		template <size_t N, size_t... Seq> struct TConcatIndexSequence<N, std::index_sequence<Seq...>> { using TType = std::index_sequence<N, Seq...>; };

		// Generates a std::index_sequence containing the indices of the RHI_BREADCRUMB_FIELD tuples in a parameter pack.
		template <size_t N, typename...                       > struct TFindFieldIndices;
		template <size_t N                                    > struct TFindFieldIndices<N                  > { using TType = std::index_sequence<>; };
		template <size_t N, typename TFirst, typename... TRest> struct TFindFieldIndices<N, TFirst, TRest...>
		{
			using TType = typename std::conditional<TIsField<TFirst>::Value
				, typename TConcatIndexSequence<N, typename TFindFieldIndices<N + 1, TRest...>::TType>::TType
				, typename TFindFieldIndices<N + 1, TRest...>::TType
			>::type;
		};

		struct FForceNoSprintf {};

		// Infers the size of a string literal character array. Size is zero for any type that is not a string literal.
		template <typename   > struct TStringLiteralSize                       { static constexpr size_t Value = 0;    };
		template <size_t Size> struct TStringLiteralSize<TCHAR const(&)[Size]> { static constexpr size_t Value = Size; };

		//
		// Helper traits for dealing wth varargs values. The std::tuple specializations are for the RHI_BREADCRUMB_FIELD values.
		//
		//		- TDescType    : Type passed to the TRHIBreadcrumbDesc template.
		//		- TValueType   : The under-lying non-reference type of the vararg. Fields are unpacked.
		//		- TValueRef    : A reference to the TValueType type.
		//		- ForwardValue : Similar to std::forward, but also unpacks and forwards the value from a std::tuple RHI_BREADCRUMB_FIELD.
		//
		template <          class T> struct TFieldTraits                                       { using TDescType = TRemoveCVRef<T>;                                using TValueType = T; using TValueRef = T  ; static TValueRef ForwardValue(auto&& Arg) { return std::forward<TValueRef>(Arg); } };
		template <          class T> struct TFieldTraits<T& >                                  { using TDescType = TRemoveCVRef<T>;                                using TValueType = T; using TValueRef = T& ; static TValueRef ForwardValue(auto&& Arg) { return std::forward<TValueRef>(Arg); } };
		template <          class T> struct TFieldTraits<T&&>                                  { using TDescType = TRemoveCVRef<T>;                                using TValueType = T; using TValueRef = T&&; static TValueRef ForwardValue(auto&& Arg) { return std::forward<TValueRef>(Arg); } };
		template <size_t N, class T> struct TFieldTraits<std::tuple<TCHAR const(&)[N], T  >&&> { using TDescType = std::tuple<TCHAR const(&)[N], TRemoveCVRef<T>>; using TValueType = T; using TValueRef = T  ; static TValueRef ForwardValue(auto&& Arg) { return std::forward<TValueRef>(std::get<1>(Arg)); } };
		template <size_t N, class T> struct TFieldTraits<std::tuple<TCHAR const(&)[N], T& >&&> { using TDescType = std::tuple<TCHAR const(&)[N], TRemoveCVRef<T>>; using TValueType = T; using TValueRef = T& ; static TValueRef ForwardValue(auto&& Arg) { return std::forward<TValueRef>(std::get<1>(Arg)); } };
		template <size_t N, class T> struct TFieldTraits<std::tuple<TCHAR const(&)[N], T&&>&&> { using TDescType = std::tuple<TCHAR const(&)[N], TRemoveCVRef<T>>; using TValueType = T; using TValueRef = T&&; static TValueRef ForwardValue(auto&& Arg) { return std::forward<TValueRef>(std::get<1>(Arg)); } };

		// The breadcrumb macros work with both RHI command lists and RHI contexts. This helper retrieves the relevant RHI command list from both types.
		inline FRHIComputeCommandList& GetRHICmdList(FRHIComputeCommandList& RHICmdList);
		inline FRHIComputeCommandList& GetRHICmdList(IRHIComputeContext    & RHIContext);

		// Returns the full path string for the breadcrumb currently at the top of the CPU stack,
		// for either RHI command lists or RHI contexts. Used by the breadcrumb check macros.
		inline FString GetSafeBreadcrumbPath(auto&& RHICmdList_Or_RHIContext);

		template <size_t Size, bool bHasVarargs>
		struct TFormatString
		{
			TCHAR const(&FormatString)[Size];

			TFormatString(TCHAR const(&FormatString)[Size])
				: FormatString(FormatString)
			{}

		protected:
			template <typename TValuesTuple, size_t... Indices>
			TCHAR const* ToStringImpl(TCHAR const* StaticName, FRHIBreadcrumbNode::FBuffer& Buffer, TValuesTuple const& Values, std::index_sequence<Indices...>) const
			{
				// Perform type conversions (call ToString() on FName etc)
				std::tuple<typename std::tuple_element_t<Indices, TValuesTuple>::FConvert...> Converted
				{
					std::get<Indices>(Values)...
				};

				FCString::Snprintf(
					Buffer.Data
					, UE_ARRAY_COUNT(Buffer.Data)
					, FormatString
					, (std::get<Indices>(Converted).Inner)...
				);

				return Buffer.Data;
			}
		};

		template <>
		struct TFormatString<0, true>
		{
			UE_DEPRECATED(5.6,
				"Use of the non-\"_F\" versions of the RHI_BREADCRUMB_EVENT family of macros with printf varargs has been deprecated. "
				"The additional values passed to these macros will be ignored, and the raw printf format string will form the name of the breadcrumb. "
				"Use the \"_F\" versions of these macros instead (which require both a static name and a format string), or remove the varargs.")
			TFormatString(std::nullptr_t)
			{}

			TFormatString(FForceNoSprintf&&)
			{}

			template <typename TValuesTuple, size_t... Indices>
			TCHAR const* ToStringImpl(TCHAR const* StaticName, FRHIBreadcrumbNode::FBuffer& Buffer, TValuesTuple const& Values, std::index_sequence<Indices...>) const
			{
				if constexpr (std::tuple_size_v<TValuesTuple> == 1)
				{
					using TConvert = typename std::tuple_element_t<0, TValuesTuple>::FConvert;

					if constexpr (std::is_same_v<decltype(std::declval<TConvert>().Inner), TCHAR const*>)
					{
						// The breadcrumb has no format string, and a single vararg which looks like a null terminated TCHAR const* string pointer.
						// Just return the pointer directly from the value.
						return TConvert(std::get<0>(Values)).Inner;
					}
					else
					{
						return StaticName;
					}
				}
				else
				{
					return StaticName;
				}
			}
		};

		template <>
		struct TFormatString<0, false>
		{
			TFormatString(std::nullptr_t)
			{}

			template <typename TValuesTuple, size_t... Indices>
			TCHAR const* ToStringImpl(TCHAR const* StaticName, FRHIBreadcrumbNode::FBuffer& Buffer, TValuesTuple const& Values, std::index_sequence<Indices...>) const
			{
				return StaticName;
			}
		};

		// Returns true if the given args are compatible with RHI breadcrumbs, i.e. they have matching TValue specializations.
		template <typename... TArgs> struct TIsValidArgs   { static constexpr bool Value = (TGetValueType<typename TFieldTraits<TArgs>::TDescType>::TType::bValidType && ...); };
		template <                 > struct TIsValidArgs<> { static constexpr bool Value = true; };

		//
		// Contains the definition of an RHI breadcrumb (i.e. the value types, number and name of fields, etc).
		// These are instantiated as static objects in RHI_BREADCRUMB_PRIVATE_DEFINE, which is used by the breadcrumb macros.
		//
		template <size_t FormatStringSize, typename... TValues>
		class TRHIBreadcrumbDesc final : public FRHIBreadcrumbData, public TFormatString<FormatStringSize, sizeof...(TValues) != 0>
		{
			using TFormatString = TFormatString<FormatStringSize, sizeof...(TValues) != 0>;

			mutable FCriticalSection Cs;

			template <typename TTuple, size_t... Indices>
			static auto ExtractFieldNames(TTuple&& Tuple, std::index_sequence<Indices...>)
			{
				return std::array<TCHAR const*, sizeof...(Indices)>
				{
					std::get<0>(std::get<Indices>(Tuple))...
				};
			}

		public:
			// Tuple of vararg value types to store in the breadcrumb. Field tuples have been unpacked and the name component discarded.
			using TValuesTuple = std::tuple<typename TGetValueType<TValues>::TType...>;
			static constexpr size_t NumValues = sizeof...(TValues);

			// std::index_sequence for all vararg values.
			using TValuesIndexSequence = std::index_sequence_for<TValues...>;

			// std::index_sequence defining the indices of the tuple field values
			using TFieldsIndexSequence = typename TFindFieldIndices<0, TValues...>::TType;
			static constexpr size_t NumFields = TFieldsIndexSequence::size();

			// Array of the string literal field names
			std::array<TCHAR const*, NumFields> const FieldNames;

			// Id of the spec of the GPU events associated with this breadcrumb.
			mutable uint32 TraceGpuSpecId = 0;
			
			// Id of the spec of the CPU events associated with this breadcrumb.
			mutable std::atomic<uint32> TraceCpuSpecId = 0;

			template <typename... TInnerValues>
			TRHIBreadcrumbDesc(FRHIBreadcrumbData&& BaseData, TFormatString FormatString, TInnerValues&&... Values)
				: FRHIBreadcrumbData(MoveTemp(BaseData))
				, TFormatString(FormatString)
				, FieldNames(ExtractFieldNames(std::forward_as_tuple(Values...), TFieldsIndexSequence()))
			{}

			TCHAR const* ToString(FRHIBreadcrumbNode::FBuffer& Buffer, TValuesTuple const& Values) const
			{
				return TFormatString::ToStringImpl(StaticName, Buffer, Values, TValuesIndexSequence());
			}

			void SerializeValues(UE::RHI::GPUProfiler::FMetadataSerializer& Serializer, TValuesTuple const& Values) const
			{
				std::apply([&](const auto&... Value)
					{
						(SerializeValue(Value, Serializer), ...);
					}, Values);
			}

			uint32 GetTraceGpuSpec() const
			{
				if (TraceGpuSpecId == 0 && UE::RHI::GPUProfiler::FGpuProfilerTrace::IsAvailable())
				{
					if constexpr (FormatStringSize > 0)
					{
						TraceGpuSpecId = UE::RHI::GPUProfiler::FGpuProfilerTrace::BreadcrumbSpec(StaticName, TFormatString::FormatString, FieldNames);
					}
					else
					{
						TraceGpuSpecId = UE::RHI::GPUProfiler::FGpuProfilerTrace::BreadcrumbSpec(StaticName, TEXT(""), FieldNames);
					}
				}

				return TraceGpuSpecId;
			}

			uint32 GetTraceCpuSpec() const
			{
#if CPUPROFILERTRACE_ENABLED
				if (TraceCpuSpecId == 0 && bool(CpuChannel))
				{
					FScopeLock Lock(&Cs);

					if (TraceCpuSpecId == 0)
					{
						TraceCpuSpecId = FCpuProfilerTrace::OutputEventType(StaticName
#if RHI_BREADCRUMBS_EMIT_LOCATION
							, File, Line
#endif
						);

						if constexpr (FormatStringSize > 0)
						{
							UE::RHI::GPUProfiler::FMetadataSerializer Serializer;
							for (const TCHAR* FieldName : FieldNames)
							{
								Serializer.AppendValue(FieldName);
							}
							FCpuProfilerTrace::OutputEventMetadataSpec(TraceCpuSpecId, StaticName, TFormatString::FormatString, Serializer.GetData());
						}
					}
				}
#endif
				return TraceCpuSpecId;
			}

		private:
			template<typename T>
			void SerializeValue(const T& Item, UE::RHI::GPUProfiler::FMetadataSerializer& Serializer) const
			{
				Item.Serialize(Serializer);
			}
		};

		// Breadcrumb implementation for printf formatted names
		// Privately inherit from the TValuesTuple to use empty-base-class optimization for breadcrumbs with no varargs.
		template <typename TDesc>
		class TRHIBreadcrumb final : public FRHIBreadcrumbNode, private TDesc::TValuesTuple
		{
			friend FRHIBreadcrumbAllocator;
			friend FRHIBreadcrumbNode;

			TDesc const& Desc;

			TRHIBreadcrumb(TRHIBreadcrumb const&) = delete;
			TRHIBreadcrumb(TRHIBreadcrumb&&     ) = delete;

		public:
			template <typename... TValues>
			TRHIBreadcrumb(FRHIBreadcrumbAllocator& Allocator, TDesc const& Desc, UE::RHI::GPUProfiler::FGPUStat* InGPUStat, TValues&&... Values)
				: FRHIBreadcrumbNode(Desc, Allocator, InGPUStat)
				, TDesc::TValuesTuple(Forward<TValues>(Values)...)
				, Desc(Desc)
			{
#if RHI_BREADCRUMBS_EMIT_CPU
				if (bool(CpuChannel))
				{
					TraceCpuSpecId = Desc.GetTraceCpuSpec();
					if (TraceCpuSpecId)
					{
						if (Desc.NumValues > 0)
						{
							UE::RHI::GPUProfiler::FMetadataSerializer Serializer;
							Desc.SerializeValues(Serializer, *this);
							TraceCpuMetadataId = FCpuProfilerTrace::OutputMetadata(TraceCpuSpecId, Serializer.GetData());
						}
					}
				}
#endif
			}

			TCHAR const* GetTCHAR(FBuffer& Buffer) const override
			{
				return Desc.ToString(Buffer, *this);
			}

			virtual void TraceBeginGPU(uint32 QueueId, uint64 GPUTimestampTOP) const override
			{
				if (uint32 SpecId = Desc.GetTraceGpuSpec())
				{
					UE::RHI::GPUProfiler::FMetadataSerializer Serializer;
					Desc.SerializeValues(Serializer, *this);
					UE::RHI::GPUProfiler::FGpuProfilerTrace::BeginBreadcrumb(SpecId, QueueId, GPUTimestampTOP, Serializer.GetData());
				}
			}

			virtual void TraceEndGPU(uint32 QueueId, uint64 GPUTimestampBOP) const override
			{
				if (Desc.TraceGpuSpecId)
				{
					UE::RHI::GPUProfiler::FGpuProfilerTrace::EndBreadcrumb(QueueId, GPUTimestampBOP);
				}
			}

		private:
			// Constructor for the sentinel value
			TRHIBreadcrumb(TDesc const& Desc)
				: FRHIBreadcrumbNode(Desc)
				, Desc(Desc)
			{}
		};
	}

	class FRHIBreadcrumbNodeRef
	{
	private:
		FRHIBreadcrumbNode* Node = nullptr;
		TSharedPtr<FRHIBreadcrumbAllocator> AllocatorRef;

	public:
		FRHIBreadcrumbNodeRef() = default;
		FRHIBreadcrumbNodeRef(FRHIBreadcrumbNode* Node)
			: Node(Node)
		{
			if (Node && Node != FRHIBreadcrumbNode::Sentinel)
			{
				AllocatorRef = Node->Allocator->AsShared();
			}
		}

		operator FRHIBreadcrumbNode* () const { return Node; }
		operator bool() const { return !!Node; }

		FRHIBreadcrumbNode* operator -> () const { return Node; }
		FRHIBreadcrumbNode* Get() const { return Node; }
	};

	struct FRHIBreadcrumbScope
	{
		FRHIComputeCommandList& RHICmdList;
		FRHIBreadcrumbNode* const Node;

	private:
		FRHIBreadcrumbScope(FRHIComputeCommandList& RHICmdList, FRHIBreadcrumbNode* Node);

	public:
		template <typename TDesc, typename... TValues>
		inline FRHIBreadcrumbScope(FRHIComputeCommandList& RHICmdList, TRHIBreadcrumbInitializer<TDesc, TValues...>&& Args);
		inline ~FRHIBreadcrumbScope();
	};

	//
	// A helper class to manually create, begin and end a breadcrumb on a given RHI command list.
	// For use in places where the Begin/End operations are separate, and a scoped breadcrumb event is not appropriate.
	//
	class FRHIBreadcrumbEventManual
	{
		// Must be a reference. End() may be called with a different RHI command list than the one we 
		// received in the constructor, so we need to keep the underlying RHI breadcrumb allocator alive.
		FRHIBreadcrumbNodeRef Node;
	#if DO_CHECK
		ERHIPipeline const Pipeline;
		uint32 ThreadId;
	#endif

		inline FRHIBreadcrumbEventManual(FRHIComputeCommandList& RHICmdList, FRHIBreadcrumbNode* Node);

	public:
		template <typename TDesc, typename... TValues>
		inline FRHIBreadcrumbEventManual(FRHIComputeCommandList& RHICmdList, TRHIBreadcrumbInitializer<TDesc, TValues...>&& Args);

		inline void End(FRHIComputeCommandList& RHICmdList);

		inline ~FRHIBreadcrumbEventManual();
	};

	// Private macro used to define a new breadcrumb type. Also forwards the given values through to a returned TRHIBreadcrumbInitializer,
	// which can then be passed to the constructors for FRHIBreadcrumbScope / FRHIBreadcrumbEventManual, or the allocator AllocBreadcrumb() function.
	#define RHI_BREADCRUMB_PRIVATE_DEFINE(StaticName, FormatString, ValueType, GPUStat)	\
		[&](auto&&... Values)															\
		{																				\
			using namespace UE::RHI::Breadcrumbs::Private;								\
																						\
			TRHIBreadcrumbDesc<															\
				TStringLiteralSize<decltype(FormatString)>::Value,						\
				typename TFieldTraits<decltype(Values)>::TDescType...					\
			> static const Desc(														\
				  FRHIBreadcrumbData(StaticName, __FILE__, __LINE__)					\
				, FormatString															\
				, Values...																\
			);																			\
																						\
			return std::tuple(															\
				&Desc,																	\
				std::tuple<typename TFieldTraits<decltype(Values)>::ValueType...>(		\
					TFieldTraits<decltype(Values)>::ForwardValue(Values)...				\
				),																		\
				GPUStat																	\
			);																			\
		}

	#define RHI_BREADCRUMB_DESC_FORWARD_VALUES(StaticName, FormatString, GPUStat) RHI_BREADCRUMB_PRIVATE_DEFINE(StaticName, FormatString, TValueRef , GPUStat)
	#define RHI_BREADCRUMB_DESC_COPY_VALUES(   StaticName, FormatString, GPUStat) RHI_BREADCRUMB_PRIVATE_DEFINE(StaticName, FormatString, TValueType, GPUStat)

	#if HAS_GPU_STATS
		#define RHI_GPU_STAT_ARGS(StatName) (static_cast<UE::RHI::GPUProfiler::FGPUStat*>(&GPUStat_##StatName))
		#define RHI_GPU_STAT_ARGS_NONE      (static_cast<UE::RHI::GPUProfiler::FGPUStat*>(nullptr))
	#else
		#define RHI_GPU_STAT_ARGS(StatName) (static_cast<UE::RHI::GPUProfiler::FGPUStat*>(nullptr))
		#define RHI_GPU_STAT_ARGS_NONE      (static_cast<UE::RHI::GPUProfiler::FGPUStat*>(nullptr))
	#endif

	// Varargs in breadcrumb macros can be given a name by wrapping them with this macro.
	// Named fields are exposed to Unreal Insights as metadata on event markers.
	#define RHI_BREADCRUMB_FIELD(Name, Value) std::forward_as_tuple(TEXT(Name), Value)

	#define RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, Stat, Condition, StaticName, Format, ...)\
		TOptional<FRHIBreadcrumbScope> UE_JOIN(BreadcrumbScope, __LINE__);										 \
		do																										 \
		{																										 \
			if (Condition)																						 \
			{																									 \
				UE_JOIN(BreadcrumbScope, __LINE__).Emplace(														 \
					UE::RHI::Breadcrumbs::Private::GetRHICmdList(RHICmdList_Or_RHIContext),						 \
					RHI_BREADCRUMB_DESC_FORWARD_VALUES(															 \
						  StaticName																			 \
						, Format																				 \
						, Stat																					 \
					)(__VA_ARGS__)																				 \
				);																								 \
			}																									 \
		} while(false)

#endif // WITH_RHI_BREADCRUMBS

#if WITH_RHI_BREADCRUMBS_FULL

	// Note, the varargs are deprecated and ignored in these macros.
	#define RHI_BREADCRUMB_EVENT(                             RHICmdList_Or_RHIContext,                  StaticName,         ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS_NONE ,      true, TEXT(StaticName),      nullptr, ##__VA_ARGS__)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL(                 RHICmdList_Or_RHIContext,       Condition, StaticName,         ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS_NONE , Condition, TEXT(StaticName),      nullptr, ##__VA_ARGS__)
	#define RHI_BREADCRUMB_EVENT_STAT(                        RHICmdList_Or_RHIContext, Stat,            StaticName,         ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS(Stat),      true, TEXT(StaticName),      nullptr, ##__VA_ARGS__)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_STAT(            RHICmdList_Or_RHIContext, Stat, Condition, StaticName,         ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS(Stat), Condition, TEXT(StaticName),      nullptr, ##__VA_ARGS__)
													          
	// Format versions of the breadcrumb macros.	          
	#define RHI_BREADCRUMB_EVENT_F(                           RHICmdList_Or_RHIContext,                  StaticName, Format, ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS_NONE ,      true, TEXT(StaticName), TEXT(Format), ##__VA_ARGS__)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_F(               RHICmdList_Or_RHIContext,       Condition, StaticName, Format, ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS_NONE , Condition, TEXT(StaticName), TEXT(Format), ##__VA_ARGS__)
	#define RHI_BREADCRUMB_EVENT_STAT_F(                      RHICmdList_Or_RHIContext, Stat,            StaticName, Format, ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS(Stat),      true, TEXT(StaticName), TEXT(Format), ##__VA_ARGS__)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_STAT_F(          RHICmdList_Or_RHIContext, Stat, Condition, StaticName, Format, ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS(Stat), Condition, TEXT(StaticName), TEXT(Format), ##__VA_ARGS__)

	// Used only for back compat with SCOPED_DRAW_EVENTF
	#define RHI_BREADCRUMB_EVENT_F_STR_DEPRECATED(            RHICmdList_Or_RHIContext,                  StaticName, Format, ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS_NONE ,      true, TEXT(StaticName),      Format , ##__VA_ARGS__)
	#define RHI_BREADCRUMB_EVENT_F_CONDITIONAL_STR_DEPRECATED(RHICmdList_Or_RHIContext,       Condition, StaticName, Format, ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS_NONE , Condition, TEXT(StaticName),      Format , ##__VA_ARGS__)

#elif WITH_RHI_BREADCRUMBS_MINIMAL

	//
	// Keep only the STAT breadcrumbs enabled in MINIMAL mode.
	// Also disable the varargs. We don't capture the format strings and varargs in MINIMAL mode
	//

	// Note, the varargs are deprecated and ignored in these macros.
	#define RHI_BREADCRUMB_EVENT(                             RHICmdList_Or_RHIContext,                  StaticName,         ...)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL(                 RHICmdList_Or_RHIContext,       Condition, StaticName,         ...)
	#define RHI_BREADCRUMB_EVENT_STAT(                        RHICmdList_Or_RHIContext, Stat,            StaticName,         ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS(Stat),      true, TEXT(StaticName), nullptr)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_STAT(            RHICmdList_Or_RHIContext, Stat, Condition, StaticName,         ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS(Stat), Condition, TEXT(StaticName), nullptr)
													          
	// Format versions of the breadcrumb macros.	          
	#define RHI_BREADCRUMB_EVENT_F(                           RHICmdList_Or_RHIContext,                  StaticName, Format, ...)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_F(               RHICmdList_Or_RHIContext,       Condition, StaticName, Format, ...)
	#define RHI_BREADCRUMB_EVENT_STAT_F(                      RHICmdList_Or_RHIContext, Stat,            StaticName, Format, ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS(Stat),      true, TEXT(StaticName), nullptr)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_STAT_F(          RHICmdList_Or_RHIContext, Stat, Condition, StaticName, Format, ...) RHI_BREADCRUMB_EVENT_PRIVATE_IMPL(RHICmdList_Or_RHIContext, RHI_GPU_STAT_ARGS(Stat), Condition, TEXT(StaticName), nullptr)

	// Used only for back compat with SCOPED_DRAW_EVENTF
	#define RHI_BREADCRUMB_EVENT_F_STR_DEPRECATED(            RHICmdList_Or_RHIContext,                  StaticName, Format, ...)
	#define RHI_BREADCRUMB_EVENT_F_CONDITIONAL_STR_DEPRECATED(RHICmdList_Or_RHIContext,       Condition, StaticName, Format, ...)

#else

	#define RHI_BREADCRUMB_FIELD(                             ...)
	#define RHI_BREADCRUMB_EVENT(                             ...)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL(                 ...)
	#define RHI_BREADCRUMB_EVENT_STAT(                        ...)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_STAT(            ...)
	#define RHI_BREADCRUMB_EVENT_F(                           ...)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_F(               ...)
	#define RHI_BREADCRUMB_EVENT_STAT_F(                      ...)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_STAT_F(          ...)
	#define RHI_BREADCRUMB_EVENT_F_STR_DEPRECATED(            ...)
	#define RHI_BREADCRUMB_EVENT_F_CONDITIONAL_STR_DEPRECATED(...)

#endif

#if WITH_RHI_BREADCRUMBS_FULL

	// Log and check macros that include the current breadcrumb path
	#define RHI_BREADCRUMB_LOG(   RHICmdList_Or_RHIContext,            CategoryName, Verbosity, Format, ...) UE_LOG(            CategoryName, Verbosity, Format TEXT("\nBreadcrumbs: %s"), ##__VA_ARGS__, *UE::RHI::Breadcrumbs::Private::GetSafeBreadcrumbPath(RHICmdList_Or_RHIContext))
	#define RHI_BREADCRUMB_CLOG(  RHICmdList_Or_RHIContext, Condition, CategoryName, Verbosity, Format, ...) UE_CLOG(Condition, CategoryName, Verbosity, Format TEXT("\nBreadcrumbs: %s"), ##__VA_ARGS__, *UE::RHI::Breadcrumbs::Private::GetSafeBreadcrumbPath(RHICmdList_Or_RHIContext))
	#define RHI_BREADCRUMB_CHECKF(RHICmdList_Or_RHIContext, Condition,                          Format, ...) checkf( Condition,                          Format TEXT("\nBreadcrumbs: %s"), ##__VA_ARGS__, *UE::RHI::Breadcrumbs::Private::GetSafeBreadcrumbPath(RHICmdList_Or_RHIContext))

#else

	// Log and check macros fall back to regular UE_LOG / check when breadcrumbs are not available
	#define RHI_BREADCRUMB_LOG(   RHICmdList_Or_RHIContext,            CategoryName, Verbosity, Format, ...) UE_LOG(            CategoryName, Verbosity, Format, ##__VA_ARGS__)
	#define RHI_BREADCRUMB_CLOG(  RHICmdList_Or_RHIContext, Condition, CategoryName, Verbosity, Format, ...) UE_CLOG(Condition, CategoryName, Verbosity, Format, ##__VA_ARGS__)
	#define RHI_BREADCRUMB_CHECKF(RHICmdList_Or_RHIContext, Condition,                          Format, ...) checkf( Condition,                          Format, ##__VA_ARGS__)

#endif

#if DO_CHECK
	#define RHI_BREADCRUMB_CHECK_SHIPPINGF(RHICmdList_Or_RHIContext, Condition, Format, ...) RHI_BREADCRUMB_CHECKF(RHICmdList_Or_RHIContext, Condition, Format, ##__VA_ARGS__)
#else
	#define RHI_BREADCRUMB_CHECK_SHIPPINGF(RHICmdList_Or_RHIContext, Condition, Format, ...) RHI_BREADCRUMB_CLOG(RHICmdList_Or_RHIContext, !(Condition), LogRHI, Error, TEXT("Check '%s' failed. ") Format, TEXT(#Condition), ##__VA_ARGS__)
#endif

#define RHI_BREADCRUMB_CHECK_SHIPPING(RHICmdList_Or_RHIContext, Condition) RHI_BREADCRUMB_CHECK_SHIPPINGF(RHICmdList_Or_RHIContext, Condition, TEXT(""))
#define RHI_BREADCRUMB_CHECK(         RHICmdList_Or_RHIContext, Condition) RHI_BREADCRUMB_CHECKF(         RHICmdList_Or_RHIContext, Condition, TEXT(""))

//
// Used to override the static_assert check for string literals in RHI breadcrumbs. This is required when using
// literals returned by functions, or choosing between two string literals with a ternary operator, like so:
// 
//    SCOPED_DRAW_EVENTF(RHICmdList, EventName, TEXT("Name=%s"), RHI_BREADCRUMB_FORCE_STRING_LITERAL(bCondition ? TEXT("True") : TEXT("False"))
// 
// !! DO NOT USE THIS MACRO FOR NON-STRING LITERALS !!
//
#define RHI_BREADCRUMB_FORCE_STRING_LITERAL [](auto&& TCharPointer) -> TCHAR const(&)[1]\
	{																					\
		return *reinterpret_cast<TCHAR const(*)[1]>(TCharPointer);						\
	}
