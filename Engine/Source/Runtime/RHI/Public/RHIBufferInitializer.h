// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIResources.h"
#include "Templates/Function.h"

// Structure used to allow optimal buffer initialization at creation time.
// Should only ever be obtained by calling RHICreateBufferInitializer().
// NO COPIES ALLOWED
struct FRHIBufferInitializer
{
	FRHIBufferInitializer() = default;
	FRHIBufferInitializer(FRHIBufferInitializer&& InOther)
		: FinalizeCallback(MoveTemp(InOther.FinalizeCallback))
		, CommandList(InOther.CommandList)
		, Buffer(InOther.Buffer)
		, WritableData(InOther.WritableData)
		, WritableSize(InOther.WritableSize)
	{
		InOther.Reset();
	}
	~FRHIBufferInitializer()
	{
		if (CommandList && Buffer)
		{
			RemovePendingBufferUpload();
		}
	}

	void WriteDataOffset(uint64 Offset, const void* Source, size_t Size)
	{
		check(Offset < GetWritableDataSize() && (Offset + Size) <= GetWritableDataSize());
		FMemory::Memcpy(reinterpret_cast<uint8*>(WritableData) + Offset, Source, Size);
	}

	void WriteData(const void* Source, size_t Size)
	{
		check(Size <= GetWritableDataSize());
		FMemory::Memcpy(WritableData, Source, Size);
	}

	void WriteDataParallel(const void* Source, size_t Size)
	{
		check(Size <= GetWritableDataSize());
		FMemory::ParallelMemcpy(WritableData, Source, Size, EMemcpyCachePolicy::StoreUncached);
	}

	void FillWithValue(uint8 Value)
	{
		FMemory::Memset(WritableData, Value, GetWritableDataSize());
	}

	size_t GetWritableDataSize() const
	{
		return WritableSize;
	}

	template<typename TElement>
	TArrayView<TElement> GetWriteView()
	{
		return TArrayView<TElement>(reinterpret_cast<TElement*>(WritableData), GetWritableDataSize() / sizeof(TElement));
	}

	RHI_API FBufferRHIRef Finalize();

protected:
	// @todo dev-pr switch to using IRHIUploadContext
	using FFinalizeCallback = TUniqueFunction<FBufferRHIRef(FRHICommandListBase&)>;

	// Should only be called by RHI derived types.
	RHI_API FRHIBufferInitializer(FRHICommandListBase& RHICmdList, FRHIBuffer* InBuffer, void* InWritableData, uint64 InWritableSize, FFinalizeCallback&& InFinalizeCallback);

	// Remove the buffer from the command list. Has to be in cpp file to prevent circular header dependency.
	RHI_API void RemovePendingBufferUpload();

	// Allow copies only for RHI derived types.
	FRHIBufferInitializer(const FRHIBufferInitializer&) = delete;
	FRHIBufferInitializer& operator=(const FRHIBufferInitializer&) = delete;
	FRHIBufferInitializer& operator=(FRHIBufferInitializer&&) = delete;

	void Reset()
	{
		FinalizeCallback = {};
		CommandList  = {};
		Buffer       = {};
		WritableData = {};
		WritableSize = {};
	}

protected:
	// Pointer only used by the RHI internals, should not be accessed outside of RHIs
	FFinalizeCallback FinalizeCallback = nullptr;

	// Command list provided on construction, used in finalize.
	FRHICommandListBase* CommandList = nullptr;

	// Current RHI Buffer being initialized. Will only be used for command list validation since each RHI implementation will manage their own buffer type.
	FRHIBuffer* Buffer = nullptr;

	// Pointer to the writable data provided by the RHI
	void* WritableData = nullptr;

	// Size of the writable data provided by the RHI
	uint64 WritableSize = 0;
};

template<typename ElementType>
struct TRHIBufferInitializer : public FRHIBufferInitializer
{
	TRHIBufferInitializer() = delete;
	TRHIBufferInitializer(TRHIBufferInitializer&&) = default;
	TRHIBufferInitializer(const TRHIBufferInitializer&) = delete;
	TRHIBufferInitializer(const FRHIBufferInitializer& InInitializer) = delete;

	TRHIBufferInitializer(FRHIBufferInitializer&& InInitializer)
		: FRHIBufferInitializer(MoveTemp(InInitializer))
	{
	}

	ElementType* GetWritableData()
	{
		return reinterpret_cast<ElementType*>(WritableData);
	}

	uint64 GetWritableElementCount() const
	{
		return GetWritableDataSize() / sizeof(ElementType);
	}

	TArrayView<ElementType> GetWriteView()
	{
		return TArrayView<ElementType>(GetWritableData(), GetWritableElementCount());
	}

	void WriteArray(size_t ElementOffset, TConstArrayView<ElementType> InData)
	{
		WriteDataOffset(ElementOffset * sizeof(ElementType), InData.GetData(), InData.NumBytes());
	}

	void WriteArray(TConstArrayView<ElementType> InData)
	{
		WriteData(InData.GetData(), InData.GetTypeSize() * InData.Num());
	}

	template<size_t TCount>
	void WriteArray(const ElementType(&InData)[TCount])
	{
		WriteData(InData, sizeof(ElementType) * TCount);
	}

	void WriteValue(const ElementType& InElement)
	{
		WriteData(&InElement, sizeof(InElement));
	}

	void WriteValueAtIndex(uint32 Index, const ElementType& InElement)
	{
		checkSlow(Index < GetWritableElementCount());
		GetWritableData()[Index] = InElement;
	}

	ElementType& operator[](int32 Index)
	{
		checkSlow(Index < GetWritableElementCount());
		return GetWritableData()[Index];
	}
};
