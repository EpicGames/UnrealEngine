// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHICommandList.h: RHI Command List definitions for queueing up & executing later.
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/Box2D.h"
#include "Math/PerspectiveMatrix.h"
#include "Math/TranslationMatrix.h"
#include "Math/ScaleMatrix.h"
#include "Math/Float16Color.h"
#include "HAL/ThreadSafeCounter.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Misc/MemStack.h"
#include "Misc/App.h"
#include "RHIStats.h"
#include "HAL/IConsoleManager.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "RHIBreadcrumbs.h"
#include "RHIGlobals.h"
#include "RHIAllocators.h"
#include "RHIShaderParameters.h"
#include "RHITextureReference.h"
#include "RHIResourceReplace.h"
#include "Stats/ThreadIdleStats.h"
#include "Trace/Trace.h"

#include "DynamicRHI.h"
#include "RHITypes.h"
#include "RHIGlobals.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(RHI_API, RHITStalls);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(RHI_API, RHITFlushes);

/** Get the best default resource state for the given texture creation flags */
extern RHI_API ERHIAccess RHIGetDefaultResourceState(ETextureCreateFlags InUsage, bool bInHasInitialData);

/** Get the best default resource state for the given buffer creation flags */
extern RHI_API ERHIAccess RHIGetDefaultResourceState(EBufferUsageFlags InUsage, bool bInHasInitialData);

// Set to 1 to capture the callstack for every RHI command. Cheap & memory efficient representation: Use the 
// value in FRHICommand::StackFrames to get the pointer to the code (ie paste on a disassembly window)
#define RHICOMMAND_CALLSTACK		0
#if RHICOMMAND_CALLSTACK
#include "HAL/PlatformStackwalk.h"
#endif

class FApp;
class FBlendStateInitializerRHI;
class FGraphicsPipelineStateInitializer;
class FRHICommandListBase;
class FRHIComputeShader;
class IRHICommandContext;
class IRHIComputeContext;
struct FDepthStencilStateInitializerRHI;
struct FRasterizerStateInitializerRHI;
struct FRHIResourceCreateInfo;
struct FRHIResourceInfo;
struct FRHIUniformBufferLayout;
struct FSamplerStateInitializerRHI;
struct FTextureMemoryStats;
class FComputePipelineState;
class FGraphicsPipelineState;
class FRayTracingPipelineState;
class FWorkGraphPipelineState;
struct FRHICommandSetTrackedAccess;

DECLARE_STATS_GROUP(TEXT("RHICmdList"), STATGROUP_RHICMDLIST, STATCAT_Advanced);

UE_TRACE_CHANNEL_EXTERN(RHICommandsChannel, RHI_API);

// set this one to get a stat for each RHI command 
#define RHI_STATS 0

#if RHI_STATS
DECLARE_STATS_GROUP(TEXT("RHICommands"),STATGROUP_RHI_COMMANDS, STATCAT_Advanced);
#define RHISTAT(Method)	DECLARE_SCOPE_CYCLE_COUNTER(TEXT(#Method), STAT_RHI##Method, STATGROUP_RHI_COMMANDS)
#else
#define RHISTAT(Method)
#endif

#if !defined(RHI_EXECUTE_API)
#define RHI_EXECUTE_API RHI_API
#endif

enum class ERHIThreadMode
{
	None,
	DedicatedThread,
	Tasks
};

// Global for handling the "r.RHIThread.Enable" command.
extern RHI_API TOptional<ERHIThreadMode> GPendingRHIThreadMode;

namespace ERenderThreadIdleTypes
{
	enum Type
	{
		WaitingForAllOtherSleep,
		WaitingForGPUQuery,
		WaitingForGPUPresent,
		Num
	};
}

/** Accumulates how many cycles the renderthread has been idle. */
extern RHI_API uint32 GRenderThreadIdle[ERenderThreadIdleTypes::Num];

/** Helper to mark scopes as idle time on the render or RHI threads. */
struct FRenderThreadIdleScope
{
	UE::Stats::FThreadIdleStats::FScopeIdle RHIThreadIdleScope;

	const ERenderThreadIdleTypes::Type Type;
	const bool bCondition;
	const uint32 Start;

	FRenderThreadIdleScope(ERenderThreadIdleTypes::Type Type, bool bInCondition = true)
		: RHIThreadIdleScope(!(bInCondition && IsInRHIThread()))
		, Type(Type)
		, bCondition(bInCondition && IsInRenderingThread())
		, Start(bCondition ? FPlatformTime::Cycles() : 0)
	{}

	~FRenderThreadIdleScope()
	{
		if (bCondition)
		{
			GRenderThreadIdle[Type] += FPlatformTime::Cycles() - Start;
		}
	}
};

/** How many cycles the from sampling input to the frame being flipped. */
extern RHI_API uint64 GInputLatencyTime;

/*UE::Trace::FChannel& FORCEINLINE GetRHICommandsChannel() 
{

}*/

/**
* Whether the RHI commands are being run in a thread other than the render thread
*/
bool inline IsRunningRHIInSeparateThread()
{
	return GIsRunningRHIInSeparateThread_InternalUseOnly;
}

/**
* Whether the RHI commands are being run on a dedicated thread other than the render thread
*/
bool inline IsRunningRHIInDedicatedThread()
{
	return GIsRunningRHIInDedicatedThread_InternalUseOnly;
}

/**
* Whether the RHI commands are being run on a dedicated thread other than the render thread
*/
bool inline IsRunningRHIInTaskThread()
{
	return GIsRunningRHIInTaskThread_InternalUseOnly;
}

extern RHI_API TAutoConsoleVariable<int32> CVarRHICmdWidth;

struct FRHICopyTextureInfo
{
	FIntRect GetSourceRect() const
	{
		return FIntRect(SourcePosition.X, SourcePosition.Y, SourcePosition.X + Size.X, SourcePosition.Y + Size.Y);
	}

	FIntRect GetDestRect() const
	{
		return FIntRect(DestPosition.X, DestPosition.Y, DestPosition.X + Size.X, DestPosition.Y + Size.Y);
	}

	// Number of texels to copy. By default it will copy the whole resource if no size is specified.
	FIntVector Size = FIntVector::ZeroValue;

	// Position of the copy from the source texture/to destination texture
	FIntVector SourcePosition = FIntVector::ZeroValue;
	FIntVector DestPosition = FIntVector::ZeroValue;

	uint32 SourceSliceIndex = 0;
	uint32 DestSliceIndex = 0;
	uint32 NumSlices = 1;

	// Mips to copy and destination mips
	uint32 SourceMipIndex = 0;
	uint32 DestMipIndex = 0;
	uint32 NumMips = 1;
};

struct FRHIBufferRange
{
	class FRHIBuffer* Buffer{ nullptr };
	uint64 Offset{ 0 };
	uint64 Size{ 0 };
};

/** Struct to hold common data between begin/end updatetexture3d */
struct FUpdateTexture3DData
{
	FUpdateTexture3DData(FRHITexture* InTexture, uint32 InMipIndex, const struct FUpdateTextureRegion3D& InUpdateRegion, uint32 InSourceRowPitch, uint32 InSourceDepthPitch, uint8* InSourceData, uint32 InDataSizeBytes, uint32 InFrameNumber)
		: Texture(InTexture)
		, MipIndex(InMipIndex)
		, UpdateRegion(InUpdateRegion)
		, RowPitch(InSourceRowPitch)
		, DepthPitch(InSourceDepthPitch)
		, Data(InSourceData)
		, DataSizeBytes(InDataSizeBytes)
		, FrameNumber(InFrameNumber)
	{
	}

	FRHITexture* Texture;
	uint32 MipIndex;
	FUpdateTextureRegion3D UpdateRegion;
	uint32 RowPitch;
	uint32 DepthPitch;
	uint8* Data;
	uint32 DataSizeBytes;
	uint32 FrameNumber;
	uint8 PlatformData[64];

private:
	FUpdateTexture3DData();
};

struct FRayTracingShaderBindings
{
	FRHITexture* Textures[64] = {};
	FRHIShaderResourceView* SRVs[64] = {};
	FRHIUniformBuffer* UniformBuffers[16] = {};
	FRHISamplerState* Samplers[32] = {};
	FRHIUnorderedAccessView* UAVs[16] = {};

	TArray<FRHIShaderParameterResource> BindlessParameters;
};

enum class ERayTracingLocalShaderBindingType : uint8
{
	Persistent,		//< Binding contains persistent data
	Transient,		//< Binding contains transient data
	Clear,			//< Clear SBT record data		
	Validation		//< Binding only used for validating persistently stored data in the SBT
};

struct FRayTracingLocalShaderBindings
{
	ERayTracingLocalShaderBindingType BindingType = ERayTracingLocalShaderBindingType::Transient;
	const FRHIRayTracingGeometry* Geometry = nullptr;
	uint32 SegmentIndex = 0;
	uint32 RecordIndex = 0;
	uint32 ShaderIndexInPipeline = 0;
	uint32 UserData = 0;
	uint16 NumUniformBuffers = 0;
	uint16 LooseParameterDataSize = 0;
	FRHIUniformBuffer** UniformBuffers = nullptr;
	uint8* LooseParameterData = nullptr;
};

enum class ERayTracingBindingType : uint8
{
	HitGroup,
	CallableShader,
	MissShader,
};

struct FLockTracker
{
	struct FLockParams
	{
		void* RHIBuffer;
		void* Buffer;
		uint32 BufferSize;
		uint32 Offset;
		EResourceLockMode LockMode;

		inline FLockParams(void* InRHIBuffer, void* InBuffer, uint32 InOffset, uint32 InBufferSize, EResourceLockMode InLockMode)
			: RHIBuffer(InRHIBuffer)
			, Buffer(InBuffer)
			, BufferSize(InBufferSize)
			, Offset(InOffset)
			, LockMode(InLockMode)
		{
		}
	};

	FCriticalSection CriticalSection;
	TArray<FLockParams, TInlineAllocator<16> > OutstandingLocks;

	FLockTracker()
	{
	}

	inline void Lock(void* RHIBuffer, void* Buffer, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode)
	{
		FScopeLock Lock(&CriticalSection);
#if DO_CHECK
		for (auto& Parms : OutstandingLocks)
		{
			check(Parms.RHIBuffer != RHIBuffer);
		}
#endif
		OutstandingLocks.Add(FLockParams(RHIBuffer, Buffer, Offset, SizeRHI, LockMode));
	}
	inline FLockParams Unlock(void* RHIBuffer)
	{
		FScopeLock Lock(&CriticalSection);
		for (int32 Index = 0; Index < OutstandingLocks.Num(); Index++)
		{
			if (OutstandingLocks[Index].RHIBuffer == RHIBuffer)
			{
				FLockParams Result = OutstandingLocks[Index];
				OutstandingLocks.RemoveAtSwap(Index, EAllowShrinking::No);
				return Result;
			}
		}
		check(!"Mismatched RHI buffer locks.");
		return FLockParams(nullptr, nullptr, 0, 0, RLM_WriteOnly);
	}
};

#ifdef CONTINUABLE_PSO_VERIFY
#define PSO_VERIFY ensure
#else
#define PSO_VERIFY	check
#endif

struct FRHICommandBase
{
	FRHICommandBase* Next = nullptr;
	virtual void ExecuteAndDestruct(FRHICommandListBase& CmdList) = 0;
};

template <typename RHICmdListType, typename LAMBDA>
struct TRHILambdaCommand final : public FRHICommandBase
{
	LAMBDA Lambda;
#if CPUPROFILERTRACE_ENABLED
	const TCHAR* Name;
#endif

	TRHILambdaCommand(LAMBDA&& InLambda, const TCHAR* InName)
		: Lambda(Forward<LAMBDA>(InLambda))
#if CPUPROFILERTRACE_ENABLED
		, Name(InName)
#endif
	{}

	void ExecuteAndDestruct(FRHICommandListBase& CmdList) override final
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(Name, RHICommandsChannel);
		Lambda(*static_cast<RHICmdListType*>(&CmdList));
		Lambda.~LAMBDA();
	}
};

template <typename RHICmdListType, typename LAMBDA>
struct TRHILambdaCommand_NoMarker final : public FRHICommandBase
{
	LAMBDA Lambda;

	TRHILambdaCommand_NoMarker(LAMBDA&& InLambda)
		: Lambda(Forward<LAMBDA>(InLambda))
	{}

	void ExecuteAndDestruct(FRHICommandListBase& CmdList) override final
	{
		Lambda(*static_cast<RHICmdListType*>(&CmdList));
		Lambda.~LAMBDA();
	}
};

template <typename RHICmdListType, typename LAMBDA>
struct TRHILambdaCommandMultiPipe final : public FRHICommandBase
{
	LAMBDA Lambda;
#if CPUPROFILERTRACE_ENABLED
	const TCHAR* Name;
#endif
	ERHIPipeline Pipelines;

	TRHILambdaCommandMultiPipe(LAMBDA&& InLambda, const TCHAR* InName, ERHIPipeline InPipelines)
		: Lambda(Forward<LAMBDA>(InLambda))
#if CPUPROFILERTRACE_ENABLED
		, Name(InName)
#endif
		, Pipelines(InPipelines)
	{}

	inline void ExecuteAndDestruct(FRHICommandListBase& CmdList) override final;
};

// Using variadic macro because some types are fancy template<A,B> stuff, which gets broken off at the comma and interpreted as multiple arguments. 
#define ALLOC_COMMAND(...) new ( AllocCommand(sizeof(__VA_ARGS__), alignof(__VA_ARGS__)) ) __VA_ARGS__
#define ALLOC_COMMAND_CL(RHICmdList, ...) new ( (RHICmdList).AllocCommand(sizeof(__VA_ARGS__), alignof(__VA_ARGS__)) ) __VA_ARGS__

// This controls if the cmd list bypass can be toggled at runtime. It is quite expensive to have these branches in there.
#define CAN_TOGGLE_COMMAND_LIST_BYPASS (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

// Issues a single fence at the end of the scope if an RHI fence is requested by commands within the scope. Can
// reduce overhead of RHIThreadFence when batch updating resources that would otherwise issue N fences.
class FRHICommandListScopedFence
{
	friend class FRHICommandListBase;
	FRHICommandListBase& RHICmdList;
    FRHICommandListScopedFence* Previous;
    bool bFenceRequested = false;

public:
	FRHICommandListScopedFence(FRHICommandListBase& RHICmdList);
    ~FRHICommandListScopedFence();
};

class FRHICommandListScopedPipelineGuard
{
	FRHICommandListBase& RHICmdList;
	bool bPipelineSet = false;

public:
	FRHICommandListScopedPipelineGuard(FRHICommandListBase& RHICmdList);
	~FRHICommandListScopedPipelineGuard();
};

class FRHICommandListScopedAllowExtraTransitions
{
	FRHICommandListBase& RHICmdList;
	bool bAllowExtraTransitions = false;

public:
	FRHICommandListScopedAllowExtraTransitions(FRHICommandListBase& RHICmdList, bool bAllowExtraTransitions);
	~FRHICommandListScopedAllowExtraTransitions();
};

class FRHICommandListBase
{
protected:
	FMemStackBase MemManager;

	RHI_API FRHICommandListBase(FRHIGPUMask InGPUMask, bool bInImmediate);

public:
	// Move only.
	FRHICommandListBase(FRHICommandListBase const&) = delete;
	FRHICommandListBase(FRHICommandListBase&& Other) = default;

	RHI_API ~FRHICommandListBase();

	inline bool IsImmediate() const;
	inline FRHICommandListImmediate& GetAsImmediate();
	const int32 GetUsedMemory() const;

	bool AllowParallelTranslate() const
	{
		// Command lists cannot be translated in parallel for various reasons...

		// Parallel translate might be explicitly disabled (e.g. platform RHI doesn't support parallel translate)
		if (!bAllowParallelTranslate)
		{
			return false;
		}

		// All commands recorded by the immediate command list must not be parallel translated.
		// This is mostly for legacy reasons, since various parts of the renderer / RHI expect immediate commands to be single-threaded.
		if (PersistentState.bImmediate)
		{
			return false;
		}

		// Command lists that use RHIThreadFence(true) are going to mutate resource state, so must be single-threaded.
		if (bUsesLockFence)
		{
			return false;
		}

		// Some shader bundle implementations do not currently support parallel translate
		if (bUsesShaderBundles && !GRHISupportsShaderBundleParallel)
		{
			return false;
		}

		return true;
	}

	//
	// Adds a graph event as a dispatch dependency. The command list will not be dispatched to the
	// RHI / parallel translate threads until all its dispatch prerequisites have been completed.
	// 
	// Not safe to call after FinishRecording().
	//
	RHI_API void AddDispatchPrerequisite(const FGraphEventRef& Prereq);

	//
	// Marks the RHI command list as completed, allowing it to be dispatched to the RHI / parallel translate threads.
	// 
	// Must be called as the last command in a parallel rendering task. It is not safe to continue using the command 
	// list after FinishRecording() has been called.
	// 
	// Never call on the immediate command list.
	//
	RHI_API void FinishRecording();

	inline void* Alloc(int64 AllocSize, int64 Alignment)
	{
		return MemManager.Alloc(AllocSize, Alignment);
	}

	inline void* AllocCopy(const void* InSourceData, int64 AllocSize, int64 Alignment)
	{
		void* NewData = Alloc(AllocSize, Alignment);
		FMemory::Memcpy(NewData, InSourceData, AllocSize);
		return NewData;
	}

	template <typename T>
	inline T* Alloc()
	{
		return (T*)Alloc(sizeof(T), alignof(T));
	}

	template <typename T>
	inline const TArrayView<T> AllocArrayUninitialized(uint32 Num)
	{
		return TArrayView<T>((T*)Alloc(Num * sizeof(T), alignof(T)), Num);
	}

	template <typename T>
	inline const TArrayView<T> AllocArray(TConstArrayView<T> InArray)
	{
		if (InArray.Num() == 0)
		{
			return TArrayView<T>();
		}

		// @todo static_assert(TIsTrivial<T>::Value, "Only trivially constructible / copyable types can be used in RHICmdList.");
		void* NewArray = AllocCopy(InArray.GetData(), InArray.Num() * sizeof(T), alignof(T));
		return TArrayView<T>((T*) NewArray, InArray.Num());
	}

	inline TCHAR* AllocString(const TCHAR* Name)
	{
		int32 Len = FCString::Strlen(Name) + 1;
		TCHAR* NameCopy  = (TCHAR*)Alloc(Len * (int32)sizeof(TCHAR), (int32)sizeof(TCHAR));
		FCString::Strncpy(NameCopy, Name, Len);
		return NameCopy;
	}

	inline void* AllocCommand(int32 AllocSize, int32 Alignment)
	{
		checkSlow(!IsExecuting());
		checkfSlow(!Bypass(), TEXT("Invalid attempt to record commands in bypass mode."));
		FRHICommandBase* Result = (FRHICommandBase*) MemManager.Alloc(AllocSize, Alignment);
		++NumCommands;
		*CommandLink = Result;
		CommandLink = &Result->Next;
		return Result;
	}

	template <typename TCmd>
	inline void* AllocCommand()
	{
		return AllocCommand(sizeof(TCmd), alignof(TCmd));
	}

	enum class EThreadFence
	{
		Enabled,
		Disabled
	};

	template <typename LAMBDA>
	inline void EnqueueLambda(EThreadFence ThreadFence, const TCHAR* LambdaName, LAMBDA&& Lambda)
	{
		if (IsBottomOfPipe())
		{
			Lambda(*this);
		}
		else
		{
			ALLOC_COMMAND(TRHILambdaCommand<FRHICommandListBase, LAMBDA>)(Forward<LAMBDA>(Lambda), LambdaName);

			if (ThreadFence == EThreadFence::Enabled)
			{
				RHIThreadFence(true);
			}
		}
	}

	template <typename LAMBDA>
	inline void EnqueueLambda(const TCHAR* LambdaName, LAMBDA&& Lambda)
	{
		FRHICommandListBase::EnqueueLambda(EThreadFence::Disabled, LambdaName, Forward<LAMBDA>(Lambda));
	}

	template <typename LAMBDA>
	inline void EnqueueLambda(LAMBDA&& Lambda)
	{
		FRHICommandListBase::EnqueueLambda(TEXT("TRHILambdaCommand"), Forward<LAMBDA>(Lambda));
	}

	template <typename LAMBDA>
	void EnqueueLambdaMultiPipe(ERHIPipeline Pipelines, EThreadFence ThreadFence, const TCHAR* LambdaName, LAMBDA&& Lambda)
	{
		checkf(IsTopOfPipe() || Bypass(), TEXT("Cannot enqueue a multi-pipe lambda from the bottom of pipe."));

		ERHIPipeline OldPipeline = ActivePipelines;
		ActivatePipelines(Pipelines);

		if (IsBottomOfPipe())
		{
			FRHIContextArray LocalContexts { InPlace, nullptr };
			for (ERHIPipeline Pipeline : MakeFlagsRange(Pipelines))
			{
				LocalContexts[Pipeline] = Contexts[Pipeline];
				check(LocalContexts[Pipeline]);
			}

			// Static cast to enforce const type in lambda args
			Lambda(static_cast<FRHIContextArray const&>(LocalContexts));
		}
		else
		{
			ALLOC_COMMAND(TRHILambdaCommandMultiPipe<FRHICommandListBase, LAMBDA>)(Forward<LAMBDA>(Lambda), LambdaName, Pipelines);
		}

		ActivatePipelines(OldPipeline);
		
		if (ThreadFence == EThreadFence::Enabled)
		{
			RHIThreadFence(true);
		}
	}

	inline bool HasCommands() const
	{
		// Assume we have commands if anything is allocated.
		return !MemManager.IsEmpty();
	}

	inline bool IsExecuting() const
	{
		return bExecuting;
	}

	inline bool IsBottomOfPipe() const
	{
		return Bypass() || IsExecuting();
	}

	inline bool IsTopOfPipe() const
	{
		return !IsBottomOfPipe();
	}

	inline bool IsGraphics() const
	{
		// Exact equality is deliberate. Only return true if the graphics pipe is the only active pipe.
		return ActivePipelines == ERHIPipeline::Graphics;
	}

	inline bool IsAsyncCompute() const
	{
		// Exact equality is deliberate. Only return true if the compute pipe is the only active pipe.
		return ActivePipelines == ERHIPipeline::AsyncCompute;
	}

	inline ERHIPipeline GetPipeline() const
	{
		check(ActivePipelines == ERHIPipeline::None || IsSingleRHIPipeline(ActivePipelines));
		return ActivePipelines;
	}

	inline ERHIPipeline GetPipelines() const
	{
		return ActivePipelines;
	}

	inline IRHICommandContext& GetContext()
	{
		checkf(IsSingleRHIPipeline(ActivePipelines), TEXT("Exactly one pipeline must be active to call GetContext(). Current pipeline mask is '0x%02x'."), static_cast<std::underlying_type_t<ERHIPipeline>>(ActivePipelines));
		checkf(GraphicsContext, TEXT("There is no active graphics context on this command list. There may be a missing call to SwitchPipeline()."));
		return *GraphicsContext;
	}

	inline IRHIComputeContext& GetComputeContext()
	{
		checkf(IsSingleRHIPipeline(ActivePipelines), TEXT("Exactly one pipeline must be active to call GetComputeContext(). Current pipeline mask is '0x%02x'."), static_cast<std::underlying_type_t<ERHIPipeline>>(ActivePipelines));
		checkf(ComputeContext, TEXT("There is no active compute context on this command list. There may be a missing call to SwitchPipeline()."));
		return *ComputeContext;
	}

	inline IRHIUploadContext& GetUploadContext()
	{
		if(!UploadContext)
		{
			UploadContext = GDynamicRHI->RHIGetUploadContext();
		}
		return *UploadContext;
	}
	
	inline bool Bypass() const;
	
	inline bool IsSubCommandList() const
	{
	   return SubRenderPassInfo.IsValid();
	}

private:
	RHI_API void InvalidBufferFatalError(const FRHIBufferCreateDesc& CreateDesc);

protected:
	RHI_API void ActivatePipelines(ERHIPipeline Pipelines);

	// Takes the array of sub command lists and inserts them logically into a render pass at this point in time.
	RHI_API void InsertParallelRenderPass_Base(TSharedPtr<FRHIParallelRenderPassInfo> const& InInfo, TArray<FRHISubCommandList*>&& SubCommandLists);

public:
	RHI_API void TransitionInternal(TConstArrayView<FRHITransitionInfo> Infos, ERHITransitionCreateFlags CreateFlags = ERHITransitionCreateFlags::None);
	inline void TransitionInternal(const FRHITransitionInfo& Info, ERHITransitionCreateFlags CreateFlags = ERHITransitionCreateFlags::None)
	{
		TransitionInternal(MakeArrayView(&Info, 1), CreateFlags);
	}

	RHI_API ERHIPipeline SwitchPipeline(ERHIPipeline Pipeline);

	inline FRHIGPUMask GetGPUMask() const { return PersistentState.CurrentGPUMask; }

	bool IsRecursive		   () const { return PersistentState.bRecursive; }
	bool IsOutsideRenderPass   () const { return !PersistentState.bInsideRenderPass; }
	bool IsInsideRenderPass    () const { return PersistentState.bInsideRenderPass;  }
	bool IsInsideComputePass   () const { return PersistentState.bInsideComputePass; }

	RHI_API FGraphEventRef RHIThreadFence(bool bSetLockFence = false);

	inline void* LockBuffer(FRHIBuffer* Buffer, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode)
	{
		checkf(IsTopOfPipe() || Bypass(), TEXT("Buffers may only be locked while recording RHI command lists, not during RHI command list execution."));

		FRHICommandListScopedPipelineGuard ScopedPipeline(*this);
		return GDynamicRHI->RHILockBuffer(*this, Buffer, Offset, SizeRHI, LockMode);
	}

	inline void UnlockBuffer(FRHIBuffer* Buffer)
	{
		checkf(IsTopOfPipe() || Bypass(), TEXT("Buffers may only be unlocked while recording RHI command lists, not during RHI command list execution."));

		FRHICommandListScopedPipelineGuard ScopedPipeline(*this);
		GDynamicRHI->RHIUnlockBuffer(*this, Buffer);
	}

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	RHI_API void UpdateAllocationTags(FRHIBuffer* Buffer);
#endif

	// LockBufferMGPU / UnlockBufferMGPU may ONLY be called for buffers with the EBufferUsageFlags::MultiGPUAllocate flag set!
	// And buffers with that flag set may not call the regular (single GPU) LockBuffer / UnlockBuffer.  The single GPU version
	// of LockBuffer uses driver mirroring to propagate the updated buffer to other GPUs, while the MGPU / MultiGPUAllocate
	// version requires the caller to manually lock and initialize the buffer separately on each GPU.  This can be done by
	// iterating over FRHIGPUMask::All() and calling LockBufferMGPU / UnlockBufferMGPU for each version.
	//
	// EBufferUsageFlags::MultiGPUAllocate is only needed for cases where CPU initialized data needs to be different per GPU,
	// which is a rare edge case.  Currently, this is only used for the ray tracing acceleration structure address buffer,
	// which contains virtual address references to other GPU resources, which may be in a different location on each GPU.
	//
	inline void* LockBufferMGPU(FRHIBuffer* Buffer, uint32 GPUIndex, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode)
	{
		checkf(IsTopOfPipe() || Bypass(), TEXT("Buffers may only be locked while recording RHI command lists, not during RHI command list execution."));

		return GDynamicRHI->RHILockBufferMGPU(*this, Buffer, GPUIndex, Offset, SizeRHI, LockMode);
	}

	inline void UnlockBufferMGPU(FRHIBuffer* Buffer, uint32 GPUIndex)
	{
		checkf(IsTopOfPipe() || Bypass(), TEXT("Buffers may only be unlocked while recording RHI command lists, not during RHI command list execution."));

		GDynamicRHI->RHIUnlockBufferMGPU(*this, Buffer, GPUIndex);
	}

	[[nodiscard]]
	inline FRHIBufferInitializer CreateBufferInitializer(const FRHIBufferCreateDesc& CreateDesc)
	{
		FRHICommandListScopedPipelineGuard ScopedPipeline(*this);
		return GDynamicRHI->RHICreateBufferInitializer(*this, CreateDesc);
	}

	// Shortcut for creating a buffer without writing to an initializer
	[[nodiscard]]
	inline FBufferRHIRef CreateBuffer(const FRHIBufferCreateDesc& CreateDesc)
	{
		if (CreateDesc.Size == 0 && !CreateDesc.IsNull())
		{
			InvalidBufferFatalError(CreateDesc);
		}

		checkf(CreateDesc.InitAction != ERHIBufferInitAction::Initializer, TEXT("Buffer InitAction set to Initializer when calling CreateBuffer which doesn't write to its initializer"));

		FRHICommandListScopedPipelineGuard ScopedPipeline(*this);
		FRHIBufferInitializer Initializer = GDynamicRHI->RHICreateBufferInitializer(*this, CreateDesc);
		return Initializer.Finalize();
	}

	inline void UpdateUniformBuffer(FRHIUniformBuffer* UniformBufferRHI, const void* Contents)
	{
		FRHICommandListScopedPipelineGuard ScopedPipeline(*this);
		GDynamicRHI->RHIUpdateUniformBuffer(*this, UniformBufferRHI, Contents);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE::RHI::UniformBufferContentMap::Update(UniformBufferRHI, Contents);
#endif
	}

	inline void UpdateStreamSourceSlot(FRHIStreamSourceSlot* StreamSourceSlotRHI, FRHIBuffer* BufferRHI)
	{
		check(StreamSourceSlotRHI);
		if (Bypass())
		{
			StreamSourceSlotRHI->Buffer = BufferRHI;
		}
		else
		{
			EnqueueLambda(EThreadFence::Enabled, TEXT("UpdateStreamSourceSlot"),
				[this, StreamSourceSlotRHI, BufferRHI](FRHICommandListBase&)
				{
					StreamSourceSlotRHI->Buffer = BufferRHI;
				});
		}
	}

	inline void UpdateTexture2D(FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData)
	{
		checkf(UpdateRegion.DestX + UpdateRegion.Width <= Texture->GetSizeX(), TEXT("UpdateTexture2D out of bounds on X. Texture: %s, %i, %i, %i"), *Texture->GetName().ToString(), UpdateRegion.DestX, UpdateRegion.Width, Texture->GetSizeX());
		checkf(UpdateRegion.DestY + UpdateRegion.Height <= Texture->GetSizeY(), TEXT("UpdateTexture2D out of bounds on Y. Texture: %s, %i, %i, %i"), *Texture->GetName().ToString(), UpdateRegion.DestY, UpdateRegion.Height, Texture->GetSizeY());
		LLM_SCOPE(ELLMTag::Textures);

		FRHICommandListScopedPipelineGuard ScopedPipeline(*this);
		GDynamicRHI->RHIUpdateTexture2D(*this, Texture, MipIndex, UpdateRegion, SourcePitch, SourceData);
	}

	[[nodiscard]]
	inline FRHITextureInitializer CreateTextureInitializer(const FRHITextureCreateDesc& CreateDesc)
	{
		LLM_SCOPE(EnumHasAnyFlags(CreateDesc.Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable) ? ELLMTag::RenderTargets : ELLMTag::Textures);

		if (CreateDesc.InitialState == ERHIAccess::Unknown)
		{
			// Need to copy the incoming descriptor since we need to override the initial state.
			FRHITextureCreateDesc NewCreateDesc(CreateDesc);
			NewCreateDesc.SetInitialState(RHIGetDefaultResourceState(CreateDesc.Flags, CreateDesc.BulkData != nullptr));

			return GDynamicRHI->RHICreateTextureInitializer(*this, NewCreateDesc);
		}

		return GDynamicRHI->RHICreateTextureInitializer(*this, CreateDesc);
	}

	inline FTextureRHIRef CreateTexture(const FRHITextureCreateDesc& CreateDesc)
	{
		FRHITextureInitializer Initializer = CreateTextureInitializer(CreateDesc);
		return Initializer.Finalize();
	}

	inline void UpdateFromBufferTexture2D(FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, FRHIBuffer* Buffer, uint32 BufferOffset)
	{
		checkf(UpdateRegion.DestX + UpdateRegion.Width <= Texture->GetSizeX(), TEXT("UpdateFromBufferTexture2D out of bounds on X. Texture: %s, %i, %i, %i"), *Texture->GetName().ToString(), UpdateRegion.DestX, UpdateRegion.Width, Texture->GetSizeX());
		checkf(UpdateRegion.DestY + UpdateRegion.Height <= Texture->GetSizeY(), TEXT("UpdateFromBufferTexture2D out of bounds on Y. Texture: %s, %i, %i, %i"), *Texture->GetName().ToString(), UpdateRegion.DestY, UpdateRegion.Height, Texture->GetSizeY());
		LLM_SCOPE(ELLMTag::Textures);

		FRHICommandListScopedPipelineGuard ScopedPipeline(*this);
		GDynamicRHI->RHIUpdateFromBufferTexture2D(*this, Texture, MipIndex, UpdateRegion, SourcePitch, Buffer, BufferOffset);
	}

	inline void UpdateTexture3D(FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData)
	{
		checkf(UpdateRegion.DestX + UpdateRegion.Width <= Texture->GetSizeX(), TEXT("UpdateTexture3D out of bounds on X. Texture: %s, %i, %i, %i"), *Texture->GetName().ToString(), UpdateRegion.DestX, UpdateRegion.Width, Texture->GetSizeX());
		checkf(UpdateRegion.DestY + UpdateRegion.Height <= Texture->GetSizeY(), TEXT("UpdateTexture3D out of bounds on Y. Texture: %s, %i, %i, %i"), *Texture->GetName().ToString(), UpdateRegion.DestY, UpdateRegion.Height, Texture->GetSizeY());
		checkf(UpdateRegion.DestZ + UpdateRegion.Depth <= Texture->GetSizeZ(), TEXT("UpdateTexture3D out of bounds on Z. Texture: %s, %i, %i, %i"), *Texture->GetName().ToString(), UpdateRegion.DestZ, UpdateRegion.Depth, Texture->GetSizeZ());
		LLM_SCOPE(ELLMTag::Textures);

		FRHICommandListScopedPipelineGuard ScopedPipeline(*this);
		GDynamicRHI->RHIUpdateTexture3D(*this, Texture, MipIndex, UpdateRegion, SourceRowPitch, SourceDepthPitch, SourceData);
	}

	inline FTextureReferenceRHIRef CreateTextureReference(FRHITexture* InReferencedTexture = nullptr)
	{
		return GDynamicRHI->RHICreateTextureReference(*this, InReferencedTexture);
	}

	RHI_API void UpdateTextureReference(FRHITextureReference* TextureRef, FRHITexture* NewTexture);

	inline FShaderResourceViewRHIRef CreateShaderResourceView(FRHIBuffer* Buffer, FRHIViewDesc::FBufferSRV::FInitializer const& ViewDesc)
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateShaderResourceView"));
		return GDynamicRHI->RHICreateShaderResourceView(*this, Buffer, ViewDesc);
	}

	inline FShaderResourceViewRHIRef CreateShaderResourceView(FRHITexture* Texture, FRHIViewDesc::FTextureSRV::FInitializer const& ViewDesc)
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateShaderResourceView"));
		checkf(Texture->GetTextureReference() == nullptr, TEXT("Creating a shader resource view of an FRHITextureReference is not supported."));

		return GDynamicRHI->RHICreateShaderResourceView(*this, Texture, ViewDesc);
	}

	inline FUnorderedAccessViewRHIRef CreateUnorderedAccessView(FRHIBuffer* Buffer, FRHIViewDesc::FBufferUAV::FInitializer const& ViewDesc)
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateUnorderedAccessView"));
		return GDynamicRHI->RHICreateUnorderedAccessView(*this, Buffer, ViewDesc);
	}

	inline FUnorderedAccessViewRHIRef CreateUnorderedAccessView(FRHITexture* Texture, FRHIViewDesc::FTextureUAV::FInitializer const& ViewDesc)
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateUnorderedAccessView"));
		checkf(Texture->GetTextureReference() == nullptr, TEXT("Creating an unordered access view of an FRHITextureReference is not supported."));

		return GDynamicRHI->RHICreateUnorderedAccessView(*this, Texture, ViewDesc);
	}

	inline FShaderResourceViewRHIRef CreateShaderResourceView(const FShaderResourceViewInitializer& Initializer)
	{
		return CreateShaderResourceView(Initializer.Buffer, Initializer);
	}

	inline FRHIResourceCollectionRef CreateResourceCollection(TConstArrayView<FRHIResourceCollectionMember> InMembers)
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateResourceCollection"));
		return GDynamicRHI->RHICreateResourceCollection(*this, InMembers);
	}

	inline void UpdateResourceCollection(FRHIResourceCollection* InResourceCollection, uint32 InStartIndex, TConstArrayView<FRHIResourceCollectionMember> InMemberUpdates)
	{
		return GDynamicRHI->RHIUpdateResourceCollection(*this, InResourceCollection, InStartIndex, InMemberUpdates);
	}

	FDescriptorRangeRHIRef CreateDescriptorRange(const FRHIDescriptorRangeDesc& Desc, TConstArrayView<FRHIDescriptorResource> InitialResources)
	{
		return GDynamicRHI->RHICreateDescriptorRange(*this, Desc, InitialResources);
	}

	void UpdateDescriptorRange(FRHIDescriptorRange* DescriptorRange, uint32 FirstResourceIndex, TConstArrayView<FRHIDescriptorResource> NewResources)
	{
		GDynamicRHI->RHIUpdateDescriptorRange(*this, DescriptorRange, FirstResourceIndex, NewResources);
	}

	inline FRayTracingGeometryRHIRef CreateRayTracingGeometry(const FRayTracingGeometryInitializer& Initializer)
	{
		return GDynamicRHI->RHICreateRayTracingGeometry(*this, Initializer);
	}

	inline FShaderBindingTableRHIRef CreateRayTracingShaderBindingTable(const FRayTracingShaderBindingTableInitializer& Initializer)
	{
		return GDynamicRHI->RHICreateShaderBindingTable(*this, Initializer);
	}
	
	inline void ReplaceResources(TArray<FRHIResourceReplaceInfo>&& ReplaceInfos)
	{
		if (ReplaceInfos.Num() == 0)
		{
			return;
		}

		GDynamicRHI->RHIReplaceResources(*this, MoveTemp(ReplaceInfos));
	}

	inline void BindDebugLabelName(FRHITexture* Texture, const TCHAR* Name)
	{
		GDynamicRHI->RHIBindDebugLabelName(*this, Texture, Name);
	}

	inline void BindDebugLabelName(FRHIBuffer* Buffer, const TCHAR* Name)
	{
		GDynamicRHI->RHIBindDebugLabelName(*this, Buffer, Name);
	}

	inline void BindDebugLabelName(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const TCHAR* Name)
	{
		GDynamicRHI->RHIBindDebugLabelName(*this, UnorderedAccessViewRHI, Name);
	}

	inline FRHIBatchedShaderParameters& GetScratchShaderParameters()
	{
		FRHIBatchedShaderParameters*& ScratchShaderParameters = ShaderParameterState.ScratchShaderParameters;

		if (!ScratchShaderParameters)
		{
			ScratchShaderParameters = new (MemManager) FRHIBatchedShaderParameters(*CreateBatchedShaderParameterAllocator(ERHIBatchedShaderParameterAllocatorPageSize::Small));
		}

		if (!ensureMsgf(!ScratchShaderParameters->HasParameters(), TEXT("Scratch shader parameters left without committed parameters")))
		{
			ScratchShaderParameters->Reset();
		}

		return *ScratchShaderParameters;
	}

	inline FRHIBatchedShaderUnbinds& GetScratchShaderUnbinds()
	{
		if (!ensureMsgf(!ScratchShaderUnbinds.HasParameters(), TEXT("Scratch shader parameters left without committed parameters")))
		{
			ScratchShaderUnbinds.Reset();
		}
		return ScratchShaderUnbinds;
	}

	// Returns true if the RHI needs unbind commands
	bool NeedsShaderUnbinds() const
	{
		return GRHIGlobals.NeedsShaderUnbinds;
	}

	// Returns true if the underlying RHI needs implicit transitions inside of certain methods.
	bool NeedsExtraTransitions() const
	{
		return GRHIGlobals.NeedsExtraTransitions && bAllowExtraTransitions;
	}

	// Returns old state of bAllowExtraTransitions
	bool SetAllowExtraTransitions(bool NewState)
	{
		bool OldState = bAllowExtraTransitions;
		bAllowExtraTransitions = NewState;
		return OldState;
	}

	FRHIBatchedShaderParametersAllocator* CreateBatchedShaderParameterAllocator(ERHIBatchedShaderParameterAllocatorPageSize PageSize)
	{
		return new (MemManager) FRHIBatchedShaderParametersAllocator(ShaderParameterState.AllocatorsRoot, *this, PageSize);
	}

	/*
	* Writes the GPU addresses of the provided resources into a GPU buffer that can be accessed in shaders.
	* Addresses should be queried immediately before use, as certain RHI operations such as lock/update/defragment may move the resources in memory.
	*/
	void WriteResourceGPUAddresses(TConstArrayView<FRHIResource*> Resources, FRHIBuffer* OutputBuffer)
	{
		if (!Bypass())
		{
			Resources = AllocArray(Resources);
		}

		for (uint32 GPUIndex : GetGPUMask())
		{
			uint64* AddressesPtr = (uint64*)LockBufferMGPU(
				OutputBuffer,
				GPUIndex,
				0,
				Resources.Num() * sizeof(uint64), RLM_WriteOnly);

			GDynamicRHI->RHIWriteResourceGPUAddresses(*this, GPUIndex, Resources, AddressesPtr);

			UnlockBufferMGPU(OutputBuffer, GPUIndex);
		}
	}

protected:
	FMemStackBase& GetAllocator() { return MemManager; }

	inline void ValidateBoundShader(const FRHIComputeShader* ShaderRHI)
	{
		checkSlow(PersistentState.BoundComputeShaderRHI == ShaderRHI);
	}

	inline void ValidateBoundShader(const FRHIGraphicsShader* ShaderRHI)
	{
#if DO_GUARD_SLOW
		switch (ShaderRHI->GetFrequency())
		{
		case SF_Vertex:        checkSlow(PersistentState.BoundShaderInput.GetVertexShader()        == ShaderRHI); break;
		case SF_Mesh:          checkSlow(PersistentState.BoundShaderInput.GetMeshShader()          == ShaderRHI); break;
		case SF_Amplification: checkSlow(PersistentState.BoundShaderInput.GetAmplificationShader() == ShaderRHI); break;
		case SF_Pixel:         checkSlow(PersistentState.BoundShaderInput.GetPixelShader()         == ShaderRHI); break;
		case SF_Geometry:      checkSlow(PersistentState.BoundShaderInput.GetGeometryShader()      == ShaderRHI); break;
		default: checkfSlow(false, TEXT("Unexpected shader type %d"), ShaderRHI->GetFrequency());
		}
#endif // DO_GUARD_SLOW
	}

	inline void ValidateShaderParameters(const FRHIBatchedShaderParameters& ShaderParameters)
	{
#if RHI_VALIDATE_BATCHED_SHADER_PARAMETERS
		check(this == &ShaderParameters.Allocator.RHICmdList);
#endif
	}

	inline void ValidateShaderBundleComputeDispatch(TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches)
	{
#if RHI_VALIDATE_BATCHED_SHADER_PARAMETERS
		for (const FRHIShaderBundleComputeDispatch& Dispatch : Dispatches)
		{
			if (Dispatch.IsValid())
			{
				ValidateShaderParameters(*Dispatch.Parameters);
			}
		}
#endif
	}

	void CacheActiveRenderTargets(const FRHIRenderPassInfo& Info)
	{
		FRHISetRenderTargetsInfo RTInfo;
		Info.ConvertToRenderTargetsInfo(RTInfo);

		for (int32 RTIdx = 0; RTIdx < RTInfo.NumColorRenderTargets; ++RTIdx)
		{
			PersistentState.CachedRenderTargets[RTIdx] = RTInfo.ColorRenderTarget[RTIdx];
		}

		PersistentState.CachedNumSimultanousRenderTargets = RTInfo.NumColorRenderTargets;
		PersistentState.CachedDepthStencilTarget = RTInfo.DepthStencilRenderTarget;
		PersistentState.bHasFragmentDensityAttachment = RTInfo.ShadingRateTexture != nullptr;
		PersistentState.MultiViewCount = RTInfo.MultiViewCount;
	}

	void IncrementSubpass()
	{
		PersistentState.SubpassIndex++;
	}
	
	void ResetSubpass(ESubpassHint SubpassHint)
	{
		PersistentState.SubpassHint = SubpassHint;
		PersistentState.SubpassIndex = 0;
	}

	void AddPendingBufferUpload(FRHIBuffer* InBuffer)
	{
		PendingBufferUploads.Emplace(InBuffer);
	}

	void RemovePendingBufferUpload(FRHIBuffer* InBuffer)
	{
		check(PendingBufferUploads.Contains(InBuffer));
		PendingBufferUploads.Remove(InBuffer);
	}

	void AddPendingTextureUpload(FRHITexture* InTexture)
	{
		PendingTextureUploads.Emplace(InTexture);
	}

	void RemovePendingTextureUpload(FRHITexture* InTexture)
	{
		check(PendingTextureUploads.Contains(InTexture));
		PendingTextureUploads.Remove(InTexture);
	}

protected:
	FRHICommandBase*    Root            = nullptr;
	FRHICommandBase**   CommandLink     = nullptr;

	// The active context into which graphics commands are recorded.
	IRHICommandContext* GraphicsContext = nullptr;

	// The active compute context into which (possibly async) compute commands are recorded.
	IRHIComputeContext* ComputeContext  = nullptr;

	// The active upload context into which RHI specific commands are recorded.
	IRHIUploadContext* UploadContext 	= nullptr;
	
	// The RHI contexts available to the command list during execution.
	// These are always set for the immediate command list, see InitializeImmediateContexts().
	FRHIContextArray Contexts { InPlace, nullptr };

	uint32 NumCommands           = 0;
	bool bExecuting              = false;
	bool bAllowParallelTranslate = true;
	bool bUsesSetTrackedAccess   = false;
	bool bUsesShaderBundles      = false;
	bool bUsesLockFence          = false;
	bool bAllowExtraTransitions  = true;

	// The currently selected pipelines that RHI commands are directed to, during command list recording.
	// This is also adjusted during command list execution based on recorded use of ActivatePipeline().
	ERHIPipeline ActivePipelines = ERHIPipeline::None;

#if DO_CHECK
	// Used to check for valid pipelines passed to ActivatePipeline().
	ERHIPipeline AllowedPipelines = ERHIPipeline::All;
#endif

	struct FRHICommandRHIThreadFence* LastLockFenceCommand = nullptr;

	TArray<FRHICommandListBase*> AttachedCmdLists;

	TSharedPtr<FRHIParallelRenderPassInfo> SubRenderPassInfo;
	TSharedPtr<FRHIParallelRenderPassInfo> ParallelRenderPassBegin;
	TSharedPtr<FRHIParallelRenderPassInfo> ParallelRenderPassEnd;

	// Graph event used to gate the execution of the command list on the completion of any dependent tasks
	// e.g. PSO async compilation and parallel RHICmdList recording tasks.
	FGraphEventRef DispatchEvent;

	struct FShaderParameterState
	{
		FRHIBatchedShaderParameters* ScratchShaderParameters = nullptr;
		FRHIBatchedShaderParametersAllocator* AllocatorsRoot = nullptr;

		FShaderParameterState() = default;

		FShaderParameterState(FShaderParameterState&& RHS)
		{
			AllocatorsRoot = RHS.AllocatorsRoot;
			ScratchShaderParameters = RHS.ScratchShaderParameters;
			RHS.AllocatorsRoot = nullptr;
			RHS.ScratchShaderParameters = nullptr;
		}

		~FShaderParameterState()
		{
			if (ScratchShaderParameters)
			{
				ScratchShaderParameters->~FRHIBatchedShaderParameters();
				ScratchShaderParameters = nullptr;
			}

			for (FRHIBatchedShaderParametersAllocator* Node = AllocatorsRoot; Node; Node = Node->Next)
			{
				Node->~FRHIBatchedShaderParametersAllocator();
			}
			AllocatorsRoot = nullptr;
		}
	};

	FShaderParameterState ShaderParameterState;
	FRHIBatchedShaderUnbinds ScratchShaderUnbinds;

#if WITH_RHI_BREADCRUMBS

	struct
	{
		FRHIBreadcrumbNode* Current = FRHIBreadcrumbNode::Sentinel;
		FRHIBreadcrumbList UnknownParentList {};
		bool bEmitBreadcrumbs = false;
	} CPUBreadcrumbState {};

	struct FBreadcrumbState
	{
		FRHIBreadcrumbNode* Current = FRHIBreadcrumbNode::Sentinel;
		TOptional<FRHIBreadcrumbNode*> Latest {};
		FRHIBreadcrumbNode* Prev = nullptr;
		FRHIBreadcrumbRange Range {};
	};

	TRHIPipelineArray<FBreadcrumbState> GPUBreadcrumbState { InPlace };

	FRHIBreadcrumbAllocatorArray BreadcrumbAllocatorRefs {};
	TSharedPtr<FRHIBreadcrumbAllocator> BreadcrumbAllocator;

	struct FActivatePipelineCommand
	{
		FActivatePipelineCommand* Next = nullptr;
		FRHIBreadcrumbNode* Target = nullptr;
		ERHIPipeline Pipelines;
	};
	struct
	{
		FActivatePipelineCommand* First = nullptr;
		FActivatePipelineCommand* Prev = nullptr;
	} ActivatePipelineCommands {};

#endif

	// The values in this struct are preserved when the command list is moved or reset.
	struct FPersistentState
	{
		uint32 CachedNumSimultanousRenderTargets = 0;
		TStaticArray<FRHIRenderTargetView, MaxSimultaneousRenderTargets> CachedRenderTargets;
		FRHIDepthRenderTargetView CachedDepthStencilTarget;

		ESubpassHint SubpassHint = ESubpassHint::None;
		uint8 SubpassIndex = 0;
		uint8 MultiViewCount = 0;

		uint8 bHasFragmentDensityAttachment		: 1;

		uint8 bInsideRenderPass					: 1;
		uint8 bInsideComputePass				: 1;
		uint8 bInsideOcclusionQueryBatch		: 1;
		uint8 bRecursive						: 1;
		uint8 bImmediate						: 1;
		uint8 bAllowResourceStateTracking		: 1;

		FRHIGPUMask CurrentGPUMask;
		FRHIGPUMask InitialGPUMask;

		FBoundShaderStateInput BoundShaderInput;
		FRHIComputeShader* BoundComputeShaderRHI = nullptr;
		FRHICommandListScopedFence* CurrentFenceScope = nullptr;

	#if WITH_RHI_BREADCRUMBS
		FRHIBreadcrumbNode* LocalBreadcrumb = FRHIBreadcrumbNode::Sentinel;
	#endif

		TStaticArray<void*, MAX_NUM_GPUS> QueryBatchData_Timestamp { InPlace, nullptr };
		TStaticArray<void*, MAX_NUM_GPUS> QueryBatchData_Occlusion { InPlace, nullptr };

		FPersistentState(FRHIGPUMask InInitialGPUMask, bool bInImmediate = false, bool bTrackResources = true)
			: bHasFragmentDensityAttachment(0)
			, bInsideRenderPass(0)
			, bInsideComputePass(0)
			, bInsideOcclusionQueryBatch(0)
			, bRecursive(0)
			, bImmediate(bInImmediate)
			, bAllowResourceStateTracking(bTrackResources)
			, CurrentGPUMask(InInitialGPUMask)
			, InitialGPUMask(InInitialGPUMask)
		{}

	} PersistentState;

	TArray<FRHIBuffer*> PendingBufferUploads;
	TArray<FRHITexture*> PendingTextureUploads;

public:
#if WITH_RHI_BREADCRUMBS
	friend FRHIBreadcrumbEventManual;
	friend FRHIBreadcrumbScope;

	FRHIBreadcrumbNode*& GetCurrentBreadcrumbRef()
	{
		return PersistentState.LocalBreadcrumb;
	}

	RHI_API void AttachBreadcrumbSubTree(FRHIBreadcrumbAllocator& Allocator, FRHIBreadcrumbList& Nodes);
#endif

	TStaticArray<void*, MAX_NUM_GPUS>& GetQueryBatchData(ERenderQueryType QueryType)
	{
		switch (QueryType)
		{
		default: checkNoEntry(); [[fallthrough]];
		case RQT_AbsoluteTime: return PersistentState.QueryBatchData_Timestamp;
		case RQT_Occlusion:    return PersistentState.QueryBatchData_Occlusion;
		}
	}

private:
	FRHICommandListBase(FPersistentState const& InPersistentState);

	// Replays recorded commands. Used internally, do not call directly.
	RHI_EXECUTE_API void Execute();

	friend class FRHIScopedResourceBarrier;
	friend class FRHICommandListExecutor;
	friend class FRHICommandListIterator;
	friend class FRHICommandListScopedFlushAndExecute;
	friend class FRHICommandListScopedFence;
	friend class FRHIComputeCommandList;
	friend class FRHISubCommandList;
	friend class FRHICommandListImmediate;
	friend class FRHICommandList_RecursiveHazardous;
	friend class FRHIComputeCommandList_RecursiveHazardous;
	friend struct FRHICommandSetGPUMask;
	friend struct FRHIBufferInitializer;
	friend struct FRHITextureInitializer;

	template <typename RHICmdListType, typename LAMBDA>
	friend struct TRHILambdaCommandMultiPipe;

#if WITH_RHI_BREADCRUMBS
	friend bool IRHIComputeContext::ShouldEmitBreadcrumbs() const;
#endif
};

#if WITH_RHI_BREADCRUMBS
//
// Returns true if RHI breadcrumb strings should be emitted to platform GPU profiling APIs.
// Platform RHI implementations should check for this inside RHIBeginBreadcrumbGPU and RHIEndBreadcrumbGPU.
//
inline bool IRHIComputeContext::ShouldEmitBreadcrumbs() const
{
#if WITH_RHI_BREADCRUMBS_FULL
	return GetExecutingCommandList().CPUBreadcrumbState.bEmitBreadcrumbs;
#else
	return false;
#endif
}
#endif

struct FUnnamedRhiCommand
{
	static const TCHAR* TStr() { return TEXT("FUnnamedRhiCommand"); }
};

template<typename TCmd, typename NameType = FUnnamedRhiCommand>
struct FRHICommand : public FRHICommandBase
{
#if RHICOMMAND_CALLSTACK
	uint64 StackFrames[16];

	FRHICommand()
	{
		FPlatformStackWalk::CaptureStackBackTrace(StackFrames, 16);
	}
#endif

	void ExecuteAndDestruct(FRHICommandListBase& CmdList) override final
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/CommandList/ExecuteAndDestruct"));
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR(NameType::TStr(), RHICommandsChannel);

		TCmd* ThisCmd = static_cast<TCmd*>(this);
		ThisCmd->Execute(CmdList);
		ThisCmd->~TCmd();
	}
};

#define FRHICOMMAND_UNNAMED(CommandName)							\
	struct CommandName final : public FRHICommand<CommandName, FUnnamedRhiCommand>

#define FRHICOMMAND_UNNAMED_TPL(TemplateType, CommandName)			\
	template<typename TemplateType>									\
	struct CommandName final : public FRHICommand<CommandName<TemplateType>, FUnnamedRhiCommand>

#define FRHICOMMAND_MACRO(CommandName)								\
	struct UE_JOIN(CommandName##String, __LINE__)					\
	{																\
		static const TCHAR* TStr() { return TEXT(#CommandName); }	\
	};																\
	struct CommandName final : public FRHICommand<CommandName, UE_JOIN(CommandName##String, __LINE__)>

#define FRHICOMMAND_MACRO_TPL(TemplateType, CommandName)			\
	struct UE_JOIN(CommandName##String, __LINE__)					\
	{																\
		static const TCHAR* TStr() { return TEXT(#CommandName); }	\
	};																\
	template<typename TemplateType>									\
	struct CommandName final : public FRHICommand<CommandName<TemplateType>, UE_JOIN(CommandName##String, __LINE__)>

FRHICOMMAND_MACRO(FRHICommandBeginUpdateMultiFrameResource)
{
	FRHITexture* Texture;
	inline FRHICommandBeginUpdateMultiFrameResource(FRHITexture* InTexture)
		: Texture(InTexture)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandEndUpdateMultiFrameResource)
{
	FRHITexture* Texture;
	inline FRHICommandEndUpdateMultiFrameResource(FRHITexture* InTexture)
		: Texture(InTexture)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandBeginUpdateMultiFrameUAV)
{
	FRHIUnorderedAccessView* UAV;
	inline FRHICommandBeginUpdateMultiFrameUAV(FRHIUnorderedAccessView* InUAV)
		: UAV(InUAV)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandEndUpdateMultiFrameUAV)
{
	FRHIUnorderedAccessView* UAV;
	inline FRHICommandEndUpdateMultiFrameUAV(FRHIUnorderedAccessView* InUAV)
		: UAV(InUAV)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

#if WITH_MGPU
FRHICOMMAND_MACRO(FRHICommandSetGPUMask)
{
	FRHIGPUMask GPUMask;
	inline FRHICommandSetGPUMask(FRHIGPUMask InGPUMask)
		: GPUMask(InGPUMask)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandTransferResources)
{
	TConstArrayView<FTransferResourceParams> Params;

	inline FRHICommandTransferResources(TConstArrayView<FTransferResourceParams> InParams)
		: Params(InParams)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandTransferResourceSignal)
{
	TConstArrayView<FTransferResourceFenceData*> FenceDatas;
	FRHIGPUMask SrcGPUMask;

	inline FRHICommandTransferResourceSignal(TConstArrayView<FTransferResourceFenceData*> InFenceDatas, FRHIGPUMask InSrcGPUMask)
		: FenceDatas(InFenceDatas)
		, SrcGPUMask(InSrcGPUMask)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandTransferResourceWait)
{
	TConstArrayView<FTransferResourceFenceData*> FenceDatas;

	inline FRHICommandTransferResourceWait(TConstArrayView<FTransferResourceFenceData*> InFenceDatas)
		: FenceDatas(InFenceDatas)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandCrossGPUTransfer)
{
	TConstArrayView<FTransferResourceParams> Params;
	TConstArrayView<FCrossGPUTransferFence*> PreTransfer;
	TConstArrayView<FCrossGPUTransferFence*> PostTransfer;

	inline FRHICommandCrossGPUTransfer(TConstArrayView<FTransferResourceParams> InParams, TConstArrayView<FCrossGPUTransferFence*> InPreTransfer, TConstArrayView<FCrossGPUTransferFence*> InPostTransfer)
		: Params(InParams)
		, PreTransfer(InPreTransfer)
		, PostTransfer(InPostTransfer)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandCrossGPUTransferSignal)
{
	TConstArrayView<FTransferResourceParams> Params;
	TConstArrayView<FCrossGPUTransferFence*> PreTransfer;

	inline FRHICommandCrossGPUTransferSignal(TConstArrayView<FTransferResourceParams> InParams, TConstArrayView<FCrossGPUTransferFence*> InPreTransfer)
		: Params(InParams)
		, PreTransfer(InPreTransfer)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandCrossGPUTransferWait)
{
	TConstArrayView<FCrossGPUTransferFence*> SyncPoints;

	inline FRHICommandCrossGPUTransferWait(TConstArrayView<FCrossGPUTransferFence*> InSyncPoints)
		: SyncPoints(InSyncPoints)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};
#endif // WITH_MGPU

FRHICOMMAND_MACRO(FRHICommandSetStencilRef)
{
	uint32 StencilRef;
	inline FRHICommandSetStencilRef(uint32 InStencilRef)
		: StencilRef(InStencilRef)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO_TPL(TRHIShader, FRHICommandSetShaderParameters)
{
	TRHIShader* Shader;
	TConstArrayView<uint8> ParametersData;
	TConstArrayView<FRHIShaderParameter> Parameters;
	TConstArrayView<FRHIShaderParameterResource> ResourceParameters;
	TConstArrayView<FRHIShaderParameterResource> BindlessParameters;

	inline FRHICommandSetShaderParameters(
		TRHIShader* InShader
		, TConstArrayView<uint8> InParametersData
		, TConstArrayView<FRHIShaderParameter> InParameters
		, TConstArrayView<FRHIShaderParameterResource> InResourceParameters
		, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters
	)
		: Shader(InShader)
		, ParametersData(InParametersData)
		, Parameters(InParameters)
		, ResourceParameters(InResourceParameters)
		, BindlessParameters(InBindlessParameters)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO_TPL(TRHIShader, FRHICommandSetShaderUnbinds)
{
	TRHIShader* Shader;
	TConstArrayView<FRHIShaderParameterUnbind> Unbinds;

	inline FRHICommandSetShaderUnbinds(TRHIShader * InShader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
		: Shader(InShader)
		, Unbinds(InUnbinds)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandDrawPrimitive)
{
	uint32 BaseVertexIndex;
	uint32 NumPrimitives;
	uint32 NumInstances;
	inline FRHICommandDrawPrimitive(uint32 InBaseVertexIndex, uint32 InNumPrimitives, uint32 InNumInstances)
		: BaseVertexIndex(InBaseVertexIndex)
		, NumPrimitives(InNumPrimitives)
		, NumInstances(InNumInstances)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandDrawIndexedPrimitive)
{
	FRHIBuffer* IndexBuffer;
	int32 BaseVertexIndex;
	uint32 FirstInstance;
	uint32 NumVertices;
	uint32 StartIndex;
	uint32 NumPrimitives;
	uint32 NumInstances;
	inline FRHICommandDrawIndexedPrimitive(FRHIBuffer* InIndexBuffer, int32 InBaseVertexIndex, uint32 InFirstInstance, uint32 InNumVertices, uint32 InStartIndex, uint32 InNumPrimitives, uint32 InNumInstances)
		: IndexBuffer(InIndexBuffer)
		, BaseVertexIndex(InBaseVertexIndex)
		, FirstInstance(InFirstInstance)
		, NumVertices(InNumVertices)
		, StartIndex(InStartIndex)
		, NumPrimitives(InNumPrimitives)
		, NumInstances(InNumInstances)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetBlendFactor)
{
	FLinearColor BlendFactor;
	inline FRHICommandSetBlendFactor(const FLinearColor& InBlendFactor)
		: BlendFactor(InBlendFactor)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetStreamSource)
{
	uint32 StreamIndex;
	FRHIBuffer* VertexBuffer;
	uint32 Offset;
	inline FRHICommandSetStreamSource(uint32 InStreamIndex, FRHIBuffer* InVertexBuffer, uint32 InOffset)
		: StreamIndex(InStreamIndex)
		, VertexBuffer(InVertexBuffer)
		, Offset(InOffset)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetViewport)
{
	float MinX;
	float MinY;
	float MinZ;
	float MaxX;
	float MaxY;
	float MaxZ;
	inline FRHICommandSetViewport(float InMinX, float InMinY, float InMinZ, float InMaxX, float InMaxY, float InMaxZ)
		: MinX(InMinX)
		, MinY(InMinY)
		, MinZ(InMinZ)
		, MaxX(InMaxX)
		, MaxY(InMaxY)
		, MaxZ(InMaxZ)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetStereoViewport)
{
	float LeftMinX;
	float RightMinX;
	float LeftMinY;
	float RightMinY;
	float MinZ;
	float LeftMaxX;
	float RightMaxX;
	float LeftMaxY;
	float RightMaxY;
	float MaxZ;
	inline FRHICommandSetStereoViewport(float InLeftMinX, float InRightMinX, float InLeftMinY, float InRightMinY, float InMinZ, float InLeftMaxX, float InRightMaxX, float InLeftMaxY, float InRightMaxY, float InMaxZ)
		: LeftMinX(InLeftMinX)
		, RightMinX(InRightMinX)
		, LeftMinY(InLeftMinY)
		, RightMinY(InRightMinY)
		, MinZ(InMinZ)
		, LeftMaxX(InLeftMaxX)
		, RightMaxX(InRightMaxX)
		, LeftMaxY(InLeftMaxY)
		, RightMaxY(InRightMaxY)
		, MaxZ(InMaxZ)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetScissorRect)
{
	bool bEnable;
	uint32 MinX;
	uint32 MinY;
	uint32 MaxX;
	uint32 MaxY;
	inline FRHICommandSetScissorRect(bool InbEnable, uint32 InMinX, uint32 InMinY, uint32 InMaxX, uint32 InMaxY)
		: bEnable(InbEnable)
		, MinX(InMinX)
		, MinY(InMinY)
		, MaxX(InMaxX)
		, MaxY(InMaxY)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandBeginRenderPass)
{
	FRHIRenderPassInfo& Info;
	const TCHAR* Name;

	FRHICommandBeginRenderPass(FRHIRenderPassInfo& InInfo, const TCHAR* InName)
		: Info(InInfo)
		, Name(InName)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandEndRenderPass)
{
	FRHICommandEndRenderPass()
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandNextSubpass)
{
	FRHICommandNextSubpass()
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetComputePipelineState)
{
	FComputePipelineState* ComputePipelineState;
	inline FRHICommandSetComputePipelineState(FComputePipelineState* InComputePipelineState)
		: ComputePipelineState(InComputePipelineState)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetGraphicsPipelineState)
{
	FGraphicsPipelineState* GraphicsPipelineState;
	uint32 StencilRef;
	bool bApplyAdditionalState;
	inline FRHICommandSetGraphicsPipelineState(FGraphicsPipelineState* InGraphicsPipelineState, uint32 InStencilRef, bool bInApplyAdditionalState)
		: GraphicsPipelineState(InGraphicsPipelineState)
		, StencilRef(InStencilRef)
		, bApplyAdditionalState(bInApplyAdditionalState)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

#if PLATFORM_USE_FALLBACK_PSO
FRHICOMMAND_MACRO(FRHICommandSetGraphicsPipelineStateFromInitializer)
{
	FGraphicsPipelineStateInitializer PsoInit;
	uint32 StencilRef;
	bool bApplyAdditionalState;
	inline FRHICommandSetGraphicsPipelineStateFromInitializer(const FGraphicsPipelineStateInitializer& InPsoInit, uint32 InStencilRef, bool bInApplyAdditionalState)
		: PsoInit(InPsoInit)
		, StencilRef(InStencilRef)
		, bApplyAdditionalState(bInApplyAdditionalState)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};
#endif

FRHICOMMAND_MACRO(FRHICommandDispatchComputeShader)
{
	uint32 ThreadGroupCountX;
	uint32 ThreadGroupCountY;
	uint32 ThreadGroupCountZ;
	inline FRHICommandDispatchComputeShader(uint32 InThreadGroupCountX, uint32 InThreadGroupCountY, uint32 InThreadGroupCountZ)
		: ThreadGroupCountX(InThreadGroupCountX)
		, ThreadGroupCountY(InThreadGroupCountY)
		, ThreadGroupCountZ(InThreadGroupCountZ)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandDispatchIndirectComputeShader)
{
	FRHIBuffer* ArgumentBuffer;
	uint32 ArgumentOffset;
	inline FRHICommandDispatchIndirectComputeShader(FRHIBuffer* InArgumentBuffer, uint32 InArgumentOffset)
		: ArgumentBuffer(InArgumentBuffer)
		, ArgumentOffset(InArgumentOffset)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

using FRHIRecordBundleComputeDispatchCallback = TFunction<void(FRHIShaderBundleComputeDispatch& Dispatch)>;
using FRHIRecordBundleGraphicsDispatchCallback = TFunction<void(FRHIShaderBundleGraphicsDispatch& Dispatch)>;

FRHICOMMAND_MACRO(FRHICommandDispatchComputeShaderBundle)
{
	FRHIShaderBundle* ShaderBundle;
	FRHIBuffer* RecordArgBuffer;
	TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters;
	TArray<FRHIShaderBundleComputeDispatch> Dispatches;
	ERHIShaderBundleMethod Method;
	inline FRHICommandDispatchComputeShaderBundle()
		: ShaderBundle(nullptr)
		, RecordArgBuffer(nullptr)
		, Method(ERHIShaderBundleMethod::Emulation)
	{
	}
	inline FRHICommandDispatchComputeShaderBundle(
		FRHIShaderBundle* InShaderBundle,
		FRHIBuffer* InRecordArgBuffer,
		TConstArrayView<FRHIShaderParameterResource> InSharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleComputeDispatch> InDispatches,
		ERHIShaderBundleMethod InMethod
	)
		: ShaderBundle(InShaderBundle)
		, RecordArgBuffer(InRecordArgBuffer)
		, SharedBindlessParameters(InSharedBindlessParameters)
		, Dispatches(InDispatches)
		, Method(InMethod)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandDispatchGraphicsShaderBundle)
{
	FRHIShaderBundle* ShaderBundle;
	FRHIBuffer* RecordArgBuffer;
	FRHIShaderBundleGraphicsState BundleState;
	TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters;
	TArray<FRHIShaderBundleGraphicsDispatch> Dispatches;
	ERHIShaderBundleMethod Method;
	inline FRHICommandDispatchGraphicsShaderBundle()
		: ShaderBundle(nullptr)
		, RecordArgBuffer(nullptr)
		, Method(ERHIShaderBundleMethod::Emulation)
	{
	}
	inline FRHICommandDispatchGraphicsShaderBundle(
		FRHIShaderBundle* InShaderBundle,
		FRHIBuffer* InRecordArgBuffer,
		const FRHIShaderBundleGraphicsState& InBundleState,
		TConstArrayView<FRHIShaderParameterResource> InSharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleGraphicsDispatch> InDispatches,
		ERHIShaderBundleMethod InMethod
	)
		: ShaderBundle(InShaderBundle)
		, RecordArgBuffer(InRecordArgBuffer)
		, BundleState(InBundleState)
		, SharedBindlessParameters(InSharedBindlessParameters)
		, Dispatches(InDispatches)
		, Method(InMethod)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetShaderRootConstants)
{
	const FUint32Vector4 Constants;
	inline FRHICommandSetShaderRootConstants()
		: Constants()
	{
	}
	inline FRHICommandSetShaderRootConstants(
		const FUint32Vector4 InConstants
	)
		: Constants(InConstants)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

#if ENABLE_RHI_VALIDATION
FRHICOMMAND_MACRO(FRHICommandBeginUAVOverlap)
{
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandEndUAVOverlap)
{
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandBeginSpecificUAVOverlap)
{
	TArrayView<FRHIUnorderedAccessView* const> UAVs;
	inline FRHICommandBeginSpecificUAVOverlap(TArrayView<FRHIUnorderedAccessView* const> InUAVs) : UAVs(InUAVs) {}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandEndSpecificUAVOverlap)
{
	TArrayView<FRHIUnorderedAccessView* const> UAVs;
	inline FRHICommandEndSpecificUAVOverlap(TArrayView<FRHIUnorderedAccessView* const> InUAVs) : UAVs(InUAVs) {}
	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};
#endif // ENABLE_RHI_VALIDATION

FRHICOMMAND_MACRO(FRHICommandDrawPrimitiveIndirect)
{
	FRHIBuffer* ArgumentBuffer;
	uint32 ArgumentOffset;
	inline FRHICommandDrawPrimitiveIndirect(FRHIBuffer* InArgumentBuffer, uint32 InArgumentOffset)
		: ArgumentBuffer(InArgumentBuffer)
		, ArgumentOffset(InArgumentOffset)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandDrawIndexedPrimitiveIndirect)
{
	FRHIBuffer* IndexBuffer;
	FRHIBuffer* ArgumentsBuffer;
	uint32 ArgumentOffset;

	inline FRHICommandDrawIndexedPrimitiveIndirect(FRHIBuffer* InIndexBuffer, FRHIBuffer* InArgumentsBuffer, uint32 InArgumentOffset)
		: IndexBuffer(InIndexBuffer)
		, ArgumentsBuffer(InArgumentsBuffer)
		, ArgumentOffset(InArgumentOffset)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandMultiDrawIndexedPrimitiveIndirect)
{
	FRHIBuffer* IndexBuffer;
	FRHIBuffer* ArgumentBuffer;
	uint32 ArgumentOffset;
	FRHIBuffer* CountBuffer;
	uint32 CountBufferOffset;
	uint32 MaxDrawArguments;
	inline FRHICommandMultiDrawIndexedPrimitiveIndirect(FRHIBuffer* InIndexBuffer, FRHIBuffer* InArgumentBuffer, uint32 InArgumentOffset, FRHIBuffer* InCountBuffer, uint32 InCountBufferOffset, uint32 InMaxDrawArguments)
		: IndexBuffer(InIndexBuffer)
		, ArgumentBuffer(InArgumentBuffer)
		, ArgumentOffset(InArgumentOffset)
		, CountBuffer(InCountBuffer)
		, CountBufferOffset(InCountBufferOffset)
		, MaxDrawArguments(InMaxDrawArguments)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandDispatchMeshShader)
{
	uint32 ThreadGroupCountX;
	uint32 ThreadGroupCountY;
	uint32 ThreadGroupCountZ;

	inline FRHICommandDispatchMeshShader(uint32 InThreadGroupCountX, uint32 InThreadGroupCountY, uint32 InThreadGroupCountZ)
		: ThreadGroupCountX(InThreadGroupCountX)
		, ThreadGroupCountY(InThreadGroupCountY)
		, ThreadGroupCountZ(InThreadGroupCountZ)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandDispatchIndirectMeshShader)
{
	FRHIBuffer* ArgumentBuffer;
	uint32 ArgumentOffset;

	inline FRHICommandDispatchIndirectMeshShader(FRHIBuffer * InArgumentBuffer, uint32 InArgumentOffset)
		: ArgumentBuffer(InArgumentBuffer)
		, ArgumentOffset(InArgumentOffset)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetDepthBounds)
{
	float MinDepth;
	float MaxDepth;

	inline FRHICommandSetDepthBounds(float InMinDepth, float InMaxDepth)
		: MinDepth(InMinDepth)
		, MaxDepth(InMaxDepth)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHIGpuHangCommandListCorruption)
{
	inline FRHIGpuHangCommandListCorruption(){}
	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetShadingRate)
{
	EVRSShadingRate   ShadingRate;
	EVRSRateCombiner  Combiner;

	inline FRHICommandSetShadingRate(EVRSShadingRate InShadingRate, EVRSRateCombiner InCombiner)
		: ShadingRate(InShadingRate),
		Combiner(InCombiner)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandClearUAVFloat)
{
	FRHIUnorderedAccessView* UnorderedAccessViewRHI;
	FVector4f Values;

	inline FRHICommandClearUAVFloat(FRHIUnorderedAccessView* InUnorderedAccessViewRHI, const FVector4f& InValues)
		: UnorderedAccessViewRHI(InUnorderedAccessViewRHI)
		, Values(InValues)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandClearUAVUint)
{
	FRHIUnorderedAccessView* UnorderedAccessViewRHI;
	FUintVector4 Values;

	inline FRHICommandClearUAVUint(FRHIUnorderedAccessView* InUnorderedAccessViewRHI, const FUintVector4& InValues)
		: UnorderedAccessViewRHI(InUnorderedAccessViewRHI)
		, Values(InValues)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandCopyTexture)
{
	FRHICopyTextureInfo CopyInfo;
	FRHITexture* SourceTexture;
	FRHITexture* DestTexture;

	inline FRHICommandCopyTexture(FRHITexture* InSourceTexture, FRHITexture* InDestTexture, const FRHICopyTextureInfo& InCopyInfo)
		: CopyInfo(InCopyInfo)
		, SourceTexture(InSourceTexture)
		, DestTexture(InDestTexture)
	{
		ensure(SourceTexture);
		ensure(DestTexture);
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandResummarizeHTile)
{
	FRHITexture* DepthTexture;

	inline FRHICommandResummarizeHTile(FRHITexture* InDepthTexture)
	: DepthTexture(InDepthTexture)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandBeginTransitions)
{
	TArrayView<const FRHITransition*> Transitions;

	FRHICommandBeginTransitions(TArrayView<const FRHITransition*> InTransitions)
		: Transitions(InTransitions)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandEndTransitions)
{
	TArrayView<const FRHITransition*> Transitions;

	FRHICommandEndTransitions(TArrayView<const FRHITransition*> InTransitions)
		: Transitions(InTransitions)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandResourceTransition)
{
	FRHITransition* Transition;

	FRHICommandResourceTransition(FRHITransition* InTransition)
		: Transition(InTransition)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetTrackedAccess)
{
	TArrayView<const FRHITrackedAccessInfo> Infos;

	FRHICommandSetTrackedAccess(TArrayView<const FRHITrackedAccessInfo> InInfos)
		: Infos(InInfos)
	{
	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetAsyncComputeBudget)
{
	EAsyncComputeBudget Budget;

	inline FRHICommandSetAsyncComputeBudget(EAsyncComputeBudget InBudget)
		: Budget(InBudget)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetComputeBudget)
{
	ESyncComputeBudget Budget;

	inline FRHICommandSetComputeBudget(ESyncComputeBudget InBudget)
		: Budget(InBudget)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandCopyToStagingBuffer)
{
	FRHIBuffer* SourceBuffer;
	FRHIStagingBuffer* DestinationStagingBuffer;
	uint32 Offset;
	uint32 NumBytes;

	inline FRHICommandCopyToStagingBuffer(FRHIBuffer* InSourceBuffer, FRHIStagingBuffer* InDestinationStagingBuffer, uint32 InOffset, uint32 InNumBytes)
		: SourceBuffer(InSourceBuffer)
		, DestinationStagingBuffer(InDestinationStagingBuffer)
		, Offset(InOffset)
		, NumBytes(InNumBytes)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandWriteGPUFence)
{
	FRHIGPUFence* Fence;

	inline FRHICommandWriteGPUFence(FRHIGPUFence* InFence)
		: Fence(InFence)
	{
		if (Fence)
		{
			Fence->NumPendingWriteCommands.Increment();
		}
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetStaticUniformBuffers)
{
	FUniformBufferStaticBindings UniformBuffers;

	inline FRHICommandSetStaticUniformBuffers(const FUniformBufferStaticBindings & InUniformBuffers)
		: UniformBuffers(InUniformBuffers)
	{}

	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetStaticUniformBuffer)
{
	FRHIUniformBuffer* Buffer;
	FUniformBufferStaticSlot Slot;

	inline FRHICommandSetStaticUniformBuffer(FUniformBufferStaticSlot InSlot, FRHIUniformBuffer* InBuffer)
		: Buffer(InBuffer)
		, Slot(InSlot)
	{}

	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetUniformBufferDynamicOffset)
{
	uint32 Offset;
	FUniformBufferStaticSlot Slot;

	inline FRHICommandSetUniformBufferDynamicOffset(FUniformBufferStaticSlot InSlot, uint32 InOffset)
		: Offset(InOffset)
		, Slot(InSlot)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandBeginRenderQuery)
{
	FRHIRenderQuery* RenderQuery;

	inline FRHICommandBeginRenderQuery(FRHIRenderQuery* InRenderQuery)
		: RenderQuery(InRenderQuery)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandEndRenderQuery)
{
	FRHIRenderQuery* RenderQuery;

	inline FRHICommandEndRenderQuery(FRHIRenderQuery* InRenderQuery)
		: RenderQuery(InRenderQuery)
	{
	}
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandPostExternalCommandsReset)
{
	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandCopyBufferRegion)
{
	FRHIBuffer* DestBuffer;
	uint64 DstOffset;
	FRHIBuffer* SourceBuffer;
	uint64 SrcOffset;
	uint64 NumBytes;

	explicit FRHICommandCopyBufferRegion(FRHIBuffer* InDestBuffer, uint64 InDstOffset, FRHIBuffer* InSourceBuffer, uint64 InSrcOffset, uint64 InNumBytes)
		: DestBuffer(InDestBuffer)
		, DstOffset(InDstOffset)
		, SourceBuffer(InSourceBuffer)
		, SrcOffset(InSrcOffset)
		, NumBytes(InNumBytes)
	{}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_UNNAMED(FRHICommandBindAccelerationStructureMemory)
{
	FRHIRayTracingScene* Scene;
	FRHIBuffer* Buffer;
	uint32 BufferOffset;

	FRHICommandBindAccelerationStructureMemory(FRHIRayTracingScene* InScene, FRHIBuffer* InBuffer, uint32 InBufferOffset)
		: Scene(InScene)
		, Buffer(InBuffer)
		, BufferOffset(InBufferOffset)
	{}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_UNNAMED(FRHICommandBuildSceneAccelerationStructures)
{
	TConstArrayView<FRayTracingSceneBuildParams> Params;

	explicit FRHICommandBuildSceneAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> InParams)
		: Params(InParams)
	{}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandCommitShaderBindingTable)
{
	FRHIShaderBindingTable* SBT;
	FRHIBuffer* InlineBindingDataBuffer;

	explicit FRHICommandCommitShaderBindingTable(FRHIShaderBindingTable* InSBT, FRHIBuffer* InInlineBindingDataBuffer)
		: SBT(InSBT), InlineBindingDataBuffer(InInlineBindingDataBuffer)	
	{}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_UNNAMED(FRHICommandBuildAccelerationStructures)
{
	TConstArrayView<FRayTracingGeometryBuildParams> Params;
	FRHIBufferRange ScratchBufferRange;
	FRHIBuffer* ScratchBuffer;

	explicit FRHICommandBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> InParams, const FRHIBufferRange& ScratchBufferRange)
		: Params(InParams)
		, ScratchBufferRange(ScratchBufferRange)
		, ScratchBuffer(ScratchBufferRange.Buffer)
	{}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_UNNAMED(FRHICommandExecuteMultiIndirectClusterOperation)
{
	FRayTracingClusterOperationParams Params;

	explicit FRHICommandExecuteMultiIndirectClusterOperation(const FRayTracingClusterOperationParams & InParams)
		: Params(InParams)
	{}

	RHI_EXECUTE_API void Execute(FRHICommandListBase & CmdList);
};

FRHICOMMAND_MACRO(FRHICommandRayTraceDispatch)
{
	FRayTracingPipelineState* Pipeline;
	FRHIRayTracingScene* Scene;
	FRHIShaderBindingTable* SBT;
	FRayTracingShaderBindings GlobalResourceBindings;
	FRHIRayTracingShader* RayGenShader;
	FRHIBuffer* ArgumentBuffer;
	uint32 ArgumentOffset;
	uint32 Width;
	uint32 Height;

	FRHICommandRayTraceDispatch(FRayTracingPipelineState* InPipeline, FRHIRayTracingShader* InRayGenShader, FRHIRayTracingScene* InScene, const FRayTracingShaderBindings& InGlobalResourceBindings, uint32 InWidth, uint32 InHeight)
		: Pipeline(InPipeline)
		, Scene(InScene)
		, SBT(nullptr)
		, GlobalResourceBindings(InGlobalResourceBindings)
		, RayGenShader(InRayGenShader)
		, ArgumentBuffer(nullptr)
		, ArgumentOffset(0)
		, Width(InWidth)
		, Height(InHeight)
	{}

	FRHICommandRayTraceDispatch(FRayTracingPipelineState* InPipeline, FRHIRayTracingShader* InRayGenShader, FRHIShaderBindingTable* InSBT, const FRayTracingShaderBindings& InGlobalResourceBindings, uint32 InWidth, uint32 InHeight)
		: Pipeline(InPipeline)
		, Scene(nullptr)
		, SBT(InSBT)
		, GlobalResourceBindings(InGlobalResourceBindings)
		, RayGenShader(InRayGenShader)
		, ArgumentBuffer(nullptr)
		, ArgumentOffset(0)
		, Width(InWidth)
		, Height(InHeight)
	{}

	FRHICommandRayTraceDispatch(FRayTracingPipelineState* InPipeline, FRHIRayTracingShader* InRayGenShader, FRHIRayTracingScene* InScene, const FRayTracingShaderBindings& InGlobalResourceBindings, FRHIBuffer* InArgumentBuffer, uint32 InArgumentOffset)
		: Pipeline(InPipeline)
		, Scene(InScene)
		, SBT(nullptr)
		, GlobalResourceBindings(InGlobalResourceBindings)
		, RayGenShader(InRayGenShader)
		, ArgumentBuffer(InArgumentBuffer)
		, ArgumentOffset(InArgumentOffset)
		, Width(0)
		, Height(0)
	{}

	FRHICommandRayTraceDispatch(FRayTracingPipelineState* InPipeline, FRHIRayTracingShader* InRayGenShader, FRHIShaderBindingTable* InSBT, const FRayTracingShaderBindings& InGlobalResourceBindings, FRHIBuffer* InArgumentBuffer, uint32 InArgumentOffset)
		: Pipeline(InPipeline)
		, Scene(nullptr)
		, SBT(InSBT)
		, GlobalResourceBindings(InGlobalResourceBindings)
		, RayGenShader(InRayGenShader)
		, ArgumentBuffer(InArgumentBuffer)
		, ArgumentOffset(InArgumentOffset)
		, Width(0)
		, Height(0)
	{}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

FRHICOMMAND_MACRO(FRHICommandSetBindingsOnShaderBindingTable)
{
	FRHIShaderBindingTable* SBT = nullptr;
	FRHIRayTracingScene* Scene = nullptr;
	FRayTracingPipelineState* Pipeline = nullptr;
	int32 NumBindings = -1;
	const FRayTracingLocalShaderBindings* Bindings = nullptr;
	ERayTracingBindingType BindingType = ERayTracingBindingType::HitGroup;

	// Bindings Batch
	FRHICommandSetBindingsOnShaderBindingTable(FRHIRayTracingScene* InScene, FRayTracingPipelineState* InPipeline, uint32 InNumBindings, const FRayTracingLocalShaderBindings* InBindings, ERayTracingBindingType InBindingType)
		: Scene(InScene)
		, Pipeline(InPipeline)
		, NumBindings(InNumBindings)
		, Bindings(InBindings)
		, BindingType(InBindingType)
	{

	}

	// Bindings Batch
	FRHICommandSetBindingsOnShaderBindingTable(FRHIShaderBindingTable* InSBT, FRayTracingPipelineState* InPipeline, uint32 InNumBindings, const FRayTracingLocalShaderBindings* InBindings, ERayTracingBindingType InBindingType)
		: SBT(InSBT)
		, Pipeline(InPipeline)
		, NumBindings(InNumBindings)
		, Bindings(InBindings)
		, BindingType(InBindingType)
	{

	}

	RHI_EXECUTE_API void Execute(FRHICommandListBase& CmdList);
};

template<> RHI_EXECUTE_API void FRHICommandSetShaderParameters           <FRHIComputeShader>::Execute(FRHICommandListBase& CmdList);
template<> RHI_EXECUTE_API void FRHICommandSetShaderUnbinds              <FRHIComputeShader>::Execute(FRHICommandListBase& CmdList);

extern RHI_API FRHIComputePipelineState*	ExecuteSetComputePipelineState(FComputePipelineState* ComputePipelineState);
extern RHI_API FRHIGraphicsPipelineState*	ExecuteSetGraphicsPipelineState(class FGraphicsPipelineState* GraphicsPipelineState);
extern RHI_API FComputePipelineState*		FindComputePipelineState(const FComputePipelineStateInitializer& Initializer, bool bVerifyUse = true);
extern RHI_API FComputePipelineState*		GetComputePipelineState(FRHIComputeCommandList& RHICmdList, const FComputePipelineStateInitializer& Initializer, bool bVerifyUse = true);
extern RHI_API FGraphicsPipelineState*		FindGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer, bool bVerifyUse = true);
extern RHI_API FGraphicsPipelineState*		GetGraphicsPipelineState(FRHICommandList& RHICmdList, const FGraphicsPipelineStateInitializer& Initializer, bool bVerifyUse = true);
extern RHI_API FRHIComputePipelineState*	GetRHIComputePipelineState(FComputePipelineState*);
extern RHI_API FRHIWorkGraphPipelineState*	GetRHIWorkGraphPipelineState(FWorkGraphPipelineState*);
extern RHI_API FRHIRayTracingPipelineState*	GetRHIRayTracingPipelineState(FRayTracingPipelineState*);
extern RHI_API uint32					    GetRHIRayTracingPipelineStateMaxLocalBindingDataSize(FRayTracingPipelineState*);

//UE_DEPRECATED(5.8, "Identifying compute pipelines specifying only a shader is deprecated, use a FComputePipelineStateInitializer.")
inline FComputePipelineState* FindComputePipelineState(FRHIComputeShader* ComputeShader, bool bVerifyUse = true)
{
	return FindComputePipelineState(FComputePipelineStateInitializer(ComputeShader), bVerifyUse);
}

//UE_DEPRECATED(5.8, "Identifying compute pipelines specifying only a shader is deprecated, use a FComputePipelineStateInitializer.")
inline FComputePipelineState* GetComputePipelineState(FRHIComputeCommandList& RHICmdList, FRHIComputeShader* ComputeShader, bool bVerifyUse = true)
{
	return GetComputePipelineState(RHICmdList, FComputePipelineStateInitializer(ComputeShader), bVerifyUse);
}


class FRHIComputeCommandList : public FRHICommandListBase
{
protected:
	void OnBoundShaderChanged(FRHIComputeShader* InBoundComputeShaderRHI)
	{
		PersistentState.BoundComputeShaderRHI = InBoundComputeShaderRHI;
	}

	FRHIComputeCommandList(FRHIGPUMask GPUMask, bool bImmediate)
		: FRHICommandListBase(GPUMask, bImmediate)
	{}

public:
	static inline FRHIComputeCommandList& Get(FRHICommandListBase& RHICmdList)
	{
		return static_cast<FRHIComputeCommandList&>(RHICmdList);
	}

	FRHIComputeCommandList(FRHIGPUMask GPUMask = FRHIGPUMask::All())
		: FRHICommandListBase(GPUMask, false)
	{}

	FRHIComputeCommandList(FRHICommandListBase&& Other)
		: FRHICommandListBase(MoveTemp(Other))
	{}

	template <typename LAMBDA>
	inline void EnqueueLambda(EThreadFence ThreadFence, const TCHAR* LambdaName, LAMBDA&& Lambda)
	{
		if (IsBottomOfPipe())
		{
			Lambda(*this);
		}
		else
		{
			ALLOC_COMMAND(TRHILambdaCommand<FRHIComputeCommandList, LAMBDA>)(Forward<LAMBDA>(Lambda), LambdaName);

			if (ThreadFence == EThreadFence::Enabled)
			{
				RHIThreadFence(true);
			}
		}
	}

	// Same as EnqueueLambda, but skips the Insights marker surrounding the lambda. Used by the RHI breadcrumb system.
	template <typename LAMBDA>
	inline void EnqueueLambda_NoMarker(LAMBDA&& Lambda)
	{
		if (IsBottomOfPipe())
		{
			Lambda(*this);
		}
		else
		{
			ALLOC_COMMAND(TRHILambdaCommand_NoMarker<FRHIComputeCommandList, LAMBDA>)(Forward<LAMBDA>(Lambda));
		}
	}

	template <typename LAMBDA>
	inline void EnqueueLambda(const TCHAR* LambdaName, LAMBDA&& Lambda)
	{
		FRHIComputeCommandList::EnqueueLambda(EThreadFence::Disabled, LambdaName, Forward<LAMBDA>(Lambda));
	}

	template <typename LAMBDA>
	inline void EnqueueLambda(LAMBDA&& Lambda)
	{
		FRHIComputeCommandList::EnqueueLambda(TEXT("TRHILambdaCommand"), Forward<LAMBDA>(Lambda));
	}

	inline FRHIComputeShader* GetBoundComputeShader() const { return PersistentState.BoundComputeShaderRHI; }

	inline void SetStaticUniformBuffers(const FUniformBufferStaticBindings& UniformBuffers)
	{
		if (Bypass())
		{
			GetComputeContext().RHISetStaticUniformBuffers(UniformBuffers);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetStaticUniformBuffers)(UniformBuffers);
	}

	inline void SetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* Buffer)
	{
		if (Bypass())
		{
			GetComputeContext().RHISetStaticUniformBuffer(Slot, Buffer);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetStaticUniformBuffer)(Slot, Buffer);
	}

	inline void SetUniformBufferDynamicOffset(FUniformBufferStaticSlot Slot, uint32 Offset)
	{
		if (Bypass())
		{
			GetContext().RHISetUniformBufferDynamicOffset(Slot, Offset);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetUniformBufferDynamicOffset)(Slot, Offset);
	}

	inline void SetShaderParameters(
		FRHIComputeShader* InShader
		, TConstArrayView<uint8> InParametersData
		, TConstArrayView<FRHIShaderParameter> InParameters
		, TConstArrayView<FRHIShaderParameterResource> InResourceParameters
		, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters
	)
	{
		ValidateBoundShader(InShader);

		if (Bypass())
		{
			GetComputeContext().RHISetShaderParameters(InShader, InParametersData, InParameters, InResourceParameters, InBindlessParameters);
			return;
		}

		ALLOC_COMMAND(FRHICommandSetShaderParameters<FRHIComputeShader>)(
			InShader
			, AllocArray(InParametersData)
			, AllocArray(InParameters)
			, AllocArray(InResourceParameters)
			, AllocArray(InBindlessParameters)
		);
	}

	inline void SetBatchedShaderParameters(FRHIComputeShader* InShader, FRHIBatchedShaderParameters& InBatchedParameters)
	{
		if (InBatchedParameters.HasParameters())
		{
			ON_SCOPE_EXIT
			{
				InBatchedParameters.Reset();
			};

			if (Bypass())
			{
				GetComputeContext().RHISetShaderParameters(InShader, InBatchedParameters.ParametersData, InBatchedParameters.Parameters, InBatchedParameters.ResourceParameters, InBatchedParameters.BindlessParameters);
				return;
			}

			ValidateBoundShader(InShader);
			ValidateShaderParameters(InBatchedParameters);
			ALLOC_COMMAND(FRHICommandSetShaderParameters<FRHIComputeShader>)(InShader, InBatchedParameters.ParametersData, InBatchedParameters.Parameters, InBatchedParameters.ResourceParameters, InBatchedParameters.BindlessParameters);
		}
	}

	inline void SetShaderUnbinds(FRHIComputeShader* InShader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
	{
		if (NeedsShaderUnbinds())
		{
			ValidateBoundShader(InShader);

			if (Bypass())
			{
				GetComputeContext().RHISetShaderUnbinds(InShader, InUnbinds);
				return;
			}

			ALLOC_COMMAND(FRHICommandSetShaderUnbinds<FRHIComputeShader>)(InShader, AllocArray(InUnbinds));
		}
	}

	inline void SetBatchedShaderUnbinds(FRHIComputeShader* InShader, FRHIBatchedShaderUnbinds& InBatchedUnbinds)
	{
		if (InBatchedUnbinds.HasParameters())
		{
			SetShaderUnbinds(InShader, InBatchedUnbinds.Unbinds);

			InBatchedUnbinds.Reset();
		}
	}

	inline void SetComputePipelineState(FComputePipelineState* ComputePipelineState, FRHIComputeShader* ComputeShader)
	{
		OnBoundShaderChanged(ComputeShader);
		if (Bypass())
		{
			FRHIComputePipelineState* RHIComputePipelineState = ExecuteSetComputePipelineState(ComputePipelineState);
			GetComputeContext().RHISetComputePipelineState(RHIComputePipelineState);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetComputePipelineState)(ComputePipelineState);
	}

	inline void SetAsyncComputeBudget(EAsyncComputeBudget Budget)
	{
		if (Bypass())
		{
			GetComputeContext().RHISetAsyncComputeBudget(Budget);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetAsyncComputeBudget)(Budget);
	}

	inline void SetComputeBudget(ESyncComputeBudget Budget)
	{
		if (Bypass())
		{
			GetComputeContext().RHISetComputeBudget(Budget);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetComputeBudget)(Budget);
	}

	inline void DispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
	{
		if (Bypass())
		{
			GetComputeContext().RHIDispatchComputeShader(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
			return;
		}
		ALLOC_COMMAND(FRHICommandDispatchComputeShader)(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
	}

	inline void DispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset)
	{
		if (Bypass())
		{
			GetComputeContext().RHIDispatchIndirectComputeShader(ArgumentBuffer, ArgumentOffset);
			return;
		}
		ALLOC_COMMAND(FRHICommandDispatchIndirectComputeShader)(ArgumentBuffer, ArgumentOffset);
	}

	inline void ClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values)
	{
		if (Bypass())
		{
			GetComputeContext().RHIClearUAVFloat(UnorderedAccessViewRHI, Values);
			return;
		}
		ALLOC_COMMAND(FRHICommandClearUAVFloat)(UnorderedAccessViewRHI, Values);
	}

	inline void ClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values)
	{
		if (Bypass())
		{
			GetComputeContext().RHIClearUAVUint(UnorderedAccessViewRHI, Values);
			return;
		}
		ALLOC_COMMAND(FRHICommandClearUAVUint)(UnorderedAccessViewRHI, Values);
	}

	inline void BeginTransitions(TArrayView<const FRHITransition*> Transitions)
	{
		if (Bypass())
		{
			GetComputeContext().RHIBeginTransitions(Transitions);

			for (const FRHITransition* Transition : Transitions)
			{
				Transition->MarkBegin(GetPipeline());
			}
		}
		else
		{
			// Copy the transition array into the command list
			FRHITransition** DstTransitionArray = (FRHITransition**)Alloc(sizeof(FRHITransition*) * Transitions.Num(), alignof(FRHITransition*));
			FMemory::Memcpy(DstTransitionArray, Transitions.GetData(), sizeof(FRHITransition*) * Transitions.Num());
			ALLOC_COMMAND(FRHICommandBeginTransitions)(MakeArrayView((const FRHITransition**)DstTransitionArray, Transitions.Num()));
		}
	}

	inline void EndTransitions(TArrayView<const FRHITransition*> Transitions)
	{
		if (Bypass())
		{
			GetComputeContext().RHIEndTransitions(Transitions);

			for (const FRHITransition* Transition : Transitions)
			{
				Transition->MarkEnd(GetPipeline());
			}
		}
		else
		{
			// Copy the transition array into the command list
			FRHITransition** DstTransitionArray = (FRHITransition**)Alloc(sizeof(FRHITransition*) * Transitions.Num(), alignof(FRHITransition*));
			FMemory::Memcpy(DstTransitionArray, Transitions.GetData(), sizeof(FRHITransition*) * Transitions.Num());

			ALLOC_COMMAND(FRHICommandEndTransitions)(MakeArrayView((const FRHITransition**)DstTransitionArray, Transitions.Num()));
		}
	}

	RHI_API void Transition(TArrayView<const FRHITransitionInfo> Infos, ERHITransitionCreateFlags CreateFlags = ERHITransitionCreateFlags::None);

	inline void BeginTransition(const FRHITransition* Transition)
	{
		BeginTransitions(MakeArrayView(&Transition, 1));
	}

	inline void EndTransition(const FRHITransition* Transition)
	{
		EndTransitions(MakeArrayView(&Transition, 1));
	}

	inline void Transition(const FRHITransitionInfo& Info, ERHITransitionCreateFlags CreateFlags = ERHITransitionCreateFlags::None)
	{
		Transition(MakeArrayView(&Info, 1), CreateFlags);
	}

	//
	// Performs an immediate transition with the option of broadcasting to multiple pipelines.
	// Uses both the immediate and async compute contexts. Falls back to graphics-only if async compute is not supported.
	//
	RHI_API void Transition(TArrayView<const FRHITransitionInfo> Infos, ERHIPipeline SrcPipelines, ERHIPipeline DstPipelines, ERHITransitionCreateFlags TransitionCreateFlags = ERHITransitionCreateFlags::None);

	inline void SetTrackedAccess(TArrayView<const FRHITrackedAccessInfo> Infos)
	{
		if (Bypass())
		{
			for (const FRHITrackedAccessInfo& Info : Infos)
			{
				GetComputeContext().SetTrackedAccess(Info);
			}
		}
		else
		{
			ALLOC_COMMAND(FRHICommandSetTrackedAccess)(AllocArray(Infos));
			RHIThreadFence(true);
		}
	}

	inline void SetTrackedAccess(TArrayView<const FRHITransitionInfo> Infos, ERHIPipeline PipelinesAfter)
	{
		FRHITrackedAccessInfo* TrackedAccessInfos = reinterpret_cast<FRHITrackedAccessInfo*>(Alloc(sizeof(FRHITrackedAccessInfo) * Infos.Num(), alignof(FRHITrackedAccessInfo)));
		int32 NumTrackedAccessInfos = 0;

		for (const FRHITransitionInfo& Info : Infos)
		{
			ensureMsgf(Info.IsWholeResource(), TEXT("The Transition method only supports whole resource transitions."));

			if (FRHIViewableResource* Resource = GetViewableResource(Info))
			{
				new (&TrackedAccessInfos[NumTrackedAccessInfos++]) FRHITrackedAccessInfo(Resource, Info.AccessAfter, PipelinesAfter);
			}
		}

		if (NumTrackedAccessInfos > 0)
		{
			SetTrackedAccess(TArrayView<FRHITrackedAccessInfo>(TrackedAccessInfos, NumTrackedAccessInfos));
		}
	}

	inline void SetShaderRootConstants(const FUint32Vector4& Constants)
	{
		if (Bypass())
		{
			GetContext().RHISetShaderRootConstants(Constants);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetShaderRootConstants)(Constants);
	}

	inline void SetComputeShaderRootConstants(const FUint32Vector4& Constants)
	{
		if (Bypass())
		{
			GetComputeContext().RHISetShaderRootConstants(Constants);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetShaderRootConstants)(Constants);
	}

	inline void DispatchComputeShaderBundle(
		FRHIShaderBundle* ShaderBundle,
		FRHIBuffer* RecordArgBuffer,
		TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches,
		ERHIShaderBundleMethod Method
	)
	{
		bUsesShaderBundles = true;

		if (Bypass())
		{
			GetContext().RHIDispatchComputeShaderBundle(ShaderBundle, RecordArgBuffer, SharedBindlessParameters, Dispatches, Method);
			return;
		}
		ValidateShaderBundleComputeDispatch(Dispatches);
		ALLOC_COMMAND(FRHICommandDispatchComputeShaderBundle)(ShaderBundle, RecordArgBuffer, SharedBindlessParameters, Dispatches, Method);
	}

	inline void DispatchComputeShaderBundle(
		TFunction<void(FRHICommandDispatchComputeShaderBundle&)>&& RecordCallback
	)
	{
		bUsesShaderBundles = true;

		// Need to explicitly enqueue the RHI command so we can avoid an unnecessary copy of the dispatches array.
		if (Bypass())
		{
			FRHICommandDispatchComputeShaderBundle DispatchBundleCommand;
			RecordCallback(DispatchBundleCommand);
			DispatchBundleCommand.Execute(*this);
		}
		else
		{
			FRHICommandDispatchComputeShaderBundle& DispatchBundleCommand = *ALLOC_COMMAND_CL(*this, FRHICommandDispatchComputeShaderBundle);
			RecordCallback(DispatchBundleCommand);
			ValidateShaderBundleComputeDispatch(DispatchBundleCommand.Dispatches);
		}
	}

	inline void DispatchGraphicsShaderBundle(
		FRHIShaderBundle* ShaderBundle,
		FRHIBuffer* RecordArgBuffer,
		const FRHIShaderBundleGraphicsState& BundleState,
		TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleGraphicsDispatch> Dispatches,
		ERHIShaderBundleMethod Method
	)
	{
		bUsesShaderBundles = true;

		if (Bypass())
		{
			GetContext().RHIDispatchGraphicsShaderBundle(ShaderBundle, RecordArgBuffer, BundleState, SharedBindlessParameters, Dispatches, Method);
			return;
		}
		ALLOC_COMMAND(FRHICommandDispatchGraphicsShaderBundle)(ShaderBundle, RecordArgBuffer, BundleState, SharedBindlessParameters, Dispatches, Method);
	}

	inline void DispatchGraphicsShaderBundle(
		TFunction<void(FRHICommandDispatchGraphicsShaderBundle&)>&& RecordCallback
	)
	{
		bUsesShaderBundles = true;

		// Need to explicitly enqueue the RHI command so we can avoid an unnecessary copy of the dispatches array.
		if (Bypass())
		{
			FRHICommandDispatchGraphicsShaderBundle DispatchBundleCommand;
			RecordCallback(DispatchBundleCommand);
			DispatchBundleCommand.Execute(*this);
		}
		else
		{
			FRHICommandDispatchGraphicsShaderBundle& DispatchBundleCommand = *ALLOC_COMMAND_CL(*this, FRHICommandDispatchGraphicsShaderBundle);
			RecordCallback(DispatchBundleCommand);
		}
	}

	inline void BeginUAVOverlap()
	{
	#if ENABLE_RHI_VALIDATION
		if (Bypass())
		{
			GetComputeContext().RHIBeginUAVOverlap();
			return;
		}
		ALLOC_COMMAND(FRHICommandBeginUAVOverlap)();
	#endif
	}

	inline void EndUAVOverlap()
	{
	#if ENABLE_RHI_VALIDATION
		if (Bypass())
		{
			GetComputeContext().RHIEndUAVOverlap();
			return;
		}
		ALLOC_COMMAND(FRHICommandEndUAVOverlap)();
	#endif
	}

	inline void BeginUAVOverlap(FRHIUnorderedAccessView* UAV)
	{
	#if ENABLE_RHI_VALIDATION
		FRHIUnorderedAccessView* UAVs[1] = { UAV };
		BeginUAVOverlap(MakeArrayView(UAVs, 1));
	#endif
	}

	inline void EndUAVOverlap(FRHIUnorderedAccessView* UAV)
	{
	#if ENABLE_RHI_VALIDATION
		FRHIUnorderedAccessView* UAVs[1] = { UAV };
		EndUAVOverlap(MakeArrayView(UAVs, 1));
	#endif
	}

	inline void BeginUAVOverlap(TArrayView<FRHIUnorderedAccessView* const> UAVs)
	{
	#if ENABLE_RHI_VALIDATION
		if (Bypass())
		{
			GetComputeContext().RHIBeginUAVOverlap(UAVs);
			return;
		}

		const uint32 AllocSize = UAVs.Num() * sizeof(FRHIUnorderedAccessView*);
		FRHIUnorderedAccessView** InlineUAVs = (FRHIUnorderedAccessView**)Alloc(AllocSize, alignof(FRHIUnorderedAccessView*));
		FMemory::Memcpy(InlineUAVs, UAVs.GetData(), AllocSize);
		ALLOC_COMMAND(FRHICommandBeginSpecificUAVOverlap)(MakeArrayView(InlineUAVs, UAVs.Num()));
	#endif
	}

	inline void EndUAVOverlap(TArrayView<FRHIUnorderedAccessView* const> UAVs)
	{
	#if ENABLE_RHI_VALIDATION
		if (Bypass())
		{
			GetComputeContext().RHIEndUAVOverlap(UAVs);
			return;
		}

		const uint32 AllocSize = UAVs.Num() * sizeof(FRHIUnorderedAccessView*);
		FRHIUnorderedAccessView** InlineUAVs = (FRHIUnorderedAccessView**)Alloc(AllocSize, alignof(FRHIUnorderedAccessView*));
		FMemory::Memcpy(InlineUAVs, UAVs.GetData(), AllocSize);
		ALLOC_COMMAND(FRHICommandEndSpecificUAVOverlap)(MakeArrayView(InlineUAVs, UAVs.Num()));
	#endif
	}

#if WITH_RHI_BREADCRUMBS
	inline FRHIBreadcrumbAllocator& GetBreadcrumbAllocator()
	{
		if (!BreadcrumbAllocator.IsValid())
		{
			BreadcrumbAllocator = MakeShared<FRHIBreadcrumbAllocator>();
			BreadcrumbAllocatorRefs.Add(BreadcrumbAllocator.ToSharedRef()); // Not AddUnique because we know this is a new instance.
		}

		return *BreadcrumbAllocator;
	}

	inline void BeginBreadcrumbCPU(FRHIBreadcrumbNode* Breadcrumb, bool bLink)
	{
		check(Breadcrumb && Breadcrumb != FRHIBreadcrumbNode::Sentinel);
		BreadcrumbAllocatorRefs.AddUnique(Breadcrumb->Allocator);

		if (IsTopOfPipe())
		{
			// Recording thread
			Breadcrumb->TraceBeginCPU();
			PersistentState.LocalBreadcrumb = Breadcrumb;

			if (bLink)
			{
				CPUBreadcrumbState.Current = Breadcrumb;

				if (Breadcrumb->GetParent() == FRHIBreadcrumbNode::Sentinel)
				{
					CPUBreadcrumbState.UnknownParentList.Append(Breadcrumb);
				}
			}
		}

		EnqueueLambda_NoMarker([Breadcrumb, bLink](FRHICommandListBase& ExecutingCmdList)
		{
			// Translating thread
			ExecutingCmdList.PersistentState.LocalBreadcrumb = Breadcrumb;

			if (bLink)
			{
				ExecutingCmdList.CPUBreadcrumbState.Current = Breadcrumb;
				Breadcrumb->TraceBeginCPU();
			}
		});
	}

	inline void EndBreadcrumbCPU(FRHIBreadcrumbNode* Breadcrumb, bool bLink)
	{
		check(Breadcrumb && Breadcrumb != FRHIBreadcrumbNode::Sentinel);
		BreadcrumbAllocatorRefs.AddUnique(Breadcrumb->Allocator);

		if (IsTopOfPipe())
		{
			// Recording thread
			Breadcrumb->TraceEndCPU();
			PersistentState.LocalBreadcrumb = Breadcrumb->GetParent();

			if (bLink)
			{
				CPUBreadcrumbState.Current = Breadcrumb->GetParent();
			}
		}

		EnqueueLambda_NoMarker([Breadcrumb, bLink](FRHICommandListBase& ExecutingCmdList)
		{
			// Translating thread
			ExecutingCmdList.PersistentState.LocalBreadcrumb = Breadcrumb->GetParent();
			check(ExecutingCmdList.PersistentState.LocalBreadcrumb != FRHIBreadcrumbNode::Sentinel);

			if (bLink)
			{
				ExecutingCmdList.CPUBreadcrumbState.Current = Breadcrumb->GetParent();
				check(ExecutingCmdList.CPUBreadcrumbState.Current != FRHIBreadcrumbNode::Sentinel);

				Breadcrumb->TraceEndCPU();
			}
		});
	}

	inline void BeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb, ERHIPipeline Pipeline)
	{
		check(Breadcrumb && Breadcrumb != FRHIBreadcrumbNode::Sentinel);
		check(IsSingleRHIPipeline(Pipeline));
		check(EnumHasAllFlags(ActivePipelines, Pipeline));
#if DO_CHECK
		check(!EnumHasAnyFlags(ERHIPipeline(Breadcrumb->BeginPipes.fetch_or(std::underlying_type_t<ERHIPipeline>(Pipeline))), Pipeline));
#endif

		BreadcrumbAllocatorRefs.AddUnique(Breadcrumb->Allocator);

		auto& State = GPUBreadcrumbState[Pipeline];
		State.Current = Breadcrumb;
		State.Latest = Breadcrumb;

		EnqueueLambda(TEXT("BeginBreadcrumbGPU"), [Breadcrumb, Pipeline](FRHICommandListBase& ExecutingCmdList)
		{
			auto& State = ExecutingCmdList.GPUBreadcrumbState[Pipeline];

			State.Range.InsertAfter(Breadcrumb, State.Prev, Pipeline);
			State.Prev = Breadcrumb;

			State.Current = Breadcrumb;
			State.Latest = Breadcrumb;

			ExecutingCmdList.Contexts[Pipeline]->RHIBeginBreadcrumbGPU(Breadcrumb);
		});
	}

	inline void EndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb, ERHIPipeline Pipeline)
	{
		check(Breadcrumb && Breadcrumb != FRHIBreadcrumbNode::Sentinel);
		check(IsSingleRHIPipeline(Pipeline));
		check(EnumHasAllFlags(ActivePipelines, Pipeline));
#if DO_CHECK
		check(!EnumHasAnyFlags(ERHIPipeline(Breadcrumb->EndPipes.fetch_or(std::underlying_type_t<ERHIPipeline>(Pipeline))), Pipeline));
#endif

		BreadcrumbAllocatorRefs.AddUnique(Breadcrumb->Allocator);

		auto& State = GPUBreadcrumbState[Pipeline];
		State.Current = Breadcrumb->GetParent();
		State.Latest = Breadcrumb->GetParent();

		EnqueueLambda(TEXT("EndBreadcrumbGPU"), [Breadcrumb, Pipeline](FRHICommandListBase& ExecutingCmdList)
		{
			auto& State = ExecutingCmdList.GPUBreadcrumbState[Pipeline];

			State.Current = Breadcrumb->GetParent();
			check(State.Current != FRHIBreadcrumbNode::Sentinel);

			State.Latest = Breadcrumb->GetParent();
			check(State.Latest != FRHIBreadcrumbNode::Sentinel);

			ExecutingCmdList.Contexts[Pipeline]->RHIEndBreadcrumbGPU(Breadcrumb);
		});
	}
#endif // WITH_RHI_BREADCRUMBS

	UE_DEPRECATED(5.8, "SubmitCommandsHint is deprecated, and has no effect if called on a non-immediate RHI command list. Consider calling ImmediateFlush(EImmediateFlushType::DispatchToRHIThread) on the immediate command list instead.")
	inline void SubmitCommandsHint();

	inline void CopyToStagingBuffer(FRHIBuffer* SourceBuffer, FRHIStagingBuffer* DestinationStagingBuffer, uint32 Offset, uint32 NumBytes)
	{
		if (Bypass())
		{
			GetComputeContext().RHICopyToStagingBuffer(SourceBuffer, DestinationStagingBuffer, Offset, NumBytes);
			return;
		}
		ALLOC_COMMAND(FRHICommandCopyToStagingBuffer)(SourceBuffer, DestinationStagingBuffer, Offset, NumBytes);
	}

	inline void WriteGPUFence(FRHIGPUFence* Fence)
	{
		GDynamicRHI->RHIWriteGPUFence_TopOfPipe(*this, Fence);
	}

	inline void SetGPUMask(FRHIGPUMask InGPUMask)
	{
		if (PersistentState.CurrentGPUMask != InGPUMask)
		{
			PersistentState.CurrentGPUMask = InGPUMask;
#if WITH_MGPU
			if (Bypass())
			{
				// Apply the new mask to all contexts owned by this command list.
				for (IRHIComputeContext* Context : Contexts)
				{
					if (Context)
					{
						Context->RHISetGPUMask(PersistentState.CurrentGPUMask);
					}
				}
				return;
			}
			else
			{
				ALLOC_COMMAND(FRHICommandSetGPUMask)(PersistentState.CurrentGPUMask);
			}
#endif // WITH_MGPU
		}
	}

	inline void TransferResources(TConstArrayView<FTransferResourceParams> Params)
	{
#if WITH_MGPU
		FRHIGPUMask PrevGPUMask = GetGPUMask();

		if (NeedsExtraTransitions())
		{
			for (const FTransferResourceParams& Param : Params)
			{
				FRHIGPUMask GPUMaskSrc = FRHIGPUMask::FromIndex(Param.SrcGPUIndex);
				FRHIGPUMask GPUMaskDest = FRHIGPUMask::FromIndex(Param.DestGPUIndex);

				if (Param.Texture)
				{
					SetGPUMask(GPUMaskSrc);
					TransitionInternal(FRHITransitionInfo(Param.Texture.GetReference(), ERHIAccess::Unknown, ERHIAccess::CopySrc, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);

					SetGPUMask(GPUMaskDest);
					TransitionInternal(FRHITransitionInfo(Param.Texture.GetReference(), ERHIAccess::Unknown, ERHIAccess::CopyDest, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);

				}
				else
				{
					SetGPUMask(GPUMaskSrc);
					TransitionInternal(FRHITransitionInfo(Param.Buffer.GetReference(), ERHIAccess::Unknown, ERHIAccess::CopySrc, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);
					SetGPUMask(GPUMaskDest);
					TransitionInternal(FRHITransitionInfo(Param.Buffer.GetReference(), ERHIAccess::Unknown, ERHIAccess::CopyDest, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);
				}
			}
		}

		SetGPUMask(PrevGPUMask);

		if (Bypass())
		{
			GetComputeContext().RHITransferResources(Params);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandTransferResources)(AllocArray(Params));
		}

		if (NeedsExtraTransitions())
		{
			for (const FTransferResourceParams& Param : Params)
			{
				FRHIGPUMask GPUMaskSrc = FRHIGPUMask::FromIndex(Param.SrcGPUIndex);
				FRHIGPUMask GPUMaskDest = FRHIGPUMask::FromIndex(Param.DestGPUIndex);

				if (Param.Texture)
				{
					SetGPUMask(GPUMaskSrc);
					TransitionInternal(FRHITransitionInfo(Param.Texture.GetReference(), ERHIAccess::CopySrc, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);

					SetGPUMask(GPUMaskDest);
					TransitionInternal(FRHITransitionInfo(Param.Texture.GetReference(), ERHIAccess::CopyDest, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);

				}
				else
				{
					SetGPUMask(GPUMaskSrc);
					TransitionInternal(FRHITransitionInfo(Param.Buffer.GetReference(), ERHIAccess::CopySrc, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);

					SetGPUMask(GPUMaskDest);
					TransitionInternal(FRHITransitionInfo(Param.Buffer.GetReference(), ERHIAccess::CopyDest, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);
				}
			}
		}

		SetGPUMask(PrevGPUMask);

#endif // WITH_MGPU
	}

	inline void TransferResourceSignal(TConstArrayView<FTransferResourceFenceData*> FenceDatas, FRHIGPUMask SrcGPUMask)
	{
#if WITH_MGPU
		if (Bypass())
		{
			GetComputeContext().RHITransferResourceSignal(FenceDatas, SrcGPUMask);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandTransferResourceSignal)(AllocArray(FenceDatas), SrcGPUMask);
		}
#endif // WITH_MGPU
	}

	inline void TransferResourceWait(TConstArrayView<FTransferResourceFenceData*> FenceDatas)
	{
#if WITH_MGPU
		if (Bypass())
		{
			GetComputeContext().RHITransferResourceWait(FenceDatas);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandTransferResourceWait)(AllocArray(FenceDatas));
		}
#endif // WITH_MGPU
	}

	inline void CrossGPUTransfer(TConstArrayView<FTransferResourceParams> Params, TConstArrayView<FCrossGPUTransferFence*> PreTransfer, TConstArrayView<FCrossGPUTransferFence*> PostTransfer)
	{
#if WITH_MGPU
		FRHIGPUMask PrevGPUMask = GetGPUMask();

		if (NeedsExtraTransitions())
		{
			for (const FTransferResourceParams& Param : Params)
			{
				FRHIGPUMask GPUMask = FRHIGPUMask::FromIndex(Param.SrcGPUIndex);
				SetGPUMask(GPUMask);
				
				if (Param.Texture)
				{
					
					TransitionInternal(FRHITransitionInfo(Param.Texture.GetReference(), ERHIAccess::Unknown, ERHIAccess::CopySrc, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);
				}
				else
				{
					TransitionInternal(FRHITransitionInfo(Param.Buffer.GetReference(), ERHIAccess::Unknown, ERHIAccess::CopySrc, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);
				}
			}
		}

		SetGPUMask(PrevGPUMask);

		if (Bypass())
		{
			GetComputeContext().RHICrossGPUTransfer(Params, PreTransfer, PostTransfer);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandCrossGPUTransfer)(AllocArray(Params), AllocArray(PreTransfer), AllocArray(PostTransfer));
		}

		if (NeedsExtraTransitions())
		{
			for (const FTransferResourceParams& Param : Params)
			{
				FRHIGPUMask GPUMask = FRHIGPUMask::FromIndex(Param.SrcGPUIndex);
				SetGPUMask(GPUMask);

				if (Param.Texture)
				{
					TransitionInternal(FRHITransitionInfo(Param.Texture.GetReference(), ERHIAccess::CopySrc, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);
				}
				else
				{
					TransitionInternal(FRHITransitionInfo(Param.Buffer.GetReference(), ERHIAccess::CopySrc, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState),
						ERHITransitionCreateFlags::AllowDuringRenderPass);
				}
			}
		}

		SetGPUMask(PrevGPUMask);

#endif // WITH_MGPU
	}

	inline void CrossGPUTransferSignal(TConstArrayView<FTransferResourceParams> Params, TConstArrayView<FCrossGPUTransferFence*> PreTransfer)
	{
#if WITH_MGPU

		FRHIGPUMask PrevGPUMask = GetGPUMask();

		if (NeedsExtraTransitions())
		{
			for (const FTransferResourceParams& Param : Params)
			{
				FRHIGPUMask GPUMask = FRHIGPUMask::FromIndex(Param.DestGPUIndex);
				SetGPUMask(GPUMask);

				if (Param.Texture)
				{
					TransitionInternal(FRHITransitionInfo(Param.Texture.GetReference(), ERHIAccess::Unknown, ERHIAccess::CopyDest, EResourceTransitionFlags::IgnoreAfterState), 
						ERHITransitionCreateFlags::AllowDuringRenderPass);
				}
				else
				{
					TransitionInternal(FRHITransitionInfo(Param.Buffer.GetReference(), ERHIAccess::Unknown, ERHIAccess::CopyDest, EResourceTransitionFlags::IgnoreAfterState), 
						ERHITransitionCreateFlags::AllowDuringRenderPass);
				}
			}
		}

		SetGPUMask(PrevGPUMask);
		
		if (Bypass())
		{
			GetComputeContext().RHICrossGPUTransferSignal(Params, PreTransfer);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandCrossGPUTransferSignal)(AllocArray(Params), AllocArray(PreTransfer));
		}

		if (NeedsExtraTransitions())
		{
			for (const FTransferResourceParams& Param : Params)
			{
				FRHIGPUMask GPUMask = FRHIGPUMask::FromIndex(Param.DestGPUIndex);
				SetGPUMask(GPUMask);

				if (Param.Texture)
				{
					TransitionInternal(FRHITransitionInfo(Param.Texture.GetReference(), ERHIAccess::CopyDest, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState), 
						ERHITransitionCreateFlags::AllowDuringRenderPass);
				}
				else
				{
					TransitionInternal(FRHITransitionInfo(Param.Buffer.GetReference(), ERHIAccess::CopyDest, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState), 
						ERHITransitionCreateFlags::AllowDuringRenderPass);
				}
			}
		}

		SetGPUMask(PrevGPUMask);
#endif // WITH_MGPU
	}

	inline void CrossGPUTransferWait(TConstArrayView<FCrossGPUTransferFence*> SyncPoints)
	{
#if WITH_MGPU
		if (Bypass())
		{
			GetComputeContext().RHICrossGPUTransferWait(SyncPoints);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandCrossGPUTransferWait)(AllocArray(SyncPoints));
		}
#endif // WITH_MGPU
	}
	
	inline void RayTraceDispatch(FRayTracingPipelineState* Pipeline, FRHIRayTracingShader* RayGenShader, FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings, uint32 Width, uint32 Height)
	{
		check(SBT != nullptr);
		if (Bypass())
		{
			GetComputeContext().RHIRayTraceDispatch(GetRHIRayTracingPipelineState(Pipeline), RayGenShader, SBT, GlobalResourceBindings, Width, Height);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandRayTraceDispatch)(Pipeline, RayGenShader, SBT, GlobalResourceBindings, Width, Height);
		}
	}

	/*
	* Compatibility adaptor that operates on the new FRHIBatchedShaderParameters instead of legacy FRayTracingShaderBindings (planned for deprecation).
	* This will become the default native code path in a future UE version.
	*/
	RHI_API void RayTraceDispatch(FRayTracingPipelineState* Pipeline, FRHIRayTracingShader* RayGenShader, FRHIShaderBindingTable* SBT, FRHIBatchedShaderParameters& GlobalResourceBindings, uint32 Width, uint32 Height);

	inline void RayTraceDispatchIndirect(FRayTracingPipelineState* Pipeline, FRHIRayTracingShader* RayGenShader, FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset)
	{
		check(SBT != nullptr);
		if (Bypass())
		{
			GetComputeContext().RHIRayTraceDispatchIndirect(GetRHIRayTracingPipelineState(Pipeline), RayGenShader, SBT, GlobalResourceBindings, ArgumentBuffer, ArgumentOffset);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandRayTraceDispatch)(Pipeline, RayGenShader, SBT, GlobalResourceBindings, ArgumentBuffer, ArgumentOffset);
		}
	}

	/*
	* Compatibility adaptor that operates on the new FRHIBatchedShaderParameters instead of legacy FRayTracingShaderBindings (planned for deprecation).
	* This will become the default native code path in a future UE version.
	*/
	RHI_API void RayTraceDispatchIndirect(FRayTracingPipelineState* Pipeline, FRHIRayTracingShader* RayGenShader, FRHIShaderBindingTable* SBT, FRHIBatchedShaderParameters& GlobalResourceBindings, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset);

	FORCEINLINE_DEBUGGABLE void ExecuteMultiIndirectClusterOperation(const FRayTracingClusterOperationParams& Params)
	{
		if (Bypass())
		{
			GetComputeContext().RHIExecuteMultiIndirectClusterOperation(Params);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandExecuteMultiIndirectClusterOperation)(Params);
			RHIThreadFence(true);
		}
	}

	RHI_API void BuildAccelerationStructure(FRHIRayTracingGeometry* Geometry);
	RHI_API void BuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params);

	inline void BuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange)
	{
		if (Bypass())
		{
			GetComputeContext().RHIBuildAccelerationStructures(Params, ScratchBufferRange);
		}
		else
		{
			// Copy the params themselves as well their segment lists, if there are any.
			// AllocArray() can't be used here directly, as we have to modify the params after copy.
			size_t DataSize = sizeof(FRayTracingGeometryBuildParams) * Params.Num();
			FRayTracingGeometryBuildParams* InlineParams = (FRayTracingGeometryBuildParams*) Alloc(DataSize, alignof(FRayTracingGeometryBuildParams));
			FMemory::Memcpy(InlineParams, Params.GetData(), DataSize);
			for (int32 i=0; i<Params.Num(); ++i)
			{
				if (Params[i].Segments.Num())
				{
					InlineParams[i].Segments = AllocArray(Params[i].Segments);
				}
			}
			ALLOC_COMMAND(FRHICommandBuildAccelerationStructures)(MakeArrayView(InlineParams, Params.Num()), ScratchBufferRange);

			RHIThreadFence(true);
		}
	}

	inline void BuildAccelerationStructure(const FRayTracingSceneBuildParams& SceneBuildParams)
	{
		BuildAccelerationStructures(MakeConstArrayView(&SceneBuildParams, 1));
	}

	inline void BuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params)
	{
		if (Bypass())
		{
			GetComputeContext().RHIBuildAccelerationStructures(Params);
		}
		else
		{
			// Copy the params themselves as well their ReferencedGeometries/ReferencedBuffers, if there are any.
			// AllocArray() can't be used here directly, as we have to modify the params after copy.
			size_t DataSize = sizeof(FRayTracingSceneBuildParams) * Params.Num();
			FRayTracingSceneBuildParams* InlineParams = (FRayTracingSceneBuildParams*)Alloc(DataSize, alignof(FRayTracingSceneBuildParams));
			FMemory::Memcpy(InlineParams, Params.GetData(), DataSize);
			for (int32 Index = 0; Index < Params.Num(); ++Index)
			{
				if (Params[Index].ReferencedGeometries.Num())
				{
					InlineParams[Index].ReferencedGeometries = AllocArray(Params[Index].ReferencedGeometries);
				}

				if (Params[Index].ReferencedBuffers.Num())
				{
					InlineParams[Index].ReferencedBuffers = AllocArray(Params[Index].ReferencedBuffers);
				}
			}

			ALLOC_COMMAND(FRHICommandBuildSceneAccelerationStructures)(MakeConstArrayView(InlineParams, Params.Num()));

			// This RHI command modifies members of the FRHIRayTracingScene inside platform RHI implementations.
			// It therefore needs the RHI lock fence to prevent races on those members.
			RHIThreadFence(true);
		}
	}

	inline void BindAccelerationStructureMemory(FRHIRayTracingScene* Scene, FRHIBuffer* Buffer, uint32 BufferOffset)
	{
		if (Bypass())
		{
			GetComputeContext().RHIBindAccelerationStructureMemory(Scene, Buffer, BufferOffset);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandBindAccelerationStructureMemory)(Scene, Buffer, BufferOffset);

			// This RHI command modifies members of the FRHIRayTracingScene inside platform RHI implementations.
			// It therefore needs the RHI lock fence to prevent races on those members.
			RHIThreadFence(true);
		}
	}

	inline void PostExternalCommandsReset()
	{
		if (Bypass())
		{
			GetContext().RHIPostExternalCommandsReset();
			return;
		}
		ALLOC_COMMAND(FRHICommandPostExternalCommandsReset)();
	}
};

template<> RHI_EXECUTE_API void FRHICommandSetShaderParameters           <FRHIGraphicsShader>::Execute(FRHICommandListBase& CmdList);
template<> RHI_EXECUTE_API void FRHICommandSetShaderUnbinds              <FRHIGraphicsShader>::Execute(FRHICommandListBase& CmdList);

class FRHICommandList : public FRHIComputeCommandList
{
protected:
	using FRHIComputeCommandList::OnBoundShaderChanged;

	void OnBoundShaderChanged(const FBoundShaderStateInput& InBoundShaderStateInput)
	{
		PersistentState.BoundShaderInput = InBoundShaderStateInput;
	}

	FRHICommandList(FRHIGPUMask GPUMask, bool bImmediate)
		: FRHIComputeCommandList(GPUMask, bImmediate)
	{}

public:
	static inline FRHICommandList& Get(FRHICommandListBase& RHICmdList)
	{
		return static_cast<FRHICommandList&>(RHICmdList);
	}

	FRHICommandList(FRHIGPUMask GPUMask = FRHIGPUMask::All())
		: FRHIComputeCommandList(GPUMask)
	{}

	FRHICommandList(FRHICommandListBase&& Other)
		: FRHIComputeCommandList(MoveTemp(Other))
	{}

	inline FRHIVertexShader*        GetBoundVertexShader       () const { return PersistentState.BoundShaderInput.VertexShaderRHI;          }
	inline FRHIMeshShader*          GetBoundMeshShader         () const { return PersistentState.BoundShaderInput.GetMeshShader();          }
	inline FRHIAmplificationShader* GetBoundAmplificationShader() const { return PersistentState.BoundShaderInput.GetAmplificationShader(); }
	inline FRHIPixelShader*         GetBoundPixelShader        () const { return PersistentState.BoundShaderInput.PixelShaderRHI;           }
	inline FRHIGeometryShader*      GetBoundGeometryShader     () const { return PersistentState.BoundShaderInput.GetGeometryShader();      }

	template <typename LAMBDA>
	inline void EnqueueLambda(EThreadFence ThreadFence, const TCHAR* LambdaName, LAMBDA&& Lambda)
	{
		if (IsBottomOfPipe())
		{
			Lambda(*this);
		}
		else
		{
			ALLOC_COMMAND(TRHILambdaCommand<FRHICommandList, LAMBDA>)(Forward<LAMBDA>(Lambda), LambdaName);

			if (ThreadFence == EThreadFence::Enabled)
			{
				RHIThreadFence(true);
			}
		}
	}

	template <typename LAMBDA>
	inline void EnqueueLambda(const TCHAR* LambdaName, LAMBDA&& Lambda)
	{
		FRHICommandList::EnqueueLambda(EThreadFence::Disabled, LambdaName, Forward<LAMBDA>(Lambda));
	}

	template <typename LAMBDA>
	inline void EnqueueLambda(LAMBDA&& Lambda)
	{
		FRHICommandList::EnqueueLambda(TEXT("TRHILambdaCommand"), Forward<LAMBDA>(Lambda));
	}

	using FRHIComputeCommandList::SetShaderParameters;

	inline void SetShaderParameters(
		FRHIGraphicsShader* InShader
		, TConstArrayView<uint8> InParametersData
		, TConstArrayView<FRHIShaderParameter> InParameters
		, TConstArrayView<FRHIShaderParameterResource> InResourceParameters
		, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters
	)
	{
		ValidateBoundShader(InShader);

		if (Bypass())
		{
			GetContext().RHISetShaderParameters(InShader, InParametersData, InParameters, InResourceParameters, InBindlessParameters);
			return;
		}

		ALLOC_COMMAND(FRHICommandSetShaderParameters<FRHIGraphicsShader>)(
			InShader
			, AllocArray(InParametersData)
			, AllocArray(InParameters)
			, AllocArray(InResourceParameters)
			, AllocArray(InBindlessParameters)
			);
	}

	using FRHIComputeCommandList::SetBatchedShaderParameters;

	inline void SetBatchedShaderParameters(FRHIGraphicsShader* InShader, FRHIBatchedShaderParameters& InBatchedParameters)
	{
		if (InBatchedParameters.HasParameters())
		{
			ON_SCOPE_EXIT
			{
				InBatchedParameters.Reset();
			};

			if (Bypass())
			{
				GetContext().RHISetShaderParameters(InShader, InBatchedParameters.ParametersData, InBatchedParameters.Parameters, InBatchedParameters.ResourceParameters, InBatchedParameters.BindlessParameters);
				return;
			}

			ValidateBoundShader(InShader);
			ValidateShaderParameters(InBatchedParameters);
			ALLOC_COMMAND(FRHICommandSetShaderParameters<FRHIGraphicsShader>)(InShader, InBatchedParameters.ParametersData, InBatchedParameters.Parameters, InBatchedParameters.ResourceParameters, InBatchedParameters.BindlessParameters);
		}
	}

	using FRHIComputeCommandList::SetShaderUnbinds;

	inline void SetShaderUnbinds(FRHIGraphicsShader* InShader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
	{
		if (NeedsShaderUnbinds())
		{
			ValidateBoundShader(InShader);

			if (Bypass())
			{
				GetContext().RHISetShaderUnbinds(InShader, InUnbinds);
				return;
			}

			ALLOC_COMMAND(FRHICommandSetShaderUnbinds<FRHIGraphicsShader>)(InShader, AllocArray(InUnbinds));
		}
	}

	using FRHIComputeCommandList::SetBatchedShaderUnbinds;

	inline void SetBatchedShaderUnbinds(FRHIGraphicsShader* InShader, FRHIBatchedShaderUnbinds& InBatchedUnbinds)
	{
		if (InBatchedUnbinds.HasParameters())
		{
			SetShaderUnbinds(InShader, InBatchedUnbinds.Unbinds);

			InBatchedUnbinds.Reset();
		}
	}

	inline void SetBlendFactor(const FLinearColor& BlendFactor = FLinearColor::White)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHISetBlendFactor(BlendFactor);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetBlendFactor)(BlendFactor);
	}

	inline void DrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHIDrawPrimitive(BaseVertexIndex, NumPrimitives, NumInstances);
			return;
		}
		ALLOC_COMMAND(FRHICommandDrawPrimitive)(BaseVertexIndex, NumPrimitives, NumInstances);
	}

	inline void DrawIndexedPrimitive(FRHIBuffer* IndexBuffer, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHIDrawIndexedPrimitive(IndexBuffer, BaseVertexIndex, FirstInstance, NumVertices, StartIndex, NumPrimitives, NumInstances);
			return;
		}
		ALLOC_COMMAND(FRHICommandDrawIndexedPrimitive)(IndexBuffer, BaseVertexIndex, FirstInstance, NumVertices, StartIndex, NumPrimitives, NumInstances);
	}

	inline void SetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBuffer, uint32 Offset)
	{
		if (Bypass())
		{
			GetContext().RHISetStreamSource(StreamIndex, VertexBuffer, Offset);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetStreamSource)(StreamIndex, VertexBuffer, Offset);
	}

	inline void SetStreamSourceSlot(uint32 StreamIndex, FRHIStreamSourceSlot* StreamSourceSlot, uint32 Offset)
	{
		EnqueueLambda([StreamIndex, StreamSourceSlot, Offset] (FRHICommandListBase& RHICmdList)
		{
			FRHICommandSetStreamSource Command(StreamIndex, StreamSourceSlot ? StreamSourceSlot->Buffer : nullptr, Offset);
			Command.Execute(RHICmdList);
		});
	}

	inline void SetStencilRef(uint32 StencilRef)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHISetStencilRef(StencilRef);
			return;
		}

		ALLOC_COMMAND(FRHICommandSetStencilRef)(StencilRef);
	}

	inline void SetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHISetViewport(MinX, MinY, MinZ, MaxX, MaxY, MaxZ);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetViewport)(MinX, MinY, MinZ, MaxX, MaxY, MaxZ);
	}

	inline void SetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHISetStereoViewport(LeftMinX, RightMinX, LeftMinY, RightMinY, MinZ, LeftMaxX, RightMaxX, LeftMaxY, RightMaxY, MaxZ);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetStereoViewport)(LeftMinX, RightMinX, LeftMinY, RightMinY, MinZ, LeftMaxX, RightMaxX, LeftMaxY, RightMaxY, MaxZ);
	}

	inline void SetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHISetScissorRect(bEnable, MinX, MinY, MaxX, MaxY);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetScissorRect)(bEnable, MinX, MinY, MaxX, MaxY);
	}

	void ApplyCachedRenderTargets(
		FGraphicsPipelineStateInitializer& GraphicsPSOInit
		)
	{
		GraphicsPSOInit.RenderTargetsEnabled = PersistentState.CachedNumSimultanousRenderTargets;

		for (uint32 i = 0; i < GraphicsPSOInit.RenderTargetsEnabled; ++i)
		{
			if (PersistentState.CachedRenderTargets[i].Texture)
			{
				GraphicsPSOInit.RenderTargetFormats[i] = UE_PIXELFORMAT_TO_UINT8(PersistentState.CachedRenderTargets[i].Texture->GetFormat());
				GraphicsPSOInit.RenderTargetFlags[i] = PersistentState.CachedRenderTargets[i].Texture->GetFlags();
			}
			else
			{
				GraphicsPSOInit.RenderTargetFormats[i] = PF_Unknown;
			}

			if (GraphicsPSOInit.RenderTargetFormats[i] != PF_Unknown)
			{
				GraphicsPSOInit.NumSamples = static_cast<uint16>(PersistentState.CachedRenderTargets[i].Texture->GetNumSamples());
			}
		}

		if (PersistentState.CachedDepthStencilTarget.Texture)
		{
			GraphicsPSOInit.DepthStencilTargetFormat = PersistentState.CachedDepthStencilTarget.Texture->GetFormat();
			GraphicsPSOInit.DepthStencilTargetFlag = PersistentState.CachedDepthStencilTarget.Texture->GetFlags();
			const FRHITexture* TextureArray = PersistentState.CachedDepthStencilTarget.Texture->GetTexture2DArray();
		}
		else
		{
			GraphicsPSOInit.DepthStencilTargetFormat = PF_Unknown;
		}

		GraphicsPSOInit.DepthTargetLoadAction = PersistentState.CachedDepthStencilTarget.DepthLoadAction;
		GraphicsPSOInit.DepthTargetStoreAction = PersistentState.CachedDepthStencilTarget.DepthStoreAction;
		GraphicsPSOInit.StencilTargetLoadAction = PersistentState.CachedDepthStencilTarget.StencilLoadAction;
		GraphicsPSOInit.StencilTargetStoreAction = PersistentState.CachedDepthStencilTarget.GetStencilStoreAction();
		GraphicsPSOInit.DepthStencilAccess = PersistentState.CachedDepthStencilTarget.GetDepthStencilAccess();

		if (GraphicsPSOInit.DepthStencilTargetFormat != PF_Unknown)
		{
			GraphicsPSOInit.NumSamples =  static_cast<uint16>(PersistentState.CachedDepthStencilTarget.Texture->GetNumSamples());
		}

		GraphicsPSOInit.SubpassHint = PersistentState.SubpassHint;
		GraphicsPSOInit.SubpassIndex = PersistentState.SubpassIndex;
		GraphicsPSOInit.MultiViewCount = PersistentState.MultiViewCount;
		GraphicsPSOInit.bHasFragmentDensityAttachment = PersistentState.bHasFragmentDensityAttachment;
	}

	inline void SetGraphicsPipelineState(class FGraphicsPipelineState* GraphicsPipelineState, const FBoundShaderStateInput& ShaderInput, uint32 StencilRef, bool bApplyAdditionalState)
	{
		//check(IsOutsideRenderPass());
		OnBoundShaderChanged(ShaderInput);
		if (Bypass())
		{
			FRHIGraphicsPipelineState* RHIGraphicsPipelineState = ExecuteSetGraphicsPipelineState(GraphicsPipelineState);
			GetContext().RHISetGraphicsPipelineState(RHIGraphicsPipelineState, StencilRef, bApplyAdditionalState);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetGraphicsPipelineState)(GraphicsPipelineState, StencilRef, bApplyAdditionalState);
	}

#if PLATFORM_USE_FALLBACK_PSO
	inline void SetGraphicsPipelineState(const FGraphicsPipelineStateInitializer& PsoInit, uint32 StencilRef, bool bApplyAdditionalState)
	{
		//check(IsOutsideRenderPass());
		OnBoundShaderChanged(PsoInit.BoundShaderState);
		if (Bypass())
		{
			GetContext().RHISetGraphicsPipelineState(PsoInit, StencilRef, bApplyAdditionalState);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetGraphicsPipelineStateFromInitializer)(PsoInit, StencilRef, bApplyAdditionalState);
	}
#endif

	inline void DrawPrimitiveIndirect(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHIDrawPrimitiveIndirect(ArgumentBuffer, ArgumentOffset);
			return;
		}
		ALLOC_COMMAND(FRHICommandDrawPrimitiveIndirect)(ArgumentBuffer, ArgumentOffset);
	}

	inline void DrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentsBuffer, uint32 ArgumentOffset)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHIDrawIndexedPrimitiveIndirect(IndexBuffer, ArgumentsBuffer, ArgumentOffset);
			return;
		}
		ALLOC_COMMAND(FRHICommandDrawIndexedPrimitiveIndirect)(IndexBuffer, ArgumentsBuffer, ArgumentOffset);
	}

	inline void MultiDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentsBuffer, uint32 ArgumentOffset, FRHIBuffer* CountBuffer, uint32 CountBufferOffset, uint32 MaxDrawArguments)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHIMultiDrawIndexedPrimitiveIndirect(IndexBuffer, ArgumentsBuffer, ArgumentOffset, CountBuffer, CountBufferOffset, MaxDrawArguments);
			return;
		}
		ALLOC_COMMAND(FRHICommandMultiDrawIndexedPrimitiveIndirect)(IndexBuffer, ArgumentsBuffer, ArgumentOffset, CountBuffer, CountBufferOffset, MaxDrawArguments);
	}

	inline void DispatchMeshShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
	{
		if (Bypass())
		{
			GetContext().RHIDispatchMeshShader(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
			return;
		}
		ALLOC_COMMAND(FRHICommandDispatchMeshShader)(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
	}

	inline void DispatchIndirectMeshShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset)
	{
		if (Bypass())
		{
			GetContext().RHIDispatchIndirectMeshShader(ArgumentBuffer, ArgumentOffset);
			return;
		}
		ALLOC_COMMAND(FRHICommandDispatchIndirectMeshShader)(ArgumentBuffer, ArgumentOffset);
	}

	inline void SetDepthBounds(float MinDepth, float MaxDepth)
	{
		//check(IsOutsideRenderPass());
		if (Bypass())
		{
			GetContext().RHISetDepthBounds(MinDepth, MaxDepth);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetDepthBounds)(MinDepth, MaxDepth);
	}
	
	inline void GpuHangCommandListCorruption()
	{
		if (Bypass())
		{
			GetContext().RHIGpuHangCommandListCorruption();
			return;
		}
		ALLOC_COMMAND(FRHIGpuHangCommandListCorruption)();
	}

	inline void SetShadingRate(EVRSShadingRate ShadingRate, EVRSRateCombiner Combiner)
	{
#if PLATFORM_SUPPORTS_VARIABLE_RATE_SHADING
		if (Bypass())
		{
			GetContext().RHISetShadingRate(ShadingRate, Combiner);
			return;
		}
		ALLOC_COMMAND(FRHICommandSetShadingRate)(ShadingRate, Combiner);
#endif
	}

	inline void CopyTexture(FRHITexture* SourceTextureRHI, FRHITexture* DestTextureRHI, const FRHICopyTextureInfo& CopyInfo)
	{
		check(SourceTextureRHI && DestTextureRHI);
		check(SourceTextureRHI != DestTextureRHI);
		check(IsOutsideRenderPass());

		if (Bypass())
		{
			GetContext().RHICopyTexture(SourceTextureRHI, DestTextureRHI, CopyInfo);
			return;
		}
		ALLOC_COMMAND(FRHICommandCopyTexture)(SourceTextureRHI, DestTextureRHI, CopyInfo);
	}

	inline void ResummarizeHTile(FRHITexture* DepthTexture)
	{
		if (Bypass())
		{
			GetContext().RHIResummarizeHTile(DepthTexture);
			return;
		}
		ALLOC_COMMAND(FRHICommandResummarizeHTile)(DepthTexture);
	}

	inline void BeginRenderQuery(FRHIRenderQuery* RenderQuery)
	{
		GDynamicRHI->RHIBeginRenderQuery_TopOfPipe(*this, RenderQuery);
	}

	inline void EndRenderQuery(FRHIRenderQuery* RenderQuery)
	{
		GDynamicRHI->RHIEndRenderQuery_TopOfPipe(*this, RenderQuery);
	}

	inline void BeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* Name)
	{
		check(!IsInsideRenderPass());
		check(!IsInsideComputePass());

		InInfo.Validate();

		if (Bypass())
		{
			GetContext().RHIBeginRenderPass(InInfo, Name);
		}
		else
		{
			// Copy the transition array into the command list
			FRHIRenderPassInfo* InfoCopy = (FRHIRenderPassInfo*)Alloc(sizeof(FRHIRenderPassInfo), alignof(FRHIRenderPassInfo));
			FMemory::Memcpy(InfoCopy, &InInfo, sizeof(InInfo));

			TCHAR* NameCopy  = AllocString(Name);
			ALLOC_COMMAND(FRHICommandBeginRenderPass)(*InfoCopy, NameCopy);
		}

		CacheActiveRenderTargets(InInfo);
		ResetSubpass(InInfo.SubpassHint);
		PersistentState.bInsideRenderPass = true;

		if (InInfo.NumOcclusionQueries)
		{
			PersistentState.bInsideOcclusionQueryBatch = true;
			GDynamicRHI->RHIBeginRenderQueryBatch_TopOfPipe(*this, RQT_Occlusion);
		}
	}

	void EndRenderPass()
	{
		check(IsInsideRenderPass());
		check(!IsInsideComputePass());

		if (PersistentState.bInsideOcclusionQueryBatch)
		{
			GDynamicRHI->RHIEndRenderQueryBatch_TopOfPipe(*this, RQT_Occlusion);
			PersistentState.bInsideOcclusionQueryBatch = false;
		}

		if (Bypass())
		{
			GetContext().RHIEndRenderPass();
		}
		else
		{
			ALLOC_COMMAND(FRHICommandEndRenderPass)();
		}
		PersistentState.bInsideRenderPass = false;
		ResetSubpass(ESubpassHint::None);
	}

	// Takes the array of sub command lists and inserts them logically into a render pass at this point in time.
	void InsertParallelRenderPass(TSharedPtr<FRHIParallelRenderPassInfo> const& InInfo, TArray<FRHISubCommandList*>&& SubCommandLists)
	{
		InsertParallelRenderPass_Base(InInfo, MoveTemp(SubCommandLists));
	}

	inline void NextSubpass()
	{
		check(IsInsideRenderPass());
		if (Bypass())
		{
			GetContext().RHINextSubpass();
		}
		else
		{
			ALLOC_COMMAND(FRHICommandNextSubpass)();
		}
		IncrementSubpass();
	}

	inline void CopyBufferRegion(FRHIBuffer* DestBuffer, uint64 DstOffset, FRHIBuffer* SourceBuffer, uint64 SrcOffset, uint64 NumBytes)
	{
		// No copy/DMA operation inside render passes
		check(IsOutsideRenderPass());

		RHI_BREADCRUMB_CHECK_SHIPPING(*this, SourceBuffer != DestBuffer);
		RHI_BREADCRUMB_CHECK_SHIPPING(*this, DstOffset + NumBytes <= DestBuffer->GetSize());
		RHI_BREADCRUMB_CHECK_SHIPPING(*this, SrcOffset + NumBytes <= SourceBuffer->GetSize());

		if (Bypass())
		{
			GetContext().RHICopyBufferRegion(DestBuffer, DstOffset, SourceBuffer, SrcOffset, NumBytes);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandCopyBufferRegion)(DestBuffer, DstOffset, SourceBuffer, SrcOffset, NumBytes);
		}
	}

	// Ray tracing API

	void CommitShaderBindingTable(FRHIShaderBindingTable* SBT)
	{
		checkf(!EnumHasAnyFlags(SBT->GetInitializer().ShaderBindingMode, ERayTracingShaderBindingMode::Inline), TEXT("Use CommitShaderBindingTable function which also provides the InlineBindingDataBuffer when SBT has inline binding mode set"));		
		CommitShaderBindingTable(SBT, nullptr);
	}
	
	inline void CommitShaderBindingTable(FRHIShaderBindingTable* SBT, FRHIBuffer* InlineBindingDataBuffer)
	{
		if (Bypass())
		{
			GetContext().RHICommitShaderBindingTable(SBT, InlineBindingDataBuffer);
		}
		else
		{
			ALLOC_COMMAND(FRHICommandCommitShaderBindingTable)(SBT, InlineBindingDataBuffer);

			// This RHI command modifies members of the FRHIShaderBindingTable inside platform RHI implementations.
			// It therefore needs the RHI lock fence to prevent races on those members.
			RHIThreadFence(true);
		}
	}

	UE_DEPRECATED(5.8, "ClearShaderBindingTable is deprecated and does nothing. Remove any remaining calls to this function.")
	inline void ClearShaderBindingTable(FRHIShaderBindingTable* SBT)
	{
	}
	
	inline void SetBindingsOnShaderBindingTable(
		FRHIShaderBindingTable* SBT, FRayTracingPipelineState* Pipeline,
		uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings,
		ERayTracingBindingType BindingType,
		bool bCopyDataToInlineStorage = true)
	{
		if (Bypass())
		{
			GetContext().RHISetBindingsOnShaderBindingTable(SBT, GetRHIRayTracingPipelineState(Pipeline), NumBindings, Bindings, BindingType);
		}
		else
		{
			check(GetRHIRayTracingPipelineStateMaxLocalBindingDataSize(Pipeline) <= SBT->GetInitializer().LocalBindingDataSize);

			FRayTracingLocalShaderBindings* InlineBindings = nullptr;

			// By default all batch binding data is stored in the command list memory.
			// However, user may skip this copy if they take responsibility for keeping data alive until this command is executed.
			if (bCopyDataToInlineStorage)
			{
				if (NumBindings)
				{
					uint32 Size = sizeof(FRayTracingLocalShaderBindings) * NumBindings;
					InlineBindings = (FRayTracingLocalShaderBindings*)Alloc(Size, alignof(FRayTracingLocalShaderBindings));
					FMemory::Memcpy(InlineBindings, Bindings, Size);
				}

				for (uint32 i = 0; i < NumBindings; ++i)
				{
					if (InlineBindings[i].NumUniformBuffers)
					{
						InlineBindings[i].UniformBuffers = (FRHIUniformBuffer**)Alloc(sizeof(FRHIUniformBuffer*) * InlineBindings[i].NumUniformBuffers, alignof(FRHIUniformBuffer*));
						for (uint32 Index = 0; Index < InlineBindings[i].NumUniformBuffers; ++Index)
						{
							InlineBindings[i].UniformBuffers[Index] = Bindings[i].UniformBuffers[Index];
						}
					}

					if (InlineBindings[i].LooseParameterDataSize)
					{
						InlineBindings[i].LooseParameterData = (uint8*)Alloc(InlineBindings[i].LooseParameterDataSize, 16);
						FMemory::Memcpy(InlineBindings[i].LooseParameterData, Bindings[i].LooseParameterData, InlineBindings[i].LooseParameterDataSize);
					}
				}

				ALLOC_COMMAND(FRHICommandSetBindingsOnShaderBindingTable)(SBT, Pipeline, NumBindings, InlineBindings, BindingType);
			}
			else
			{
				ALLOC_COMMAND(FRHICommandSetBindingsOnShaderBindingTable)(SBT, Pipeline, NumBindings, Bindings, BindingType);
			}

			RHIThreadFence(true);
		}
	}

	inline void SetRayTracingHitGroups(
		FRHIShaderBindingTable* SBT, FRayTracingPipelineState* Pipeline,
		uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings,
		bool bCopyDataToInlineStorage = true)
	{
		SetBindingsOnShaderBindingTable(SBT, Pipeline, NumBindings, Bindings, ERayTracingBindingType::HitGroup, bCopyDataToInlineStorage);
	}

	inline void SetRayTracingCallableShaders(
		FRHIShaderBindingTable* SBT, FRayTracingPipelineState* Pipeline,
		uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings,
		bool bCopyDataToInlineStorage = true)
	{
		SetBindingsOnShaderBindingTable(SBT, Pipeline, NumBindings, Bindings, ERayTracingBindingType::CallableShader, bCopyDataToInlineStorage);
	}

	inline void SetRayTracingMissShaders(
		FRHIShaderBindingTable* SBT, FRayTracingPipelineState* Pipeline,
		uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings,
		bool bCopyDataToInlineStorage = true)
	{
		SetBindingsOnShaderBindingTable(SBT, Pipeline, NumBindings, Bindings, ERayTracingBindingType::MissShader, bCopyDataToInlineStorage);
	}

	inline void SetRayTracingHitGroup(
		FRHIShaderBindingTable* SBT, uint32 RecordIndex, FRHIRayTracingGeometry* Geometry, uint32 GeometrySegmentIndex,
		FRayTracingPipelineState* Pipeline, uint32 HitGroupIndex,
		uint32 NumUniformBuffers, FRHIUniformBuffer* const* UniformBuffers,
		uint32 LooseParameterDataSize, const void* LooseParameterData,
		uint32 UserData)
	{
		check(NumUniformBuffers <= UINT16_MAX);
		check(LooseParameterDataSize <= UINT16_MAX);

		FRayTracingLocalShaderBindings* InlineBindings = new(Alloc<FRayTracingLocalShaderBindings>()) FRayTracingLocalShaderBindings();
		InlineBindings->RecordIndex = RecordIndex;
		InlineBindings->Geometry = Geometry;
		InlineBindings->SegmentIndex = GeometrySegmentIndex;
		InlineBindings->ShaderIndexInPipeline = HitGroupIndex;
		InlineBindings->UserData = UserData;
		InlineBindings->NumUniformBuffers = (uint16)NumUniformBuffers;
		InlineBindings->LooseParameterDataSize = (uint16)LooseParameterDataSize;

		if (NumUniformBuffers)
		{
			InlineBindings->UniformBuffers = (FRHIUniformBuffer**)Alloc(sizeof(FRHIUniformBuffer*) * NumUniformBuffers, alignof(FRHIUniformBuffer*));
			for (uint32 Index = 0; Index < NumUniformBuffers; ++Index)
			{
				InlineBindings->UniformBuffers[Index] = UniformBuffers[Index];
			}
		}

		if (LooseParameterDataSize)
		{
			InlineBindings->LooseParameterData = (uint8*)Alloc(LooseParameterDataSize, 16);
			FMemory::Memcpy(InlineBindings->LooseParameterData, LooseParameterData, LooseParameterDataSize);
		}

		SetBindingsOnShaderBindingTable(SBT, Pipeline, 1, InlineBindings, ERayTracingBindingType::HitGroup, /*bCopyDataToInlineStorage*/ false);
	}

	inline void SetDefaultRayTracingHitGroup(
		FRHIShaderBindingTable* SBT, FRayTracingPipelineState* Pipeline, uint32 HitGroupIndex)
	{
		FRayTracingLocalShaderBindings* InlineBindings = new(Alloc<FRayTracingLocalShaderBindings>()) FRayTracingLocalShaderBindings();
		InlineBindings->ShaderIndexInPipeline = HitGroupIndex;
		InlineBindings->RecordIndex = 0; //< Default hit group always stored at index 0

		SetBindingsOnShaderBindingTable(SBT, Pipeline, 1, InlineBindings, ERayTracingBindingType::HitGroup, /*bCopyDataToInlineStorage*/ false);
	}

	inline void SetRayTracingCallableShader(
		FRHIShaderBindingTable* SBT, uint32 RecordIndex,
		FRayTracingPipelineState* Pipeline, uint32 ShaderIndexInPipeline,
		uint32 NumUniformBuffers, FRHIUniformBuffer* const* UniformBuffers,
		uint32 UserData)
	{
		FRayTracingLocalShaderBindings* InlineBindings = new(Alloc<FRayTracingLocalShaderBindings>()) FRayTracingLocalShaderBindings();
		InlineBindings->RecordIndex = RecordIndex;
		InlineBindings->ShaderIndexInPipeline = ShaderIndexInPipeline;
		InlineBindings->UserData = UserData;
		InlineBindings->NumUniformBuffers = (uint16)NumUniformBuffers;

		if (NumUniformBuffers)
		{
			InlineBindings->UniformBuffers = (FRHIUniformBuffer**)Alloc(sizeof(FRHIUniformBuffer*) * NumUniformBuffers, alignof(FRHIUniformBuffer*));
			for (uint32 Index = 0; Index < NumUniformBuffers; ++Index)
			{
				InlineBindings->UniformBuffers[Index] = UniformBuffers[Index];
			}
		}

		SetBindingsOnShaderBindingTable(SBT, Pipeline, 1, InlineBindings, ERayTracingBindingType::CallableShader, /*bCopyDataToInlineStorage*/ false);
	}

	inline void SetRayTracingMissShader(
		FRHIShaderBindingTable* SBT, uint32 RecordIndex,
		FRayTracingPipelineState* Pipeline, uint32 ShaderIndexInPipeline,
		uint32 NumUniformBuffers, FRHIUniformBuffer* const* UniformBuffers,
		uint32 UserData)
	{
		FRayTracingLocalShaderBindings* InlineBindings = new(Alloc<FRayTracingLocalShaderBindings>()) FRayTracingLocalShaderBindings();
		InlineBindings->RecordIndex = RecordIndex;
		InlineBindings->ShaderIndexInPipeline = ShaderIndexInPipeline;
		InlineBindings->UserData = UserData;
		InlineBindings->NumUniformBuffers = (uint16)NumUniformBuffers;

		if (NumUniformBuffers)
		{
			checkSlow(UniformBuffers);
			InlineBindings->UniformBuffers = (FRHIUniformBuffer**)Alloc(sizeof(FRHIUniformBuffer*) * NumUniformBuffers, alignof(FRHIUniformBuffer*));
			for (uint32 Index = 0; Index < NumUniformBuffers; ++Index)
			{
				InlineBindings->UniformBuffers[Index] = UniformBuffers[Index];
			}
		}

		SetBindingsOnShaderBindingTable(SBT, Pipeline, 1, InlineBindings, ERayTracingBindingType::MissShader, /*bCopyDataToInlineStorage*/ false);
	}
};

namespace EImmediateFlushType
{
	enum Type
	{ 
		WaitForOutstandingTasksOnly  = 0, 
		DispatchToRHIThread          = 1, 
		FlushRHIThread               = 2,
		FlushRHIThreadFlushResources = 3
	};
};

class FScopedRHIThreadStaller
{
	class FRHICommandListImmediate* Immed; // non-null if we need to unstall
public:
	FScopedRHIThreadStaller() = delete;
	FScopedRHIThreadStaller(class FRHICommandListImmediate& InImmed, bool bDoStall = true);
	~FScopedRHIThreadStaller();
};

extern RHI_API ERHIAccess RHIGetDefaultResourceState(ETextureCreateFlags InUsage, bool bInHasInitialData);
extern RHI_API ERHIAccess RHIGetDefaultResourceState(EBufferUsageFlags InUsage, bool bInHasInitialData);

enum class ERHISubmitFlags
{
	None = 0,

	// All submitted work will be processed, and the resulting platform command lists will be submitted to the GPU.
	SubmitToGPU = 1 << 0,

	// Processes the delete queue until it is empty.
	DeleteResources = 1 << 1,

	// Indicates that the entire RHI thread pipeline will be flushed. 
	// If combined with DeleteResources, the pending deletes queue is processed in a loop until all released resources have been deleted.
	FlushRHIThread = 1 << 2,

	// Marks the end of an engine frame. Causes RHI draw stats etc to be accumulated,
	// and calls RHIEndFrame for platform RHIs to do various cleanup tasks.
	EndFrame = 1 << 3,

	// When combined with EndFrame, triggers a GPU profile on the following frame.
	ProfileNextFrame = 1 << 4,

#if CAN_TOGGLE_COMMAND_LIST_BYPASS
	// Used when toggling RHI command bypass.
	EnableBypass  = 1 << 5,
	DisableBypass = 1 << 6,
#endif

#if WITH_RHI_BREADCRUMBS
	EnableDrawEvents  = 1 << 7,
	DisableDrawEvents = 1 << 8
#endif
};

ENUM_CLASS_FLAGS(ERHISubmitFlags);

class FRHICommandListImmediate : public FRHICommandList
{
	friend class FRHICommandListExecutor;
	friend class FRHICommandListScopedExtendResourceLifetime;

	friend void RHI_API RHIResourceLifetimeReleaseRef(FRHICommandListImmediate&, int32);

	FRHICommandListImmediate()
		: FRHICommandList(FRHIGPUMask::All(), true)
	{
	#if WITH_RHI_BREADCRUMBS
		PersistentState.LocalBreadcrumb = nullptr;
	#endif
	}

	~FRHICommandListImmediate()
	{
		FinishRecording();
	}

public:
	static inline FRHICommandListImmediate& Get();

	static inline FRHICommandListImmediate& Get(FRHICommandListBase& RHICmdList)
	{
		check(RHICmdList.IsImmediate());
		return static_cast<FRHICommandListImmediate&>(RHICmdList);
	}

	UE_DEPRECATED(5.7, "FRHICommandListImmediate::BeginDrawingViewport is deprecated. Calls to this function are unnecessary and can be removed. There is no replacement.")
	inline void BeginDrawingViewport(FRHIViewport* Viewport, FRHITexture* RenderTargetRHI) {}

	UE_DEPRECATED(5.7, "FRHICommandListImmediate::EndDrawingViewport(FRHIViewport* Viewport, bool bPresent, bool bLockToVsync) is deprecated. Use FRHIPresentArgs to pass the parameters needed.")
	inline void EndDrawingViewport(FRHIViewport* Viewport, bool bPresent, bool bLockToVsync)
	{
		EndDrawingViewport(Viewport, FRHIPresentArgs(0, bPresent, bLockToVsync));
	}
	RHI_API void EndDrawingViewport(FRHIViewport* Viewport, const FRHIPresentArgs& InPresentArgs);

	RHI_API void EndFrame();

	struct FQueuedCommandList
	{
		// The command list to enqueue.
		FRHICommandListBase* CmdList = nullptr;

		FQueuedCommandList() = default;
		FQueuedCommandList(FRHICommandListBase* InCmdList)
			: CmdList(InCmdList)
		{}
	};

	enum class ETranslatePriority
	{
		Disabled, // Parallel translate is disabled. Command lists will be replayed by the RHI thread into the default context.
		Normal,   // Parallel translate is enabled, and runs on a normal priority task thread.
		High      // Parallel translate is enabled, and runs on a high priority task thread.
	};

	//
	// Chains together one or more RHI command lists into the immediate command list, allowing in-order submission of parallel rendering work.
	// The provided command lists are not dispatched until FinishRecording() is called on them, and their dispatch prerequisites have been completed.
	//

	// @todo dev-pr : deprecate
	RHI_API void QueueAsyncCommandListSubmit(TArrayView<FQueuedCommandList> CommandLists, ETranslatePriority ParallelTranslatePriority = ETranslatePriority::Disabled, int32 MinDrawsPerTranslate = 0);

	// @todo dev-pr : deprecate
	inline void QueueAsyncCommandListSubmit(FQueuedCommandList QueuedCommandList, ETranslatePriority ParallelTranslatePriority = ETranslatePriority::Disabled, int32 MinDrawsPerTranslate = 0)
	{
		QueueAsyncCommandListSubmit(MakeArrayView(&QueuedCommandList, 1), ParallelTranslatePriority, MinDrawsPerTranslate);
	}

	//
	// Dispatches work to the RHI thread and the GPU.
	// Also optionally waits for its completion on the RHI thread. Does not wait for the GPU.
	//
	RHI_API void ImmediateFlush(EImmediateFlushType::Type FlushType, ERHISubmitFlags SubmitFlags = ERHISubmitFlags::None);

	RHI_API bool StallRHIThread();
	RHI_API void UnStallRHIThread();
	RHI_API static bool IsStalled();

	RHI_API void InitializeImmediateContexts();

	template <typename LAMBDA>
	inline void EnqueueLambda(EThreadFence ThreadFence, const TCHAR* LambdaName, LAMBDA&& Lambda)
	{
		if (IsBottomOfPipe())
		{
			Lambda(*this);
		}
		else
		{
			ALLOC_COMMAND(TRHILambdaCommand<FRHICommandListImmediate, LAMBDA>)(Forward<LAMBDA>(Lambda), LambdaName);

			if (ThreadFence == EThreadFence::Enabled)
			{
				RHIThreadFence(true);
			}
		}
	}

	template <typename LAMBDA>
	inline void EnqueueLambda(const TCHAR* LambdaName, LAMBDA&& Lambda)
	{
		FRHICommandListImmediate::EnqueueLambda(EThreadFence::Disabled, TEXT("TRHILambdaCommand"), Forward<LAMBDA>(Lambda));
	}

	template <typename LAMBDA>
	inline void EnqueueLambda(LAMBDA&& Lambda)
	{
		FRHICommandListImmediate::EnqueueLambda(TEXT("TRHILambdaCommand"), Forward<LAMBDA>(Lambda));
	}
	
	inline void* LockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI)
	{
		return GDynamicRHI->LockStagingBuffer_RenderThread(*this, StagingBuffer, Fence, Offset, SizeRHI);
	}

	inline void* LockStagingBuffer(FRHIStagingBuffer* StagingBuffer, uint32 Offset, uint32 SizeRHI)
	{
		return LockStagingBuffer(StagingBuffer, nullptr, Offset, SizeRHI);
	}
	
	inline void UnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer)
	{
		GDynamicRHI->UnlockStagingBuffer_RenderThread(*this, StagingBuffer);
	}

	inline bool GetTextureMemoryVisualizeData(FColor* TextureData,int32 SizeX,int32 SizeY,int32 Pitch,int32 PixelSize)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_GetTextureMemoryVisualizeData_Flush);
		ImmediateFlush(EImmediateFlushType::FlushRHIThread); 
		return GDynamicRHI->RHIGetTextureMemoryVisualizeData(TextureData,SizeX,SizeY,Pitch,PixelSize);
	}
	
	RHI_API FTextureRHIRef AsyncReallocateTexture2D(FRHITexture* Texture2D, int32 NewMipCount, int32 NewSizeX, int32 NewSizeY, FThreadSafeCounter* RequestStatus);

	inline FRHILockTextureResult LockTexture(const FRHILockTextureArgs& Arguments)
	{
		LLM_SCOPE(ELLMTag::Textures);
		return GDynamicRHI->RHILockTexture(*this, Arguments);
	}

	inline void UnlockTexture(const FRHILockTextureArgs& Arguments)
	{
		LLM_SCOPE(ELLMTag::Textures);
		GDynamicRHI->RHIUnlockTexture(*this, Arguments);
	}
	
	//UE_DEPRECATED(5.6, "LockTexture should be used for all texture type locks.")
	inline void* LockTexture2D(FRHITexture* Texture, uint32 MipIndex, EResourceLockMode LockMode, uint32& DestStride, bool bLockWithinMiptail, bool bFlushRHIThread = true, uint64* OutLockedByteCount = nullptr)
	{
		LLM_SCOPE(ELLMTag::Textures);

		const FRHILockTextureResult Result = LockTexture(FRHILockTextureArgs::Lock2D(Texture, MipIndex, LockMode, bLockWithinMiptail, bFlushRHIThread));
		DestStride = Result.Stride;
		if (OutLockedByteCount)
		{
			*OutLockedByteCount = Result.ByteCount;
		}
		return Result.Data;
	}
	
	//UE_DEPRECATED(5.6, "UnlockTexture should be used for all texture type unlocks.")
	inline void UnlockTexture2D(FRHITexture* Texture, uint32 MipIndex, bool bLockWithinMiptail, bool bFlushRHIThread = true)
	{
		UnlockTexture(FRHILockTextureArgs::Lock2D(Texture, MipIndex, RLM_Num, bLockWithinMiptail, bFlushRHIThread));
	}
	
	//UE_DEPRECATED(5.6, "LockTexture should be used for all texture type locks.")
	inline void* LockTexture2DArray(FRHITexture* Texture, uint32 ArrayIndex, uint32 MipIndex, EResourceLockMode LockMode, uint32& DestStride, bool bLockWithinMiptail)
	{
		LLM_SCOPE(ELLMTag::Textures);
		const FRHILockTextureResult Result = LockTexture(FRHILockTextureArgs::Lock2DArray(Texture, ArrayIndex, MipIndex, LockMode, bLockWithinMiptail));
		DestStride = Result.Stride;
		return Result.Data;
	}
	
	//UE_DEPRECATED(5.6, "UnlockTexture should be used for all texture type unlocks.")
	inline void UnlockTexture2DArray(FRHITexture* Texture, uint32 ArrayIndex, uint32 MipIndex, bool bLockWithinMiptail)
	{
		LLM_SCOPE(ELLMTag::Textures);
		UnlockTexture(FRHILockTextureArgs::Lock2DArray(Texture, ArrayIndex, MipIndex, RLM_Num, bLockWithinMiptail));
	}

	//UE_DEPRECATED(5.6, "LockTexture should be used for all texture type locks.")
	inline void* LockTextureCubeFace(FRHITexture* Texture, uint32 FaceIndex, uint32 ArrayIndex, uint32 MipIndex, EResourceLockMode LockMode, uint32& DestStride, bool bLockWithinMiptail)
	{
		LLM_SCOPE(ELLMTag::Textures);
		const FRHILockTextureResult Result = LockTexture(FRHILockTextureArgs::LockCubeFace(Texture, FaceIndex, ArrayIndex, MipIndex, LockMode, bLockWithinMiptail));
		DestStride = Result.Stride;
		return Result.Data;
	}

	//UE_DEPRECATED(5.6, "UnlockTexture should be used for all texture type unlocks.")
	inline void UnlockTextureCubeFace(FRHITexture* Texture, uint32 FaceIndex, uint32 ArrayIndex, uint32 MipIndex, bool bLockWithinMiptail)
	{
		LLM_SCOPE(ELLMTag::Textures);
		UnlockTexture(FRHILockTextureArgs::LockCubeFace(Texture, FaceIndex, ArrayIndex, MipIndex, RLM_Num, bLockWithinMiptail));
	}
	
	inline FUpdateTexture3DData BeginUpdateTexture3D(FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion)
	{
		checkf(UpdateRegion.DestX + UpdateRegion.Width <= Texture->GetSizeX(), TEXT("UpdateTexture3D out of bounds on X. Texture: %s, %i, %i, %i"), *Texture->GetName().ToString(), UpdateRegion.DestX, UpdateRegion.Width, Texture->GetSizeX());
		checkf(UpdateRegion.DestY + UpdateRegion.Height <= Texture->GetSizeY(), TEXT("UpdateTexture3D out of bounds on Y. Texture: %s, %i, %i, %i"), *Texture->GetName().ToString(), UpdateRegion.DestY, UpdateRegion.Height, Texture->GetSizeY());
		checkf(UpdateRegion.DestZ + UpdateRegion.Depth <= Texture->GetSizeZ(), TEXT("UpdateTexture3D out of bounds on Z. Texture: %s, %i, %i, %i"), *Texture->GetName().ToString(), UpdateRegion.DestZ, UpdateRegion.Depth, Texture->GetSizeZ());
		LLM_SCOPE(ELLMTag::Textures);
		return GDynamicRHI->RHIBeginUpdateTexture3D(*this, Texture, MipIndex, UpdateRegion);
	}

	inline void EndUpdateTexture3D(FUpdateTexture3DData& UpdateData)
	{
		LLM_SCOPE(ELLMTag::Textures);
		GDynamicRHI->RHIEndUpdateTexture3D(*this, UpdateData);
	}

	inline void EndMultiUpdateTexture3D(TArray<FUpdateTexture3DData>& UpdateDataArray)
	{
		LLM_SCOPE(ELLMTag::Textures);
		GDynamicRHI->RHIEndMultiUpdateTexture3D(*this, UpdateDataArray);
	}
	
	// ReadSurfaceFloatData reads texture data into FColor
	//	pixels in other formats are converted to FColor
	inline void ReadSurfaceData(FRHITexture* Texture,FIntRect Rect,TArray<FColor>& OutData,FReadSurfaceDataFlags InFlags)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_ReadSurfaceData_Flush);
		LLM_SCOPE(ELLMTag::Textures);
		ImmediateFlush(EImmediateFlushType::FlushRHIThread);  
		GDynamicRHI->RHIReadSurfaceData(Texture,Rect,OutData,InFlags);
	}
	
	// ReadSurfaceFloatData reads texture data into FLinearColor
	//	pixels in other formats are converted to FLinearColor
	// reading from float surfaces remaps the values into an interpolation of their {min,max} ; use RCM_MinMax to prevent that
	inline void ReadSurfaceData(FRHITexture* Texture, FIntRect Rect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_ReadSurfaceData_Flush);
		LLM_SCOPE(ELLMTag::Textures);
		ImmediateFlush(EImmediateFlushType::FlushRHIThread);
		GDynamicRHI->RHIReadSurfaceData(Texture, Rect, OutData, InFlags);
	}
	
	inline void MapStagingSurface(FRHITexture* Texture, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex = INDEX_NONE)
	{
		LLM_SCOPE(ELLMTag::Textures);
		GDynamicRHI->RHIMapStagingSurface_RenderThread(*this, Texture, GPUIndex, nullptr, OutData, OutWidth, OutHeight);
	}

	inline void MapStagingSurface(FRHITexture* Texture, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex = INDEX_NONE)
	{
		LLM_SCOPE(ELLMTag::Textures);
		GDynamicRHI->RHIMapStagingSurface_RenderThread(*this, Texture, GPUIndex, Fence, OutData, OutWidth, OutHeight);
	}
	
	inline void UnmapStagingSurface(FRHITexture* Texture, uint32 GPUIndex = INDEX_NONE)
	{
		LLM_SCOPE(ELLMTag::Textures);
		GDynamicRHI->RHIUnmapStagingSurface_RenderThread(*this, Texture, GPUIndex);
	}
	
	// ReadSurfaceFloatData reads texture data into FFloat16Color
	//	it only works if Texture is exactly PF_FloatRGBA (RGBA16F) !
	//	no conversion is done
	inline void ReadSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,TArray<FFloat16Color>& OutData,ECubeFace CubeFace,int32 ArrayIndex,int32 MipIndex)
	{
		LLM_SCOPE(ELLMTag::Textures);
		GDynamicRHI->RHIReadSurfaceFloatData_RenderThread(*this, Texture,Rect,OutData,CubeFace,ArrayIndex,MipIndex);
	}

	inline void ReadSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,TArray<FFloat16Color>& OutData,FReadSurfaceDataFlags Flags)
	{
		LLM_SCOPE(ELLMTag::Textures);
		GDynamicRHI->RHIReadSurfaceFloatData_RenderThread(*this, Texture,Rect,OutData,Flags);
	}

	inline void Read3DSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,FIntPoint ZMinMax,TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags Flags = FReadSurfaceDataFlags())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_Read3DSurfaceFloatData_Flush);
		LLM_SCOPE(ELLMTag::Textures);
		ImmediateFlush(EImmediateFlushType::FlushRHIThread);  
		GDynamicRHI->RHIRead3DSurfaceFloatData(Texture,Rect,ZMinMax,OutData,Flags);
	}

	inline void SubmitAndBlockUntilGPUIdle(const bool bDeleteResources = false)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_SubmitAndBlockUntilGPUIdle_Flush);

		const EImmediateFlushType::Type FlushType = bDeleteResources ? EImmediateFlushType::FlushRHIThreadFlushResources : EImmediateFlushType::FlushRHIThread;
		
		// Ensure all prior work is submitted down to the GPU, and the RHI thread is idle.
		ImmediateFlush(FlushType);

		// Block the calling thread (the render thread) until the GPU completes all work.
		GDynamicRHI->RHIBlockUntilGPUIdle();
	}
	
	UE_DEPRECATED(5.9, "BlockUntilGPUIdle() is deprecated. Call SubmitAndBlockUntilGPUIdle() instead.")
	inline void BlockUntilGPUIdle()
	{
		this->SubmitAndBlockUntilGPUIdle();
	}

	UE_DEPRECATED(5.9, "SubmitCommandsAndFlushGPU() is deprecated. Call SubmitAndBlockUntilGPUIdle() instead.")
	inline void SubmitCommandsAndFlushGPU()
	{
		this->SubmitAndBlockUntilGPUIdle();
	}
	
	inline bool IsRenderingSuspended()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_IsRenderingSuspended_Flush);
		ImmediateFlush(EImmediateFlushType::FlushRHIThread); 
		return GDynamicRHI->RHIIsRenderingSuspended();
	}
	
	inline void VirtualTextureSetFirstMipInMemory(FRHITexture* Texture, uint32 FirstMip)
	{
		GDynamicRHI->RHIVirtualTextureSetFirstMipInMemory(*this, Texture, FirstMip);
	}
	
	inline void VirtualTextureSetFirstMipVisible(FRHITexture* Texture, uint32 FirstMip)
	{
		GDynamicRHI->RHIVirtualTextureSetFirstMipVisible(*this, Texture, FirstMip);
	}

#if !UE_BUILD_SHIPPING
	inline void SerializeAccelerationStructure(FRHIRayTracingScene* Scene, const TCHAR* Path)
	{
		GDynamicRHI->RHISerializeAccelerationStructure(*this, Scene, Path);
	}
#endif
	
	inline void* GetNativeDevice()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_GetNativeDevice_Flush);
		ImmediateFlush(EImmediateFlushType::FlushRHIThread); 
		 
		return GDynamicRHI->RHIGetNativeDevice();
	}
	
	UE_DEPRECATED(5.8, "GetNativePhysicalDevice is deprecated. Use GetIVulkanDynamicRHI()->RHIGetVkPhysicalDevice().")
	inline void* GetNativePhysicalDevice()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_GetNativePhysicalDevice_Flush);
		ImmediateFlush(EImmediateFlushType::FlushRHIThread); 
		 
		return GDynamicRHI->RHIGetNativePhysicalDevice();
	}
	
	UE_DEPRECATED(5.8, "GetNativeGraphicsQueue is deprecated. Use platform-specific RHI interfaces instead.")
	inline void* GetNativeGraphicsQueue()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_GetNativeGraphicsQueue_Flush);
		ImmediateFlush(EImmediateFlushType::FlushRHIThread); 
		 
		return GDynamicRHI->RHIGetNativeGraphicsQueue();
	}
	
	UE_DEPRECATED(5.8, "GetNativeComputeQueue is deprecated. Use platform-specific RHI interfaces instead.")
	inline void* GetNativeComputeQueue()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_GetNativeComputeQueue_Flush);
		ImmediateFlush(EImmediateFlushType::FlushRHIThread); 
		 
		return GDynamicRHI->RHIGetNativeComputeQueue();
	}
	
	UE_DEPRECATED(5.8, "GetNativeInstance is deprecated. The only useful implementation was in Vulkan RHI. Use GetIVulkanDynamicRHI()->RHIGetVkInstance() instead.")
	inline void* GetNativeInstance()
	{
		return nullptr;
	}
	
	UE_DEPRECATED(5.8,
		"GetNativeCommandBuffer is dangerous, and has been deprecated. "
		"Command buffers are owned by RHI contexts, and can therefore only be retrieved by bottom-of-pipe threads (the RHI thread, or parallel translate tasks). "
		"Use RHI platform-specific interfaces to retrieve the native command buffer from a bottom-of-pipe RHI command list in a typesafe way.")
	inline void* GetNativeCommandBuffer()
	{
		if (ensure(IsBottomOfPipe()))
		{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return GetContext().RHIGetNativeCommandBuffer();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		return nullptr;
	}

	UE_DEPRECATED(5.8, "SubmitCommandsHint is deprecated. Consider calling ImmediateFlush(EImmediateFlushType::DispatchToRHIThread) instead.")
	inline void SubmitCommandsHint()
	{
		ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);
	}
};

// All command list members should be contained within FRHICommandListBase. The Immediate/Compute/regular types are just interfaces.
static_assert(sizeof(FRHICommandListImmediate) == sizeof(FRHICommandListBase), "FRHICommandListImmediate should not contain additional members.");
static_assert(sizeof(FRHIComputeCommandList  ) == sizeof(FRHICommandListBase), "FRHIComputeCommandList should not contain additional members.");
static_assert(sizeof(FRHICommandList         ) == sizeof(FRHICommandListBase), "FRHICommandList should not contain additional members.");

class FRHICommandListScopedFlushAndExecute
{
	FRHICommandListImmediate& RHICmdList;

public:
	FRHICommandListScopedFlushAndExecute(FRHICommandListImmediate& InRHICmdList)
		: RHICmdList(InRHICmdList)
	{
		check(RHICmdList.IsTopOfPipe());
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
		RHICmdList.bExecuting = true;
	}
	~FRHICommandListScopedFlushAndExecute()
	{
		RHICmdList.bExecuting = false;
	}
};

/** Takes a reference to defer deletion of RHI resources. */
void RHI_API RHIResourceLifetimeAddRef(int32 NumRefs = 1);

/** Releases a reference to defer deletion of RHI resources. If the reference count hits zero, resources are queued for deletion. */
void RHI_API RHIResourceLifetimeReleaseRef(FRHICommandListImmediate& RHICmdList, int32 NumRefs = 1);

class FRHICommandListScopedExtendResourceLifetime
{
public:
	FRHICommandListScopedExtendResourceLifetime(FRHICommandListImmediate& InRHICmdList)
		: RHICmdList(InRHICmdList)
	{
		RHIResourceLifetimeAddRef();
	}

	~FRHICommandListScopedExtendResourceLifetime()
	{
		RHIResourceLifetimeReleaseRef(RHICmdList);
	}

private:
	FRHICommandListImmediate& RHICmdList;
};

//
// Helper to activate a specific RHI pipeline within a block of renderer code.
// Allows command list recording code to switch between graphics / async compute etc.
// Restores the previous active pipeline when the scope is ended.
//
class FRHICommandListScopedPipeline
{
	FRHICommandListBase& RHICmdList;
	ERHIPipeline PreviousPipeline;

public:
	FRHICommandListScopedPipeline(FRHICommandListBase& RHICmdList, ERHIPipeline Pipeline)
		: RHICmdList(RHICmdList)
		, PreviousPipeline(RHICmdList.SwitchPipeline(Pipeline))
	{
	}

	~FRHICommandListScopedPipeline()
	{
		RHICmdList.SwitchPipeline(PreviousPipeline);
	}
};

struct FRHIScopedGPUMask
{
	FRHIComputeCommandList& RHICmdList;
	FRHIGPUMask PrevGPUMask;

	inline FRHIScopedGPUMask(FRHIComputeCommandList& InRHICmdList, FRHIGPUMask InGPUMask)
		: RHICmdList(InRHICmdList)
		, PrevGPUMask(InRHICmdList.GetGPUMask())
	{
		InRHICmdList.SetGPUMask(InGPUMask);
	}

	inline ~FRHIScopedGPUMask()
	{
		RHICmdList.SetGPUMask(PrevGPUMask);
	}

	FRHIScopedGPUMask(FRHIScopedGPUMask const&) = delete;
	FRHIScopedGPUMask(FRHIScopedGPUMask&&) = delete;
};

#if WITH_MGPU
	#define SCOPED_GPU_MASK(RHICmdList, GPUMask) FRHIScopedGPUMask UE_JOIN(ScopedGPUMask, __LINE__){ RHICmdList, GPUMask }
#else
	#define SCOPED_GPU_MASK(RHICmdList, GPUMask)
#endif // WITH_MGPU

struct FScopedUniformBufferStaticBindings
{
	FScopedUniformBufferStaticBindings(FRHIComputeCommandList& InRHICmdList, FUniformBufferStaticBindings UniformBuffers)
		: RHICmdList(InRHICmdList)
	{
#if VALIDATE_UNIFORM_BUFFER_STATIC_BINDINGS
		OnScopeEnter();
#endif

		RHICmdList.SetStaticUniformBuffers(UniformBuffers);
	}

	template <typename... TArgs>
	FScopedUniformBufferStaticBindings(FRHIComputeCommandList& InRHICmdList, TArgs... Args)
		: FScopedUniformBufferStaticBindings(InRHICmdList, FUniformBufferStaticBindings{ Args... })
	{}

	~FScopedUniformBufferStaticBindings()
	{
		RHICmdList.SetStaticUniformBuffers(FUniformBufferStaticBindings());

#if VALIDATE_UNIFORM_BUFFER_STATIC_BINDINGS
		OnScopeExit();
#endif
	}

	FRHIComputeCommandList& RHICmdList;

private:
#if VALIDATE_UNIFORM_BUFFER_STATIC_BINDINGS
	RHI_API static void OnScopeEnter();
	RHI_API static void OnScopeExit();
#endif
};

#define SCOPED_UNIFORM_BUFFER_STATIC_BINDINGS(RHICmdList, UniformBuffers) FScopedUniformBufferStaticBindings UE_JOIN(UniformBuffers, __LINE__){ RHICmdList, UniformBuffers }

// Helper to enable the use of graphics RHI command lists from within platform RHI implementations.
// Recorded commands are dispatched when the command list is destructed. Intended for use on the stack / in a scope block.
class FRHICommandList_RecursiveHazardous : public FRHICommandList
{
public:
	RHI_API FRHICommandList_RecursiveHazardous(IRHICommandContext* Context);
	RHI_API ~FRHICommandList_RecursiveHazardous();
};

// Helper class used internally by RHIs to make use of FRHICommandList_RecursiveHazardous safer.
// Access to the underlying context is exposed via RunOnContext() to ensure correct ordering of commands.
template <typename ContextType>
class TRHICommandList_RecursiveHazardous : public FRHICommandList_RecursiveHazardous
{
	template <typename LAMBDA>
	struct TRHILambdaCommand final : public FRHICommandBase
	{
		LAMBDA Lambda;

		TRHILambdaCommand(LAMBDA&& InLambda)
			: Lambda(Forward<LAMBDA>(InLambda))
		{}

		void ExecuteAndDestruct(FRHICommandListBase& CmdList) override final
		{
			// RunOnContext always requires the lowest level (platform) context, not the validation RHI context.
			ContextType& Context = static_cast<ContextType&>(CmdList.GetContext().GetLowestLevelContext());
			Context.BeginRecursiveCommand();

			Lambda(Context);
			Lambda.~LAMBDA();
		}
	};

public:
	TRHICommandList_RecursiveHazardous(ContextType* Context)
		: FRHICommandList_RecursiveHazardous(Context)
	{}

	template <typename LAMBDA>
	inline void RunOnContext(LAMBDA&& Lambda)
	{
		if (Bypass())
		{
			// RunOnContext always requires the lowest level (platform) context, not the validation RHI context.
			ContextType& Context = static_cast<ContextType&>(GetContext().GetLowestLevelContext());
			Context.BeginRecursiveCommand();

			Lambda(Context);
		}
		else
		{
			ALLOC_COMMAND(TRHILambdaCommand<LAMBDA>)(Forward<LAMBDA>(Lambda));
		}
	}
};

// Helper to enable the use of compute RHI command lists from within platform RHI implementations.
// Recorded commands are dispatched when the command list is destructed. Intended for use on the stack / in a scope block.
class FRHIComputeCommandList_RecursiveHazardous : public FRHIComputeCommandList
{
public:
	RHI_API FRHIComputeCommandList_RecursiveHazardous(IRHIComputeContext* Context);
	RHI_API ~FRHIComputeCommandList_RecursiveHazardous();
};

// Helper class used internally by RHIs to make use of FRHIComputeCommandList_RecursiveHazardous safer.
// Access to the underlying context is exposed via RunOnContext() to ensure correct ordering of commands.
template <typename ContextType>
class TRHIComputeCommandList_RecursiveHazardous : public FRHIComputeCommandList_RecursiveHazardous
{
	template <typename LAMBDA>
	struct TRHILambdaCommand final : public FRHICommandBase
	{
		LAMBDA Lambda;

		TRHILambdaCommand(LAMBDA&& InLambda)
			: Lambda(Forward<LAMBDA>(InLambda))
		{}

		void ExecuteAndDestruct(FRHICommandListBase& CmdList) override final
		{
			// RunOnContext always requires the lowest level (platform) context, not the validation RHI context.
			ContextType& Context = static_cast<ContextType&>(CmdList.GetComputeContext().GetLowestLevelContext());
			Context.BeginRecursiveCommand();

			Lambda(Context);
			Lambda.~LAMBDA();
		}
	};

public:
	TRHIComputeCommandList_RecursiveHazardous(ContextType* Context)
		: FRHIComputeCommandList_RecursiveHazardous(Context)
	{}

	template <typename LAMBDA>
	inline void RunOnContext(LAMBDA&& Lambda)
	{
		if (Bypass())
		{
			// RunOnContext always requires the lowest level (platform) context, not the validation RHI context.
			ContextType& Context = static_cast<ContextType&>(GetComputeContext().GetLowestLevelContext());
			Context.BeginRecursiveCommand();

			Lambda(Context);
		}
		else
		{
			ALLOC_COMMAND(TRHILambdaCommand<LAMBDA>)(Forward<LAMBDA>(Lambda));
		}
	}
};

class FRHISubCommandList : public FRHICommandList
{
public:
	FRHISubCommandList(FRHIGPUMask GPUMask, TSharedPtr<FRHIParallelRenderPassInfo> const& RenderPassInfo)
		: FRHICommandList(GPUMask)
	{
		RenderPassInfo->Validate();
		SubRenderPassInfo = RenderPassInfo;
		CacheActiveRenderTargets(*RenderPassInfo);
	}
};

class FRHICommandListExecutor
{
public:
	static inline FRHICommandListImmediate& GetImmediateCommandList();
	RHI_API void LatchBypass();

	RHI_API FGraphEventRef Submit(TConstArrayView<FRHICommandListBase*> AdditionalCommandLists, ERHISubmitFlags SubmitFlags);

	RHI_API static void WaitOnRHIThreadFence(FGraphEventRef& Fence);

	//
	// Blocks the calling thread until all dispatch prerequisites of enqueued parallel command lists are completed.
	//
	void WaitForTasks()
	{
		WaitForTasks(WaitOutstandingTasks);
	}

	//
	// Blocks the calling thread until all specified tasks are completed.
	//
	RHI_API void WaitForTasks(FGraphEventArray& OutstandingTasks);

	// Global graph events must be destroyed explicitly to avoid undefined order of static destruction, as they can be destroyed after their allocator.
	void CleanupGraphEvents();

	inline bool Bypass() const
	{
#if CAN_TOGGLE_COMMAND_LIST_BYPASS
		return bLatchedBypass;
#else
		return false;
#endif
	}

	inline bool UseParallelAlgorithms() const
	{
#if CAN_TOGGLE_COMMAND_LIST_BYPASS
		return bLatchedUseParallelAlgorithms;
#else
		return  FApp::ShouldUseThreadingForPerformance() && !Bypass() && (GSupportsParallelRenderingTasksWithSeparateRHIThread || !IsRunningRHIInSeparateThread());
#endif
	}

	//
	// Returns true if any RHI dispatch, translate or submission tasks are currently running.
	// This works regardless of engine threading mode (i.e. with or without an RHI thread and parallel translate).
	// 
	// When this function returns false, we can be sure there are no threads active within the platform RHI, besides the render thread.
	//
	RHI_API static bool AreRHITasksActive();

	//
	// Adds a prerequisite for subsequent Submit dispatch tasks.
	// 
	// This function should only be called from the render thread.
	//
	RHI_API void AddNextDispatchPrerequisite(FGraphEventRef Prereq);

	//
	// Gets the CompletionEvent for the most recent submit to GPU
	// 
	// This function should only be called from the render thread.
	//
	const FGraphEventRef& GetCompletionEvent() const
	{
		return CompletionEvent;
	}

	FGraphEventArray WaitOutstandingTasks;

private:
	bool bLatchedBypass = false;
	bool bLatchedUseParallelAlgorithms = false;
#if WITH_RHI_BREADCRUMBS
	bool bEmitBreadcrumbs = false;
#endif

	friend class FRHICommandListBase;
	friend class FRHICommandListImmediate;
	FRHICommandListImmediate CommandListImmediate;

	//
	// Helper for efficiently enqueuing work to TaskGraph threads. Work items within a single pipe are always executed in-order (FIFO) even if they have no prerequisites.
	// Uses an atomic compare-and-swap mechanism to append new tasks to the end of existing ones, avoiding the overhead of having the TaskGraph itself do the task scheduling.
	//
	class FTaskPipe
	{
		struct FTask;

		FTask* Current = nullptr;
		FGraphEventRef LastEvent = nullptr;
		TOptional<ENamedThreads::Type> LastThread {};

		FGraphEventRef LaunchTask(FTask* Task) const;
		void Execute(FTask* Task, FGraphEventRef const& CurrentEvent) const;

	public:
		struct ITask;
		using FFunc = TFunction<void(ITask&)>;

		struct ITask
		{
			virtual void ExtendTask(FGraphEventArray&& NewPrereqs, FFunc&& NewLambda) = 0;
		};

		// Enqueues the given lambda to run on the named thread.
		void Enqueue(ENamedThreads::Type NamedThread, FGraphEventArray&& Prereqs, FFunc&& Lambda);

		// Returns a graph event that will be signalled once all work submitted prior to calling Close() has completed.
		FGraphEventRef Close();

		void CleanupGraphEvents()
		{
			LastEvent = nullptr;
		}
	};

	struct FSubmitState;

	// One per RHI context array, multiple RHICmdLists replayed into it
	struct FTranslateState
	{
		struct FPipelineState
		{
#if WITH_RHI_BREADCRUMBS
			FRHIBreadcrumbRange Range {};
#endif
			IRHIComputeContext* Context = nullptr;
		};
		TRHIPipelineArray<FPipelineState> PipelineStates {};

		IRHIUploadContext* UploadContext = nullptr;

		FDynamicRHI::FRHICloseTranslateChainResult FinalizedCommandLists;

#if WITH_RHI_BREADCRUMBS
		FRHIBreadcrumbAllocatorArray BreadcrumbAllocatorRefs {};
#endif

		FTaskPipe TranslatePipe;
		uint32 NumCommands = 0;

		FGraphEventRef TranslateEvent;
		std::atomic<FTranslateState*> Next { nullptr };

		// True when parallel translate is enabled globally
		const bool bAllowParallelTranslate;

		// True when this particular chain is a parallel translate chain
		const bool bParallel;

		const bool bUsingSubCmdLists;

		bool bShouldFinalize = true;

		FTaskPipe* EnqueueTranslateTask(FGraphEventArray&& Prereqs, FTaskPipe::FFunc&& Lambda);

		void Translate_ExecuteCommandList(FRHICommandListBase* CmdList);
		void Translate_CloseChain();

		FTranslateState(const FSubmitState& SubmitState, FRHICommandListBase* FirstCmdList);
	};

	// One per call to RHISubmitCommandLists
	struct FSubmitState
	{
		FGraphEventRef CompletionEvent;

		struct
		{
			TArray<TUniquePtr<FTranslateState>> All;
			FGraphEventArray Events;

			FTranslateState* First = nullptr;
			FTranslateState* Current = nullptr;
			FTranslateState* Previous = nullptr;
		} TranslateJobs;

		int32 MaxCommandsPerTranslate = 0;
		bool bAllowSingleParallelCombine = false;
		bool bAllowParallelTranslate = true;
		bool bBatchSubmissions = true;

	#if WITH_RHI_BREADCRUMBS
		bool bEmitBreadcrumbs = false;
	#endif

		ERHISubmitFlags SubmitFlags = ERHISubmitFlags::None;
		TArray<FRHIResource*> ResourcesToDelete {};
		bool bIncludeExtendedLifetimeResources = false;
		
		FGraphEventRef BeginGraphEvent;
		FGraphEventArray ChildGraphEvents;

		struct FCloseArgs
		{
#if WITH_RHI_BREADCRUMBS
			TRHIPipelineArray<FRHIBreadcrumbNode*> GPUBreadcrumbs;
#endif
#if STATS
			TOptional<int64> StatsFrame;
#endif
		};

		void Dispatch_ProcessCommandList(FRHICommandListBase* CmdList);
		void Dispatch_PrepareBreadcrumbs(FRHICommandListBase* CmdList);
		FGraphEventRef Dispatch_CloseTranslateChain(bool bLastChainInSubmission, uint32 RTSubmissionID);
		void Dispatch_CloseSubmission(TOptional<int64> LocalStatsFrame, uint32 RTSubmissionID);

		void Submit_SubmitToGPU(FTranslateState* First, FTaskPipe::ITask& Task, uint64 TracingSubmissionID);
		void Submit_Close(const FCloseArgs& Args);
	} *SubmitState = nullptr;

	FTaskPipe DispatchPipe;
	FTaskPipe SingleTranslatePipe;
	FTaskPipe SubmitPipe;

	FGraphEventRef LastMutate;
	FGraphEventRef LastClose;
	FGraphEventRef CompletionEvent;

	void EnqueueDispatchTask(FGraphEventArray&& Prereqs, FTaskPipe::FFunc&& Lambda);
	void EnqueueSubmitTask  (FGraphEventArray&& Prereqs, FTaskPipe::FFunc&& Lambda);
	FGraphEventArray NextDispatchTaskPrerequisites;

#if WITH_RHI_BREADCRUMBS

	struct FBreadcrumbState
	{
		FRHIBreadcrumbNodeRef Current{}; // Used by dispatch thread
		FRHIBreadcrumbNodeRef Last   {}; // Used by submit thread
	};

	struct
	{
		FBreadcrumbState CPU {};
		TRHIPipelineArray<FBreadcrumbState> GPU { InPlace };
	} Breadcrumbs {};
	
#endif

	// Counts the number of calls to RHIEndFrame, and is used in GPU profiler frame boundary events.
	uint32 FrameNumber = 0;

	bool AllowParallel() const;
};

extern RHI_API FRHICommandListExecutor GRHICommandList;

extern RHI_API FAutoConsoleTaskPriority CPrio_SceneRenderingTask;

class FRenderTask
{
public:
	inline static ENamedThreads::Type GetDesiredThread()
	{
		return CPrio_SceneRenderingTask.Get();
	}
};

inline FRHICommandListImmediate& FRHICommandListImmediate::Get()
{
	check(IsInRenderingThread());
	return FRHICommandListExecutor::GetImmediateCommandList();
}

inline FRHICommandListImmediate& FRHICommandListExecutor::GetImmediateCommandList()
{
	// @todo - fix remaining use of the immediate command list on other threads, then uncomment this check.
	//check(IsInRenderingThread());
	return GRHICommandList.CommandListImmediate;
}

UE_DEPRECATED(5.7, "RHICreateTextureReference with an implied immediate command list is deprecated")
inline FTextureReferenceRHIRef RHICreateTextureReference(FRHITexture* InReferencedTexture = nullptr)
{
	return FRHICommandListImmediate::Get().CreateTextureReference(InReferencedTexture);
}

UE_DEPRECATED(5.7, "RHIUpdateTextureReference with an implied immediate command list is deprecated")
inline void RHIUpdateTextureReference(FRHITextureReference* TextureRef, FRHITexture* NewTexture)
{
	FRHICommandListImmediate::Get().UpdateTextureReference(TextureRef, NewTexture);
}

// Temporary workaround until we figure out if we want to route a command list through ReleaseRHI()
inline void RHIClearTextureReference(FRHITextureReference* TextureRef)
{
	FRHICommandListImmediate::Get().UpdateTextureReference(TextureRef, nullptr);
}

UE_DEPRECATED(5.8, "RHICreateTexture with an implied immediate command list is deprecated")
inline FTextureRHIRef RHICreateTexture(const FRHITextureCreateDesc& CreateDesc)
{
	return FRHICommandListImmediate::Get().CreateTexture(CreateDesc);
}

inline FTextureRHIRef RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent)
{
	LLM_SCOPE(EnumHasAnyFlags(Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable) ? ELLMTag::RenderTargets : ELLMTag::Textures);
	const ERHIAccess ResourceState = InResourceState == ERHIAccess::Unknown ? RHIGetDefaultResourceState((ETextureCreateFlags)Flags, InitialMipData != nullptr) : InResourceState;
	return GDynamicRHI->RHIAsyncCreateTexture2D(SizeX, SizeY, Format, NumMips, Flags, ResourceState, InitialMipData, NumInitialMips, DebugName, OutCompletionEvent);
}

UE_DEPRECATED(5.8, "RHIAsyncReallocateTexture2D with an implied immediate command list is deprecated")
inline FTextureRHIRef RHIAsyncReallocateTexture2D(FRHITexture* Texture2D, int32 NewMipCount, int32 NewSizeX, int32 NewSizeY, FThreadSafeCounter* RequestStatus)
{
	return FRHICommandListExecutor::GetImmediateCommandList().AsyncReallocateTexture2D(Texture2D, NewMipCount, NewSizeX, NewSizeY, RequestStatus);
}

// UE_DEPRECATED(5.6, "RHICmdList.LockTexture should be used for all texture type locks.")
inline void* RHILockTexture2D(FRHITexture* Texture, uint32 MipIndex, EResourceLockMode LockMode, uint32& DestStride, bool bLockWithinMiptail, bool bFlushRHIThread = true, uint64* OutLockedByteCount = nullptr)
{
	return FRHICommandListExecutor::GetImmediateCommandList().LockTexture2D(Texture, MipIndex, LockMode, DestStride, bLockWithinMiptail, bFlushRHIThread, OutLockedByteCount);
}

//UE_DEPRECATED(5.6, "RHICmdList.UnlockTexture should be used for all texture type unlocks.")
inline void RHIUnlockTexture2D(FRHITexture* Texture, uint32 MipIndex, bool bLockWithinMiptail, bool bFlushRHIThread = true)
{
	FRHICommandListExecutor::GetImmediateCommandList().UnlockTexture2D(Texture, MipIndex, bLockWithinMiptail, bFlushRHIThread);
}

UE_DEPRECATED(5.8, "RHILockTexture2DArray with an implied immediate command list is deprecated")
inline void* RHILockTexture2DArray(FRHITexture* Texture, uint32 TextureIndex, uint32 MipIndex, EResourceLockMode LockMode, uint32& DestStride, bool bLockWithinMiptail)
{
	return FRHICommandListExecutor::GetImmediateCommandList().LockTexture2DArray(Texture, TextureIndex, MipIndex, LockMode, DestStride, bLockWithinMiptail);
}

UE_DEPRECATED(5.8, "RHIUnlockTexture2DArray with an implied immediate command list is deprecated")
inline void RHIUnlockTexture2DArray(FRHITexture* Texture, uint32 TextureIndex, uint32 MipIndex, bool bLockWithinMiptail)
{
	 FRHICommandListExecutor::GetImmediateCommandList().UnlockTexture2DArray(Texture, TextureIndex, MipIndex, bLockWithinMiptail);
}

UE_DEPRECATED(5.8, "RHILockTextureCubeFace with an implied immediate command list is deprecated")
inline void* RHILockTextureCubeFace(FRHITexture* Texture, uint32 FaceIndex, uint32 ArrayIndex, uint32 MipIndex, EResourceLockMode LockMode, uint32& DestStride, bool bLockWithinMiptail)
{
	return FRHICommandListExecutor::GetImmediateCommandList().LockTextureCubeFace(Texture, FaceIndex, ArrayIndex, MipIndex, LockMode, DestStride, bLockWithinMiptail);
}

UE_DEPRECATED(5.8, "RHIUnlockTextureCubeFace with an implied immediate command list is deprecated")
inline void RHIUnlockTextureCubeFace(FRHITexture* Texture, uint32 FaceIndex, uint32 ArrayIndex, uint32 MipIndex, bool bLockWithinMiptail)
{
	FRHICommandListExecutor::GetImmediateCommandList().UnlockTextureCubeFace(Texture, FaceIndex, ArrayIndex, MipIndex, bLockWithinMiptail);
}

UE_DEPRECATED(5.8, "RHIUpdateTexture2D with an implied immediate command list is deprecated")
inline void RHIUpdateTexture2D(FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData)
{
	 FRHICommandListExecutor::GetImmediateCommandList().UpdateTexture2D(Texture, MipIndex, UpdateRegion, SourcePitch, SourceData);
}

UE_DEPRECATED(5.8, "RHIBeginUpdateTexture3D with an implied immediate command list is deprecated")
inline FUpdateTexture3DData RHIBeginUpdateTexture3D(FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion)
{
	return FRHICommandListExecutor::GetImmediateCommandList().BeginUpdateTexture3D(Texture, MipIndex, UpdateRegion);
}

UE_DEPRECATED(5.8, "RHIEndUpdateTexture3D with an implied immediate command list is deprecated")
inline void RHIEndUpdateTexture3D(FUpdateTexture3DData& UpdateData)
{
	FRHICommandListExecutor::GetImmediateCommandList().EndUpdateTexture3D(UpdateData);
}

UE_DEPRECATED(5.8, "RHIEndMultiUpdateTexture3D with an implied immediate command list is deprecated")
inline void RHIEndMultiUpdateTexture3D(TArray<FUpdateTexture3DData>& UpdateDataArray)
{
	FRHICommandListExecutor::GetImmediateCommandList().EndMultiUpdateTexture3D(UpdateDataArray);
}

UE_DEPRECATED(5.8, "RHIUpdateTexture3D with an implied immediate command list is deprecated")
inline void RHIUpdateTexture3D(FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData)
{
	 FRHICommandListExecutor::GetImmediateCommandList().UpdateTexture3D(Texture, MipIndex, UpdateRegion, SourceRowPitch, SourceDepthPitch, SourceData);
}

UE_DEPRECATED(5.9, "RHIFlushResources has been deprecated, use ImmediateFlush or SubmitAndBlockUntilGPUIdle(true) instead to block")
inline void RHIFlushResources()
{
	FRHICommandListExecutor::GetImmediateCommandList().ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
}

UE_DEPRECATED(5.8, "RHIVirtualTextureSetFirstMipInMemory with an implied immediate command list is deprecated")
inline void RHIVirtualTextureSetFirstMipInMemory(FRHITexture* Texture, uint32 FirstMip)
{
	 FRHICommandListExecutor::GetImmediateCommandList().VirtualTextureSetFirstMipInMemory(Texture, FirstMip);
}

UE_DEPRECATED(5.8, "RHIVirtualTextureSetFirstMipVisible with an implied immediate command list is deprecated")
inline void RHIVirtualTextureSetFirstMipVisible(FRHITexture* Texture, uint32 FirstMip)
{
	 FRHICommandListExecutor::GetImmediateCommandList().VirtualTextureSetFirstMipVisible(Texture, FirstMip);
}

inline void* RHIGetNativeDevice()
{
	return FRHICommandListExecutor::GetImmediateCommandList().GetNativeDevice();
}

UE_DEPRECATED(5.8, "RHIGetNativePhysicalDevice is deprecated. Use GetIVulkanDynamicRHI()->RHIGetVkPhysicalDevice().")
inline void* RHIGetNativePhysicalDevice()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FRHICommandListExecutor::GetImmediateCommandList().GetNativePhysicalDevice();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UE_DEPRECATED(5.8, "RHIGetNativeGraphicsQueue is deprecated. Use platform-specific RHI interfaces instead.")
inline void* RHIGetNativeGraphicsQueue()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FRHICommandListExecutor::GetImmediateCommandList().GetNativeGraphicsQueue();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UE_DEPRECATED(5.8, "RHIGetNativeComputeQueue is deprecated. Use platform-specific RHI interfaces instead.")
inline void* RHIGetNativeComputeQueue()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FRHICommandListExecutor::GetImmediateCommandList().GetNativeComputeQueue();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UE_DEPRECATED(5.8, "RHIGetNativeInstance is deprecated. The only useful implementation was in Vulkan RHI. Use GetIVulkanDynamicRHI()->RHIGetVkInstance() instead.")
inline void* RHIGetNativeInstance()
{
	return nullptr;
}

UE_DEPRECATED(5.8,
	"GetNativeCommandBuffer is dangerous, and has been deprecated. "
	"Command buffers are owned by RHI contexts, and can therefore only be retrieved by bottom-of-pipe threads (the RHI thread, or parallel translate tasks). "
	"Use RHI platform-specific interfaces to retrieve the native command buffer from a bottom-of-pipe RHI command list in a typesafe way.")
inline void* RHIGetNativeCommandBuffer()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FRHICommandListExecutor::GetImmediateCommandList().GetNativeCommandBuffer();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

inline FRHIShaderLibraryRef RHICreateShaderLibrary(EShaderPlatform Platform, FString const& FilePath, FString const& Name)
{
    return GDynamicRHI->RHICreateShaderLibrary(Platform, FilePath, Name);
}

UE_DEPRECATED(5.8, "RHILockStagingBuffer with an implied immediate command list is deprecated")
inline void* RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, uint32 Offset, uint32 Size)
{
	return FRHICommandListExecutor::GetImmediateCommandList().LockStagingBuffer(StagingBuffer, Offset, Size);
}

UE_DEPRECATED(5.8, "RHILockStagingBuffer with an implied immediate command list is deprecated")
inline void* RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 Size)
{
	return FRHICommandListExecutor::GetImmediateCommandList().LockStagingBuffer(StagingBuffer, Fence, Offset, Size);
}

UE_DEPRECATED(5.8, "RHIUnlockStagingBuffer with an implied immediate command list is deprecated")
inline void RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer)
{
	 FRHICommandListExecutor::GetImmediateCommandList().UnlockStagingBuffer(StagingBuffer);
}

inline FRayTracingGeometryRHIRef RHICreateRayTracingGeometry(const FRayTracingGeometryInitializer& Initializer)
{
	return FRHICommandListExecutor::GetImmediateCommandList().CreateRayTracingGeometry(Initializer);
}

inline FRayTracingAccelerationStructureSize RHICalcRayTracingGeometrySize(const FRayTracingGeometryInitializer& Initializer)
{
	return GDynamicRHI->RHICalcRayTracingGeometrySize(Initializer);
}

FORCEINLINE FRayTracingClusterOperationSize RHICalcRayTracingClusterOperationSize(const FRayTracingClusterOperationInitializer& Initializer)
{
	return GDynamicRHI->RHICalcRayTracingClusterOperationSize(Initializer);
}

inline FRayTracingAccelerationStructureOfflineMetadata RHIGetRayTracingGeometryOfflineMetadata(const FRayTracingGeometryOfflineDataHeader& OfflineDataHeader)
{
	return GDynamicRHI->RHIGetRayTracingGeometryOfflineMetadata(OfflineDataHeader);
}

UE_DEPRECATED(5.7, "BindDebugLabelName with an implied immediate command list is deprecated")
inline void RHIBindDebugLabelName(FRHITexture* Texture, const TCHAR* Name)
{
	FRHICommandListImmediate::Get().BindDebugLabelName(Texture, Name);
}

UE_DEPRECATED(5.7, "BindDebugLabelName with an implied immediate command list is deprecated")
inline void RHIBindDebugLabelName(FRHIBuffer* Buffer, const TCHAR* Name)
{
	FRHICommandListImmediate::Get().BindDebugLabelName(Buffer, Name);
}

UE_DEPRECATED(5.7, "BindDebugLabelName with an implied immediate command list is deprecated")
inline void RHIBindDebugLabelName(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const TCHAR* Name)
{
	FRHICommandListImmediate::Get().BindDebugLabelName(UnorderedAccessViewRHI, Name);
}

namespace UE::RHI
{

	//
	// Copies shared mip levels from one texture to another.
	// Both textures must have full mip chains, share the same format, and have the same aspect ratio.
	// The source texture must be in the CopySrc state, and the destination texture must be in the CopyDest state.
	// 
	RHI_API void CopySharedMips(FRHICommandList& RHICmdList, FRHITexture* SrcTexture, FRHITexture* DstTexture);

	//
	// Same as CopySharedMips(), but assumes both source and destination textures are in the SRVMask state.
	// Adds transitions to move the textures to/from the CopySrc/CopyDest states, restoring SRVMask when done.
	//
	// Provided for backwards compatibility. Caller should prefer CopySharedMips() with optimally batched transitions.
	//
	RHI_API void CopySharedMips_AssumeSRVMaskState(FRHICommandList& RHICmdList, FRHITexture* SrcTexture, FRHITexture* DstTexture);

	// Backwards compatibility adaptor to convert new FRHIBatchedShaderParameters to legacy FRayTracingShaderBindings.
	// This function will be deprecated in a future release, once legacy FRayTracingShaderBindings is removed.
	RHI_API FRayTracingShaderBindings ConvertRayTracingShaderBindings(const FRHIBatchedShaderParameters& BatchedParameters);

} //! UE::RHI

#undef RHICOMMAND_CALLSTACK

#include "RHICommandList.inl"
