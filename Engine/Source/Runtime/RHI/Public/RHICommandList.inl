// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHICommandList.inl: RHI Command List inline definitions.
=============================================================================*/

#pragma once

class FRHICommandListBase;
class FRHICommandListExecutor;
class FRHICommandListImmediate;
class FRHIResource;
class FScopedRHIThreadStaller;
struct FRHICommandBase;

inline bool FRHICommandListBase::IsImmediate() const
{
	return PersistentState.bImmediate;
}

inline FRHICommandListImmediate& FRHICommandListBase::GetAsImmediate()
{
	checkf(IsImmediate(), TEXT("This operation expects the immediate command list."));
	return static_cast<FRHICommandListImmediate&>(*this);
}

inline bool FRHICommandListBase::Bypass() const
{
#if CAN_TOGGLE_COMMAND_LIST_BYPASS
	return GRHICommandList.Bypass() && IsImmediate();
#else
	return false;
#endif
}

inline FScopedRHIThreadStaller::FScopedRHIThreadStaller(class FRHICommandListImmediate& InImmed, bool bDoStall)
	: Immed(nullptr)
{
	if (bDoStall && IsRunningRHIInSeparateThread())
	{
		check(IsInRenderingThread());
		if (InImmed.StallRHIThread())
		{
			Immed = &InImmed;
		}
	}
}

inline FScopedRHIThreadStaller::~FScopedRHIThreadStaller()
{
	if (Immed)
	{
		Immed->UnStallRHIThread();
	}
}

namespace PipelineStateCache
{
	/* Evicts unused state entries based on r.pso.evictiontime time. Called in RHICommandList::BeginFrame */
	extern RHI_API void FlushResources();
}

inline void FRHIComputeCommandList::SubmitCommandsHint()
{
	if (IsImmediate())
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static_cast<FRHICommandListImmediate&>(*this).SubmitCommandsHint();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

// Helper class for traversing a FRHICommandList
class FRHICommandListIterator
{
public:
	FRHICommandListIterator(FRHICommandListBase& CmdList)
	{
		CmdPtr = CmdList.Root;
#if DO_CHECK
		NumCommands = 0;
		CmdListNumCommands = CmdList.NumCommands;
#endif
	}
	~FRHICommandListIterator()
	{
#if DO_CHECK
		checkf(CmdListNumCommands == NumCommands, TEXT("Missed %d Commands!"), CmdListNumCommands - NumCommands);
#endif
	}

	inline bool HasCommandsLeft() const
	{
		return !!CmdPtr;
	}

	inline FRHICommandBase* NextCommand()
	{
		FRHICommandBase* RHICmd = CmdPtr;
		CmdPtr = RHICmd->Next;
#if DO_CHECK
		NumCommands++;
#endif
		return RHICmd;
	}

private:
	FRHICommandBase* CmdPtr;

#if DO_CHECK
	uint32 NumCommands;
	uint32 CmdListNumCommands;
#endif
};

inline FRHICommandListScopedFence::FRHICommandListScopedFence(FRHICommandListBase& RHICmdList)
    : RHICmdList(RHICmdList)
    , Previous(RHICmdList.PersistentState.CurrentFenceScope)
{
    RHICmdList.PersistentState.CurrentFenceScope = this;
}

inline FRHICommandListScopedFence::~FRHICommandListScopedFence()
{
    if (bFenceRequested)
    {
        RHICmdList.PersistentState.CurrentFenceScope = nullptr;
        RHICmdList.RHIThreadFence(true);
    }

    RHICmdList.PersistentState.CurrentFenceScope = Previous;
}

inline FRHICommandListScopedPipelineGuard::FRHICommandListScopedPipelineGuard(FRHICommandListBase& RHICmdList)
	: RHICmdList(RHICmdList)
{
	if (RHICmdList.GetPipeline() == ERHIPipeline::None)
	{
		RHICmdList.SwitchPipeline(ERHIPipeline::Graphics);
		bPipelineSet = true;
	}
}

inline FRHICommandListScopedPipelineGuard::~FRHICommandListScopedPipelineGuard()
{
	if (bPipelineSet)
	{
		RHICmdList.SwitchPipeline(ERHIPipeline::None);
	}
}

inline FRHICommandListScopedAllowExtraTransitions::FRHICommandListScopedAllowExtraTransitions(FRHICommandListBase& RHICmdList, bool bInAllowExtraTransitions)
	: RHICmdList(RHICmdList)
{
	bAllowExtraTransitions = RHICmdList.SetAllowExtraTransitions(bInAllowExtraTransitions);
}

inline FRHICommandListScopedAllowExtraTransitions::~FRHICommandListScopedAllowExtraTransitions()
{
	RHICmdList.SetAllowExtraTransitions(bAllowExtraTransitions);
}

inline FRHIResourceReplaceBatcher::~FRHIResourceReplaceBatcher()
{
	RHICmdList.ReplaceResources(MoveTemp(Infos));
}

#if WITH_RHI_BREADCRUMBS

	namespace UE::RHI::Breadcrumbs::Private
	{
		inline FRHIComputeCommandList& GetRHICmdList(FRHIComputeCommandList& RHICmdList) { return RHICmdList; }
		inline FRHIComputeCommandList& GetRHICmdList(IRHIComputeContext    & RHIContext) { return static_cast<FRHIComputeCommandList&>(RHIContext.GetExecutingCommandList()); }

		inline FString GetSafeBreadcrumbPath(auto&& RHICmdList_Or_RHIContext)
		{
			FRHICommandListBase& RHICmdList = GetRHICmdList(RHICmdList_Or_RHIContext);
			FRHIBreadcrumbNode* Node = RHICmdList.GetCurrentBreadcrumbRef();
			return Node ? Node->GetFullPath() : TEXT("NoBreadcrumb");
		}
	}

	template <typename TDesc, typename... TValues>
	inline FRHIBreadcrumbNode* FRHIBreadcrumbAllocator::AllocBreadcrumb(TRHIBreadcrumbInitializer<TDesc, TValues...> const& Args)
	{
		TDesc const* Desc = std::get<0>(Args);
		if (!Desc)
		{
			return nullptr;
		}

		UE::RHI::GPUProfiler::FGPUStat* GPUStat = std::get<2>(Args);

		return std::apply([&](auto&&... Values)
		{
			using TType = UE::RHI::Breadcrumbs::Private::TRHIBreadcrumb<TDesc>;
			static_assert(std::is_trivially_destructible<TType>::value, "Only trivially destructable types may be used with the RHI breadcrumb allocator.");

			void* Addr = Inner.Alloc(sizeof(TType), alignof(TType));
			return new (Addr) TType(*this, *Desc, GPUStat, std::forward<decltype(Values)>(Values)...);
		}, std::get<1>(Args));
	}

	template <typename TDesc, typename... TValues>
	inline FRHIBreadcrumbScope::FRHIBreadcrumbScope(FRHIComputeCommandList& RHICmdList, TRHIBreadcrumbInitializer<TDesc, TValues...>&& Args)
		: FRHIBreadcrumbScope(RHICmdList, RHICmdList.GetBreadcrumbAllocator().AllocBreadcrumb(Args))
	{}

	inline FRHIBreadcrumbScope::FRHIBreadcrumbScope(FRHIComputeCommandList& RHICmdList, FRHIBreadcrumbNode* Node)
		: RHICmdList(RHICmdList)
		, Node(Node)
	{
		if (Node)
		{
			Node->SetParent(RHICmdList.PersistentState.LocalBreadcrumb);
			RHICmdList.BeginBreadcrumbCPU(Node, true);

			for (ERHIPipeline Pipeline : MakeFlagsRange(RHICmdList.GetPipelines()))
			{
				RHICmdList.BeginBreadcrumbGPU(Node, Pipeline);
			}
		}
	}

	inline FRHIBreadcrumbScope::~FRHIBreadcrumbScope()
	{
		if (Node)
		{
			for (ERHIPipeline Pipeline : MakeFlagsRange(RHICmdList.GetPipelines()))
			{
				RHICmdList.EndBreadcrumbGPU(Node, Pipeline);
			}

			RHICmdList.EndBreadcrumbCPU(Node, true);
		}
	}

	template <typename TDesc, typename... TValues>
	inline FRHIBreadcrumbEventManual::FRHIBreadcrumbEventManual(FRHIComputeCommandList& RHICmdList, TRHIBreadcrumbInitializer<TDesc, TValues...>&& Args)
		: Node(RHICmdList.GetBreadcrumbAllocator().AllocBreadcrumb(Args))
	#if DO_CHECK
		, Pipeline(RHICmdList.GetPipeline())
		, ThreadId(FPlatformTLS::GetCurrentThreadId())
	#endif
	{
	#if DO_CHECK
		check(Pipeline != ERHIPipeline::None);
	#endif

		Node->SetParent(RHICmdList.PersistentState.LocalBreadcrumb);
		RHICmdList.BeginBreadcrumbCPU(Node.Get(), true);
		RHICmdList.BeginBreadcrumbGPU(Node.Get(), RHICmdList.GetPipeline());
	}

	inline void FRHIBreadcrumbEventManual::End(FRHIComputeCommandList& RHICmdList)
	{
		checkf(Node, TEXT("Manual breadcrumb was already ended."));
	#if DO_CHECK
		checkf(Pipeline == RHICmdList.GetPipeline(), TEXT("Manual breadcrumb was started and ended on different pipelines. Start: %s, End: %s")
			, *GetRHIPipelineName(Pipeline)
			, *GetRHIPipelineName(RHICmdList.GetPipeline())
		);

		checkf(ThreadId == FPlatformTLS::GetCurrentThreadId(), TEXT("Manual breadcrumbs must be started and ended on the same thread."));
	#endif

		RHICmdList.EndBreadcrumbGPU(Node.Get(), RHICmdList.GetPipeline());
		RHICmdList.EndBreadcrumbCPU(Node.Get(), true);
		Node = {};
	}

	inline FRHIBreadcrumbEventManual::~FRHIBreadcrumbEventManual()
	{
		checkf(!Node, TEXT("Manual breadcrumb was destructed before it was ended."));
	}

#endif // WITH_RHI_BREADCRUMBS

template <typename RHICmdListType, typename LAMBDA>
inline void TRHILambdaCommandMultiPipe<RHICmdListType, LAMBDA>::ExecuteAndDestruct(FRHICommandListBase& CmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(Name, RHICommandsChannel);

	FRHIContextArray Contexts { InPlace, nullptr };
	for (ERHIPipeline Pipeline : MakeFlagsRange(Pipelines))
	{
		Contexts[Pipeline] = CmdList.Contexts[Pipeline];
		check(Contexts[Pipeline]);
	}

	// Static cast to enforce const type in lambda args
	Lambda(static_cast<FRHIContextArray const&>(Contexts));
	Lambda.~LAMBDA();
}