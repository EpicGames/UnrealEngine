// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHICommandListCommandExecutes.inl: RHI Command List execute functions.
=============================================================================*/

#if !defined(INTERNAL_DECORATOR)
	#define INTERNAL_DECORATOR(Method) CmdList.GetContext().Method
#endif

//for functions where the signatures do not match between gfx and compute commandlists
#if !defined(INTERNAL_DECORATOR_COMPUTE)
#define INTERNAL_DECORATOR_COMPUTE(Method) CmdList.GetComputeContext().Method
#endif

class FRHICommandListBase;
class IRHIComputeContext;
struct FComputedBSS;
struct FComputedUniformBuffer;
struct FMemory;
struct FRHICommandBeginRenderQuery;
struct FRHICommandBuildLocalBoundShaderState;
struct FRHICommandBuildLocalGraphicsPipelineState;
struct FRHICommandBuildLocalUniformBuffer;
struct FRHICommandDrawIndexedPrimitive;
struct FRHICommandDrawIndexedPrimitiveIndirect;
struct FRHICommandDrawPrimitive;
struct FRHICommandDrawPrimitiveIndirect;
struct FRHICommandMultiDrawPrimitiveIndirect;
struct FRHICommandSetDepthBounds;
struct FRHICommandEndOcclusionQueryBatch;
struct FRHICommandEndRenderQuery;
struct FRHICommandSetBlendFactor;
struct FRHICommandSetBoundShaderState;
struct FRHICommandSetLocalGraphicsPipelineState;
struct FRHICommandSetRasterizerState;
struct FRHICommandSetRenderTargets;
struct FRHICommandSetRenderTargetsAndClear;
struct FRHICommandSetScissorRect;
struct FRHICommandSetStencilRef;
struct FRHICommandSetStereoViewport;
struct FRHICommandSetStreamSource;
struct FRHICommandSetViewport;
struct FRHICommandTransitionTextures;
struct FRHICommandTransitionTexturesPipeline;
struct FRHICommandTransitionTexturesDepth;
struct FRHICommandTransitionTexturesArray;
struct FRHICommandClearBindings;
struct FRHICommandRayTraceDispatch;
struct FRHICommandSetBindings;

template <typename TRHIShader> struct FRHICommandSetLocalUniformBuffer;

#if WITH_MGPU
void FRHICommandSetGPUMask::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetGPUMask);

	// Update the RHICmdList copy of the current mask
	CmdList.PersistentState.CurrentGPUMask = GPUMask;

	// Apply the new mask to all contexts owned by this command list.
	for (IRHIComputeContext* Context : CmdList.Contexts)
	{
		if (Context)
		{
			Context->RHISetGPUMask(GPUMask);
		}
	}
}

void FRHICommandTransferResources::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(TransferResources);
	INTERNAL_DECORATOR_COMPUTE(RHITransferResources)(Params);
}

void FRHICommandTransferResourceSignal::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(TransferResourceSignal);
	INTERNAL_DECORATOR_COMPUTE(RHITransferResourceSignal)(FenceDatas, SrcGPUMask);
}

void FRHICommandTransferResourceWait::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(TransferResourceWait);
	INTERNAL_DECORATOR_COMPUTE(RHITransferResourceWait)(FenceDatas);
}

void FRHICommandCrossGPUTransfer::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(CrossGPUTransfer);
	INTERNAL_DECORATOR_COMPUTE(RHICrossGPUTransfer)(Params, PreTransfer, PostTransfer);
}

void FRHICommandCrossGPUTransferSignal::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(CrossGPUTransferSignal);
	INTERNAL_DECORATOR_COMPUTE(RHICrossGPUTransferSignal)(Params, PreTransfer);
}

void FRHICommandCrossGPUTransferWait::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(CrossGPUTransferWait);
	INTERNAL_DECORATOR_COMPUTE(RHICrossGPUTransferWait)(SyncPoints);
}

#endif // WITH_MGPU

void FRHICommandSetStencilRef::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetStencilRef);
	INTERNAL_DECORATOR(RHISetStencilRef)(StencilRef);
}

template<> RHI_EXECUTE_API void FRHICommandSetShaderParameters<FRHIComputeShader>::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetShaderParameters);
	INTERNAL_DECORATOR_COMPUTE(RHISetShaderParameters)(Shader, ParametersData, Parameters, ResourceParameters, BindlessParameters);
}

template<> RHI_EXECUTE_API void FRHICommandSetShaderParameters<FRHIGraphicsShader>::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetShaderParameters);
	INTERNAL_DECORATOR(RHISetShaderParameters)(Shader, ParametersData, Parameters, ResourceParameters, BindlessParameters);
}

template<> RHI_EXECUTE_API void FRHICommandSetShaderUnbinds<FRHIComputeShader>::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetShaderUnbinds);
	INTERNAL_DECORATOR_COMPUTE(RHISetShaderUnbinds)(Shader, Unbinds);
}

template<> RHI_EXECUTE_API void FRHICommandSetShaderUnbinds<FRHIGraphicsShader>::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetShaderUnbinds);
	INTERNAL_DECORATOR(RHISetShaderUnbinds)(Shader, Unbinds);
}

void FRHICommandDrawPrimitive::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(DrawPrimitive);
	INTERNAL_DECORATOR(RHIDrawPrimitive)(BaseVertexIndex, NumPrimitives, NumInstances);
}

void FRHICommandDrawIndexedPrimitive::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(DrawIndexedPrimitive);
	INTERNAL_DECORATOR(RHIDrawIndexedPrimitive)(IndexBuffer, BaseVertexIndex, FirstInstance, NumVertices, StartIndex, NumPrimitives, NumInstances);
}

void FRHICommandSetBlendFactor::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetBlendFactor);
	INTERNAL_DECORATOR(RHISetBlendFactor)(BlendFactor);
}

void FRHICommandSetStreamSource::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetStreamSource);
	INTERNAL_DECORATOR(RHISetStreamSource)(StreamIndex, VertexBuffer, Offset);
}

void FRHICommandSetViewport::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetViewport);
	INTERNAL_DECORATOR(RHISetViewport)(MinX, MinY, MinZ, MaxX, MaxY, MaxZ);
}

void FRHICommandSetStereoViewport::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetStereoViewport);
	INTERNAL_DECORATOR(RHISetStereoViewport)(LeftMinX, RightMinX, LeftMinY, RightMinY, MinZ, LeftMaxX, RightMaxX, LeftMaxY, RightMaxY, MaxZ);
}

void FRHICommandSetScissorRect::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetScissorRect);
	INTERNAL_DECORATOR(RHISetScissorRect)(bEnable, MinX, MinY, MaxX, MaxY);
}

void FRHICommandBeginRenderPass::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(BeginRenderPass);
	INTERNAL_DECORATOR(RHIBeginRenderPass)(Info, Name);
}

void FRHICommandEndRenderPass::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(EndRenderPass);
	INTERNAL_DECORATOR(RHIEndRenderPass)();
}

void FRHICommandNextSubpass::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(NextSubpass);
	INTERNAL_DECORATOR(RHINextSubpass)();
}

void FRHICommandSetComputePipelineState::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetComputePipelineState);
	extern RHI_API FRHIComputePipelineState* ExecuteSetComputePipelineState(FComputePipelineState* ComputePipelineState);
	FRHIComputePipelineState* RHIComputePipelineState = ExecuteSetComputePipelineState(ComputePipelineState);
	INTERNAL_DECORATOR_COMPUTE(RHISetComputePipelineState)(RHIComputePipelineState);
}

void FRHICommandSetGraphicsPipelineState::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetGraphicsPipelineState);
	extern FRHIGraphicsPipelineState* ExecuteSetGraphicsPipelineState(FGraphicsPipelineState* GraphicsPipelineState);
	FRHIGraphicsPipelineState* RHIGraphicsPipelineState = ExecuteSetGraphicsPipelineState(GraphicsPipelineState);
	INTERNAL_DECORATOR(RHISetGraphicsPipelineState)(RHIGraphicsPipelineState, StencilRef, bApplyAdditionalState);
}

#if PLATFORM_USE_FALLBACK_PSO
void FRHICommandSetGraphicsPipelineStateFromInitializer::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetGraphicsPipelineStateFromInitializer);
	INTERNAL_DECORATOR(RHISetGraphicsPipelineState)(PsoInit, StencilRef, bApplyAdditionalState);
}
#endif

void FRHICommandDispatchComputeShader::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(DispatchComputeShader);
	INTERNAL_DECORATOR_COMPUTE(RHIDispatchComputeShader)(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
}

void FRHICommandDispatchIndirectComputeShader::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(DispatchIndirectComputeShader);
	INTERNAL_DECORATOR_COMPUTE(RHIDispatchIndirectComputeShader)(ArgumentBuffer, ArgumentOffset);
}

void FRHICommandDispatchComputeShaderBundle::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(DispatchShaderBundle);
	extern RHI_API FRHIComputePipelineState* ExecuteSetComputePipelineState(FComputePipelineState* ComputePipelineState);
	for (int32 DispatchIndex = 0; DispatchIndex < Dispatches.Num(); ++DispatchIndex)
	{
		FRHIShaderBundleComputeDispatch& Dispatch = Dispatches[DispatchIndex];
#if !PLATFORM_USE_FALLBACK_PSO
		if (Dispatch.RecordIndex != ~uint32(0u) && Dispatch.PipelineState)
		{
			Dispatch.RHIPipeline = ExecuteSetComputePipelineState(Dispatch.PipelineState);
		}
#endif
	}
	INTERNAL_DECORATOR_COMPUTE(RHIDispatchComputeShaderBundle)(ShaderBundle, RecordArgBuffer, SharedBindlessParameters, Dispatches, Method);
}

void FRHICommandDispatchGraphicsShaderBundle::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(DispatchShaderBundle);
	extern RHI_API FRHIGraphicsPipelineState* ExecuteSetGraphicsPipelineState(FGraphicsPipelineState* GraphicsPipelineState);
	for (int32 DispatchIndex = 0; DispatchIndex < Dispatches.Num(); ++DispatchIndex)
	{
		FRHIShaderBundleGraphicsDispatch& Dispatch = Dispatches[DispatchIndex];
#if !PLATFORM_USE_FALLBACK_PSO
		if (Dispatch.RecordIndex != ~uint32(0u) && Dispatch.PipelineState)
		{
			Dispatch.RHIPipeline = ExecuteSetGraphicsPipelineState(Dispatch.PipelineState);
		}
#endif
	}
	INTERNAL_DECORATOR_COMPUTE(RHIDispatchGraphicsShaderBundle)(ShaderBundle, RecordArgBuffer, BundleState, SharedBindlessParameters, Dispatches, Method);
}

void FRHICommandSetShaderRootConstants::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetShaderRootConstants);
	INTERNAL_DECORATOR_COMPUTE(RHISetShaderRootConstants)(Constants);
}

#if ENABLE_RHI_VALIDATION
void FRHICommandBeginUAVOverlap::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(BeginUAVOverlap);
	INTERNAL_DECORATOR_COMPUTE(RHIBeginUAVOverlap)();
}

void FRHICommandEndUAVOverlap::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(EndUAVOverlap);
	INTERNAL_DECORATOR_COMPUTE(RHIEndUAVOverlap)();
}

void FRHICommandBeginSpecificUAVOverlap::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(BeginSpecificUAVOverlap);
	INTERNAL_DECORATOR_COMPUTE(RHIBeginUAVOverlap)(UAVs);
}

void FRHICommandEndSpecificUAVOverlap::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(EndSpecificUAVOverlap);
	INTERNAL_DECORATOR_COMPUTE(RHIEndUAVOverlap)(UAVs);
}
#endif // ENABLE_RHI_VALIDATION

void FRHICommandDrawPrimitiveIndirect::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(DrawPrimitiveIndirect);
	INTERNAL_DECORATOR(RHIDrawPrimitiveIndirect)(ArgumentBuffer, ArgumentOffset);
}

void FRHICommandDrawIndexedPrimitiveIndirect::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(DrawIndexedPrimitiveIndirect);
	INTERNAL_DECORATOR(RHIDrawIndexedPrimitiveIndirect)(IndexBuffer, ArgumentsBuffer, ArgumentOffset);
}

void FRHICommandMultiDrawIndexedPrimitiveIndirect::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(MultiDrawIndexedPrimitiveIndirect);
	INTERNAL_DECORATOR(RHIMultiDrawIndexedPrimitiveIndirect)(IndexBuffer, ArgumentBuffer, ArgumentOffset, CountBuffer, CountBufferOffset, MaxDrawArguments);
}

void FRHICommandDispatchMeshShader::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(DispatchMeshShader);
	INTERNAL_DECORATOR(RHIDispatchMeshShader)(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
}

void FRHICommandDispatchIndirectMeshShader::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(DispatchIndirectMeshShader);
	INTERNAL_DECORATOR(RHIDispatchIndirectMeshShader)(ArgumentBuffer, ArgumentOffset);
}

void FRHICommandSetShadingRate::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetShadingRate);
	INTERNAL_DECORATOR(RHISetShadingRate)(ShadingRate, Combiner);
}

void FRHICommandSetDepthBounds::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(EnableDepthBoundsTest);
	INTERNAL_DECORATOR(RHISetDepthBounds)(MinDepth, MaxDepth);
}

void FRHIGpuHangCommandListCorruption::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(GpuHangCommandListCorruption);
	INTERNAL_DECORATOR(RHIGpuHangCommandListCorruption)();
}

void FRHICommandClearUAVFloat::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(ClearUAV);
	INTERNAL_DECORATOR_COMPUTE(RHIClearUAVFloat)(UnorderedAccessViewRHI, Values);
}

void FRHICommandClearUAVUint::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(ClearUAV);
	INTERNAL_DECORATOR_COMPUTE(RHIClearUAVUint)(UnorderedAccessViewRHI, Values);
}

void FRHICommandCopyTexture::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(CopyTexture);
	INTERNAL_DECORATOR(RHICopyTexture)(SourceTexture, DestTexture, CopyInfo);
}

void FRHICommandResummarizeHTile::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(ResummarizeHTile);
	INTERNAL_DECORATOR(RHIResummarizeHTile)(DepthTexture);
}

void FRHICommandBeginTransitions::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(BeginTransitions);

	INTERNAL_DECORATOR_COMPUTE(RHIBeginTransitions)(Transitions);

	for (const FRHITransition* Transition : Transitions)
	{
		Transition->MarkBegin(CmdList.GetPipeline());
	}
}

void FRHICommandEndTransitions::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(EndTransitions);

	INTERNAL_DECORATOR_COMPUTE(RHIEndTransitions)(Transitions);

	for (const FRHITransition* Transition : Transitions)
	{
		Transition->MarkEnd(CmdList.GetPipeline());
	}
}

void FRHICommandResourceTransition::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(ResourceTransition);

	INTERNAL_DECORATOR_COMPUTE(RHIBeginTransitions)(MakeArrayView((const FRHITransition**)&Transition, 1));
	INTERNAL_DECORATOR_COMPUTE(RHIEndTransitions)(MakeArrayView((const FRHITransition**)&Transition, 1));

	Transition->Cleanup();
}

void FRHICommandSetTrackedAccess::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetTrackedAccess);
	for (const FRHITrackedAccessInfo& Info : Infos)
	{
		INTERNAL_DECORATOR_COMPUTE(SetTrackedAccess)(Info);
	}
}

void FRHICommandSetAsyncComputeBudget::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetAsyncComputeBudget);
	INTERNAL_DECORATOR_COMPUTE(RHISetAsyncComputeBudget)(Budget);
}

void FRHICommandSetComputeBudget::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetComputeBudget);
	INTERNAL_DECORATOR(RHISetComputeBudget)(Budget);
}

void FRHICommandCopyToStagingBuffer::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(EnqueueStagedRead);
	INTERNAL_DECORATOR_COMPUTE(RHICopyToStagingBuffer)(SourceBuffer, DestinationStagingBuffer, Offset, NumBytes);
}

void FRHICommandWriteGPUFence::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(WriteGPUFence);
	INTERNAL_DECORATOR_COMPUTE(RHIWriteGPUFence)(Fence);
	if (Fence)
	{
		Fence->NumPendingWriteCommands.Decrement();
	}
}

void FRHICommandSetStaticUniformBuffers::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetStaticUniformBuffers);
	INTERNAL_DECORATOR_COMPUTE(RHISetStaticUniformBuffers)(UniformBuffers);
}

void FRHICommandSetStaticUniformBuffer::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetStaticUniformBuffer);
	INTERNAL_DECORATOR_COMPUTE(RHISetStaticUniformBuffer)(Slot, Buffer);
}

void FRHICommandSetUniformBufferDynamicOffset::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetUniformBufferDynamicOffset);
	INTERNAL_DECORATOR(RHISetUniformBufferDynamicOffset)(Slot, Offset);
}

void FRHICommandBeginRenderQuery::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(BeginRenderQuery);
	INTERNAL_DECORATOR(RHIBeginRenderQuery)(RenderQuery);
}

void FRHICommandEndRenderQuery::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(EndRenderQuery);
	INTERNAL_DECORATOR(RHIEndRenderQuery)(RenderQuery);
}

void FRHICommandPostExternalCommandsReset::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(PostExternalCommandsReset);
	INTERNAL_DECORATOR(RHIPostExternalCommandsReset)();
}

void FRHICommandCopyBufferRegion::Execute(FRHICommandListBase& CmdList)
{
	INTERNAL_DECORATOR(RHICopyBufferRegion)(DestBuffer, DstOffset, SourceBuffer, SrcOffset, NumBytes);
}

void FRHICommandBindAccelerationStructureMemory::Execute(FRHICommandListBase& CmdList)
{
	INTERNAL_DECORATOR_COMPUTE(RHIBindAccelerationStructureMemory)(Scene, Buffer, BufferOffset);
}

void FRHICommandBuildSceneAccelerationStructures::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(BuildAccelerationStructure);
	INTERNAL_DECORATOR_COMPUTE(RHIBuildAccelerationStructures)(Params);
}

void FRHICommandCommitShaderBindingTable::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(CommitShaderBindingTable);
	INTERNAL_DECORATOR(RHICommitShaderBindingTable)(SBT, InlineBindingDataBuffer);
}

void FRHICommandBuildAccelerationStructures::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(BuildAccelerationStructure);
	INTERNAL_DECORATOR_COMPUTE(RHIBuildAccelerationStructures)(Params, ScratchBufferRange);
}

void FRHICommandExecuteMultiIndirectClusterOperation::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(ExecuteMultiIndirectClusterOperation);
	INTERNAL_DECORATOR_COMPUTE(RHIExecuteMultiIndirectClusterOperation)(Params);
}

void FRHICommandRayTraceDispatch::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(RayTraceDispatch);
	extern RHI_API FRHIRayTracingPipelineState* GetRHIRayTracingPipelineState(FRayTracingPipelineState*);
	
	FRHIRayTracingPipelineState* RayTracingPipelineState = GetRHIRayTracingPipelineState(Pipeline);

	if (ArgumentBuffer)
	{
		INTERNAL_DECORATOR_COMPUTE(RHIRayTraceDispatchIndirect)(RayTracingPipelineState, RayGenShader, SBT, GlobalResourceBindings, ArgumentBuffer, ArgumentOffset);
	}
	else
	{
		INTERNAL_DECORATOR_COMPUTE(RHIRayTraceDispatch)(RayTracingPipelineState, RayGenShader, SBT, GlobalResourceBindings, Width, Height);
	}
}

void FRHICommandSetBindingsOnShaderBindingTable::Execute(FRHICommandListBase& CmdList)
{
	RHISTAT(SetRayTracingHitGroup);
	extern RHI_API FRHIRayTracingPipelineState* GetRHIRayTracingPipelineState(FRayTracingPipelineState*);
	
	FRHIRayTracingPipelineState* RayTracingPipelineState = GetRHIRayTracingPipelineState(Pipeline);

	INTERNAL_DECORATOR(RHISetBindingsOnShaderBindingTable)(SBT, RayTracingPipelineState, NumBindings, Bindings, BindingType);
}
