// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHIContext.h: Interface for RHI Contexts
=============================================================================*/

#pragma once

#include "Misc/AssertionMacros.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/Box2D.h"
#include "Math/PerspectiveMatrix.h"
#include "Math/TranslationMatrix.h"
#include "Math/ScaleMatrix.h"
#include "Math/Float16Color.h"
#include "Modules/ModuleInterface.h"
#include "RHIBreadcrumbs.h"
#include "RHIResources.h"
#include "RHIShaderParameters.h"
#include "GPUProfiler.h"

class FRHIDepthRenderTargetView;
class FRHIRenderTargetView;
class FRHISetRenderTargetsInfo;
class FRHIShaderBindingLayout;
struct FViewportBounds;
struct FRayTracingShaderBindings;
struct FRayTracingGeometrySegment;
struct FRayTracingGeometryBuildParams;
struct FRayTracingClusterOperationParams;
struct FRayTracingSceneBuildParams;
struct FRayTracingLocalShaderBindings;
enum class ERayTracingBindingType : uint8;
enum class EAsyncComputeBudget;

struct FRHIBufferRange;
struct FRHIPerCategoryDrawStats;
struct FRHICopyTextureInfo;


#define VALIDATE_UNIFORM_BUFFER_STATIC_BINDINGS (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

/** A list of static uniform buffer bindings. */
class FUniformBufferStaticBindings
{
public:
	FUniformBufferStaticBindings() = default;

	template <typename... TArgs>
	FUniformBufferStaticBindings(TArgs... Args)
	{
		std::initializer_list<FRHIUniformBuffer*> InitList = { Args... };

		for (FRHIUniformBuffer* Buffer : InitList)
		{
			AddUniformBuffer(Buffer);
		}
	}

	RHI_API FUniformBufferStaticBindings(const FRHIShaderBindingLayout* InSRTDesc);

	RHI_API void AddUniformBuffer(FRHIUniformBuffer* UniformBuffer);

	inline void TryAddUniformBuffer(FRHIUniformBuffer* UniformBuffer)
	{
		if (UniformBuffer)
		{
			AddUniformBuffer(UniformBuffer);
		}
	}

	int32 GetUniformBufferCount() const
	{
		return UniformBuffers.Num();
	}

	FRHIUniformBuffer* GetUniformBuffer(int32 Index) const
	{
		return UniformBuffers[Index];
	}

	FUniformBufferStaticSlot GetSlot(int32 Index) const
	{
		return Slots[Index];
	}

	int32 GetSlotCount() const
	{
		return SlotCount;
	}

	const FRHIShaderBindingLayout* GetShaderBindingLayout() const
	{
		return ShaderBindingLayout;
	}

	RHI_API void Bind(TArray<FRHIUniformBuffer*>& Bindings) const;

private:
	static const uint32 InlineUniformBufferCount = 8;
	TArray<FUniformBufferStaticSlot, TInlineAllocator<InlineUniformBufferCount>> Slots;
	TArray<FRHIUniformBuffer*, TInlineAllocator<InlineUniformBufferCount>> UniformBuffers;
	int32 SlotCount = 0;

	// Shader binding layout used during shader generation to validate runtime bindings and know where uniform buffers need to be bound
	const FRHIShaderBindingLayout* ShaderBindingLayout = nullptr;
};

struct FTransferResourceFenceData
{
	TStaticArray<void*, MAX_NUM_GPUS> SyncPoints;
	FRHIGPUMask Mask;

	FTransferResourceFenceData()
		: SyncPoints(InPlace, nullptr)
	{}
};

struct FCrossGPUTransferFence
{
	uint32 SignalGPUIndex = 0;
	uint32 WaitGPUIndex = 0;
	void* SyncPoint = nullptr;

	FCrossGPUTransferFence() = default;
};

inline FTransferResourceFenceData* RHICreateTransferResourceFenceData()
{
#if WITH_MGPU
	return new FTransferResourceFenceData;
#else
	return nullptr;
#endif
}

inline FCrossGPUTransferFence* RHICreateCrossGPUTransferFence()
{
#if WITH_MGPU
	return new FCrossGPUTransferFence;
#else
	return nullptr;
#endif
}

/** Parameters for RHITransferResources, used to copy memory between GPUs */
struct FTransferResourceParams
{
	FTransferResourceParams() {}

	FTransferResourceParams(FRHITexture* InTexture, const FIntRect& InRect, uint32 InSrcGPUIndex, uint32 InDestGPUIndex, bool InPullData, bool InLockStepGPUs)
		: Texture(InTexture), Buffer(nullptr), Min(InRect.Min.X, InRect.Min.Y, 0), Max(InRect.Max.X, InRect.Max.Y, 1), SrcGPUIndex(InSrcGPUIndex), DestGPUIndex(InDestGPUIndex), bPullData(InPullData), bLockStepGPUs(InLockStepGPUs)
	{
		check(InTexture);
	}

	FTransferResourceParams(FRHITexture* InTexture, uint32 InSrcGPUIndex, uint32 InDestGPUIndex, bool InPullData, bool InLockStepGPUs)
		: Texture(InTexture), Buffer(nullptr), Min(0, 0, 0), Max(0, 0, 0), SrcGPUIndex(InSrcGPUIndex), DestGPUIndex(InDestGPUIndex), bPullData(InPullData), bLockStepGPUs(InLockStepGPUs)
	{
		check(InTexture);
	}

	FTransferResourceParams(FRHIBuffer* InBuffer, uint32 InSrcGPUIndex, uint32 InDestGPUIndex, bool InPullData, bool InLockStepGPUs)
		: Texture(nullptr), Buffer(InBuffer), Min(0, 0, 0), Max(0, 0, 0), SrcGPUIndex(InSrcGPUIndex), DestGPUIndex(InDestGPUIndex), bPullData(InPullData), bLockStepGPUs(InLockStepGPUs)
	{
		check(InBuffer);
	}

	void SetRect(const FIntRect& Rect)
	{
		// Do nothing if this isn't a texture
		if (!Texture)
		{
			return;
		}

		// Reset the extents to default (zero), if the rect is empty or covers the full extent of the texture.  Forces a more efficient full resource copy.
		if (Rect.IsEmpty() || (Rect.Min.X == 0 && Rect.Min.Y == 0 && Rect.Max.X == Texture->GetDesc().Extent.X && Rect.Max.Y == Texture->GetDesc().Extent.Y))
		{
			Min = FIntVector3(0);
			Max = FIntVector3(0);
			return;
		}

		// Initialize the extent
		Min = FIntVector3(Rect.Min.X, Rect.Min.Y, 0);
		Max = FIntVector3(Rect.Max.X, Rect.Max.Y, 1);
	}

	// The texture which must be must be allocated on both GPUs 
	FTextureRHIRef Texture;
	// Or alternately, a buffer that's allocated on both GPUs
	FBufferRHIRef Buffer;
	// The min rect of the texture region to copy
	FIntVector Min;
	// The max rect of the texture region to copy
	FIntVector Max;
	// The GPU index where the data will be read from.
	uint32 SrcGPUIndex;
	// The GPU index where the data will be written to.
	uint32 DestGPUIndex;
	// Whether the data is read by the dest GPU, or written by the src GPU (not allowed if the texture is a backbuffer)
	bool bPullData = true;
	// Whether the GPUs must handshake before and after the transfer. Required if the texture rect is being written to in several render passes.
	// Otherwise, minimal synchronization will be used.
	bool bLockStepGPUs = true;
	/**
	  * Optional pointer where fence data can be written if you want to delay waiting on the GPU fence for a resource transfer.
	  * Should be created via "RHICreateTransferResourceFenceData", and must later be consumed via "TransferResourceWait" command.
	  * Note that it is valid to consume the fence data, even if you don't end up implementing a transfer that uses it -- it will
	  * behave as a nop in that case.  That can simplify cases where the transfer may be conditional, and you don't want to worry
	  * about whether it occurred or not, but need to reserve the possibility.
	  */
	FTransferResourceFenceData* DelayedFence = nullptr;
	/**
	 * Optional pointer to a fence to wait on before starting the transfer.  Useful if a resource may be in use on the destination
	 * GPU, and you need to wait until it's no longer in use before copying to it from the current GPU.  Fences are created via
	 * "RHICreateTransferResourceFenceData", then signaled via "TransferResourceSignal" command, before being added to one of the
	 * transfers in a batch that's dependent on the signal.
	 */
	FTransferResourceFenceData* PreTransferFence = nullptr;
};

//
// Type representing a finalized platform GPU command list, which can be submitted to the GPU via RHISubmitCommandLists().
// This type is intended only for use by RHI command list management. Platform RHIs provide the implementation.
// Also contains RHI breadcrumb allocators and ranges that platform RHIs must use if they implement GPU crash debugging.
//
class IRHIPlatformCommandList
{
	// Prevent copying
	IRHIPlatformCommandList(IRHIPlatformCommandList const&) = delete;
	IRHIPlatformCommandList& operator = (IRHIPlatformCommandList const&) = delete;

protected:
	// Allow moving
	IRHIPlatformCommandList(IRHIPlatformCommandList&&) = default;
	IRHIPlatformCommandList& operator = (IRHIPlatformCommandList&&) = default;

	// This type is only usable by derived types (platform RHI implementations)
	IRHIPlatformCommandList() = default;
	~IRHIPlatformCommandList() = default;

public:
#if WITH_RHI_BREADCRUMBS
	FRHIBreadcrumbAllocatorArray BreadcrumbAllocators {};
	FRHIBreadcrumbRange BreadcrumbRange {};
#endif
};

/** Context that is capable of doing Compute work.  Can be async or compute on the gfx pipe. */
class IRHIComputeContext
{
public:
	virtual ~IRHIComputeContext() = default;

	virtual ERHIPipeline GetPipeline() const
	{
		return ERHIPipeline::AsyncCompute;
	}

	virtual void RHISetComputePipelineState(FRHIComputePipelineState* ComputePipelineState) = 0;

	virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) = 0;

	virtual void RHIRayTraceDispatch(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		uint32 Width, uint32 Height)
	{
		checkNoEntry();
	}

	virtual void RHIRayTraceDispatchIndirect(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset)
	{
		checkNoEntry();
	}

	virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) = 0;

	virtual void RHISetAsyncComputeBudget(EAsyncComputeBudget Budget) {}
	virtual void RHISetComputeBudget(ESyncComputeBudget Budget) {}

	virtual void RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions) = 0;

	virtual void RHIEndTransitions(TArrayView<const FRHITransition*> Transitions) = 0;

	/**
	* Clears a UAV to the multi-channel floating point value provided. Should only be called on UAVs with a floating point format, or on structured buffers.
	* Structured buffers are treated as a regular R32_UINT buffer during the clear operation, and the Values.X component is copied directly into the buffer without any format conversion. (Y,Z,W) of Values is ignored.
	* Typed floating point buffers undergo standard format conversion during the write operation. The conversion is determined by the format of the UAV.
	*
	* @param UnorderedAccessViewRHI		The UAV to clear.
	* @param Values						The values to clear the UAV to, one component per channel (XYZW = RGBA). Channels not supported by the UAV are ignored.
	*
	*/
	virtual void RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values) = 0;

	/**
	* Clears a UAV to the multi-component unsigned integer value provided. Should only be called on UAVs with an integer format, or on structured buffers.
	* Structured buffers are treated as a regular R32_UINT buffer during the clear operation, and the Values.X component is copied directly into the buffer without any format conversion. (Y,Z,W) of Values is ignored.
	* Typed integer buffers undergo standard format conversion during the write operation. The conversion is determined by the format of the UAV.
	*
	* @param UnorderedAccessViewRHI		The UAV to clear.
	* @param Values						The values to clear the UAV to, one component per channel (XYZW = RGBA). Channels not supported by the UAV are ignored.
	*
	*/
	virtual void RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values) = 0;

	virtual void RHISetShaderRootConstants(const FUint32Vector4& Constants)
	{
		checkNoEntry();
	}

	virtual void RHIDispatchComputeShaderBundle(
		FRHIShaderBundle* ShaderBundle,
		FRHIBuffer* RecordArgBuffer,
		TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches,
		ERHIShaderBundleMethod Method) {}

	virtual void RHIDispatchGraphicsShaderBundle(
		FRHIShaderBundle* ShaderBundle,
		FRHIBuffer* RecordArgBuffer,
		const FRHIShaderBundleGraphicsState& BundleState,
		TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleGraphicsDispatch> Dispatches,
		ERHIShaderBundleMethod Method) {}

#if ENABLE_RHI_VALIDATION
	virtual void RHIBeginUAVOverlap() {}
	virtual void RHIEndUAVOverlap() {}

	virtual void RHIBeginUAVOverlap(TConstArrayView<FRHIUnorderedAccessView*> UAVs) {}
	virtual void RHIEndUAVOverlap(TConstArrayView<FRHIUnorderedAccessView*> UAVs) {}
#endif

	virtual void RHISetShaderParameters(FRHIComputeShader* ComputeShader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) = 0;

	void RHISetBatchedShaderParameters(FRHIComputeShader* InShader, FRHIBatchedShaderParameters& InBatchedParameters)
	{
		RHISetShaderParameters(
			InShader,
			InBatchedParameters.ParametersData,
			InBatchedParameters.Parameters,
			InBatchedParameters.ResourceParameters,
			InBatchedParameters.BindlessParameters);

		InBatchedParameters.Reset();
	}

	virtual void RHISetShaderUnbinds(FRHIComputeShader* ComputeShader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
	{
		checkf(false, TEXT("RHISetShaderUnbinds called when the active RHI hasn't overridden it and GRHIGlobals.NeedsShaderUnbinds is set."));
	}

	virtual void RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers) = 0;

	virtual void RHISetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* UniformBuffer) = 0;

	virtual void RHISetUniformBufferDynamicOffset(FUniformBufferStaticSlot Slot, uint32 Offset)
	{
		/* empty default implementation */
	}

#if WITH_RHI_BREADCRUMBS
	virtual void RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) = 0;
	virtual void RHIEndBreadcrumbGPU  (FRHIBreadcrumbNode* Breadcrumb) = 0;
#endif

	/**
	 * Performs a copy of the data in 'SourceBuffer' to 'DestinationStagingBuffer.' This will occur inline on the GPU timeline. This is a mechanism to perform nonblocking readback of a buffer at a point in time.
	 * @param SourceBuffer The source vertex buffer that will be inlined copied.
	 * @param DestinationStagingBuffer The the host-visible destination buffer
	 * @param Offset The start of the data in 'SourceBuffer'
	 * @param NumBytes The number of bytes to copy out of 'SourceBuffer'
	 */
	virtual void RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* DestinationStagingBufferRHI, uint32 InOffset, uint32 InNumBytes)
	{
		check(false);
	}

	/**
	 * Write the fence in the GPU timeline. The fence can then be tested on the CPU to know if the previous GPU commands are completed.
	 * @param Fence 
	 */
	virtual void RHIWriteGPUFence(FRHIGPUFence* FenceRHI)
	{
		check(false);
	}

	virtual void RHISetGPUMask(FRHIGPUMask GPUMask)
	{
		ensure(GPUMask == FRHIGPUMask::GPU0());
	}

	virtual FRHIGPUMask RHIGetGPUMask() const
	{
		return FRHIGPUMask::GPU0();
	}

#if WITH_MGPU
	/**
	 * Synchronizes the content of a resource between two GPUs using a copy operation.
	 * @param Params - the parameters for each resource or texture region copied between GPUs.
	 */
	virtual void RHITransferResources(TConstArrayView<FTransferResourceParams> Params)
	{
		/* empty default implementation */
	}

	/*
	 * Signal where a cross GPU resource transfer can start.  Useful when the destination resource of a copy may still be in use, and
	 * the copy from the source GPUs needs to wait until the destination is finished with it.  SrcGPUMask must not overlap the current
	 * GPU mask of the context (which specifies the destination GPUs), and the number of items in the "FenceDatas" array MUST match the
	 * number of bits set in SrcGPUMask.
	 */
	virtual void RHITransferResourceSignal(TConstArrayView<FTransferResourceFenceData*> FenceDatas, FRHIGPUMask SrcGPUMask)
	{
		/* default noop implementation */
		for (FTransferResourceFenceData* FenceData : FenceDatas)
		{
			delete FenceData;
		}
	}

	virtual void RHITransferResourceWait(TConstArrayView<FTransferResourceFenceData*> FenceDatas)
	{
		/* default noop implementation */
		for (FTransferResourceFenceData* FenceData : FenceDatas)
		{
			delete FenceData;
		}
	}

	/**
	 * Synchronizes the content of a resource between two or more GPUs using a copy operation -- variation of above that includes separate arrays of fences.
	 * @param Params - the parameters for each resource or texture region copied between GPUs.
	 * @param PreTransfer - Fences to wait on before copying the relevant data (initialized with RHITransferResourceSignal before this function)
	 * @param PostTransfer - Fences that can be waited on after copy (waited on by RHITransferResourceWait after this function)
	 */
	virtual void RHICrossGPUTransfer(TConstArrayView<FTransferResourceParams> Params, TConstArrayView<FCrossGPUTransferFence*> PreTransfer, TConstArrayView<FCrossGPUTransferFence*> PostTransfer)
	{
		/** empty default implementation. */
	}

	virtual void RHICrossGPUTransferSignal(TConstArrayView<FTransferResourceParams> Params, TConstArrayView<FCrossGPUTransferFence*> PreTransfer)
	{
		/* default noop implementation */
		for (FCrossGPUTransferFence* SyncPoint : PreTransfer)
		{
			delete SyncPoint;
		}
	}

	virtual void RHICrossGPUTransferWait(TConstArrayView<FCrossGPUTransferFence*> SyncPoints)
	{
		/* default noop implementation */
		for (FCrossGPUTransferFence* SyncPoint : SyncPoints)
		{
			delete SyncPoint;
		}
	}
#endif // WITH_MGPU

	virtual void RHIExecuteMultiIndirectClusterOperation(const FRayTracingClusterOperationParams& Params)
	{
		checkNoEntry();
	}

	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange)
	{
		checkNoEntry();
	}

	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params)
	{
		checkNoEntry();
	}

	virtual void RHIBindAccelerationStructureMemory(FRHIRayTracingScene* Scene, FRHIBuffer* Buffer, uint32 BufferOffset)
	{
		checkNoEntry();
	}

#if ENABLE_RHI_VALIDATION

	RHIValidation::FTracker* Tracker = nullptr;
	IRHIComputeContext* WrappingContext = nullptr;

	// Always returns the platform RHI context, even when the validation RHI is active.
	virtual IRHIComputeContext& GetLowestLevelContext() { return *this; }

	// Returns the validation RHI context if the validation RHI is active, otherwise returns the platform RHI context.
	IRHIComputeContext const& GetHighestLevelContext() const { return WrappingContext ? *WrappingContext : *this; }
	IRHIComputeContext      & GetHighestLevelContext()       { return WrappingContext ? *WrappingContext : *this; }

#else

	// Fast implementations when the RHI validation layer is disabled.
	IRHIComputeContext& GetLowestLevelContext() { return *this; }

	IRHIComputeContext const& GetHighestLevelContext() const { return *this; }
	IRHIComputeContext      & GetHighestLevelContext()       { return *this; }

#endif

#if ENABLE_RHI_VALIDATION
	virtual
#endif
	void SetTrackedAccess(const FRHITrackedAccessInfo& Info)
	{
		Info.Resource->SetTrackedAccessFromContext(Info);
	}

	inline ERHIAccess GetTrackedAccess(const FRHIViewableResource* Resource) const
	{
		check(Resource);
		return Resource->TrackedAccess.Access;
	}

	inline ERHIPipeline GetTrackedPipelines(const FRHIViewableResource* Resource) const
	{
		check(Resource);
		// The assumption is that all resources begin life on the gfx pipe
		return Resource->TrackedAccess.Pipelines != ERHIPipeline::None ? Resource->TrackedAccess.Pipelines : ERHIPipeline::Graphics;
	}

	UE_DEPRECATED(5.8, "RHIGetNativeCommandBuffer has been deprecated. Use RHI platform-specific interfaces to retrieve the native command buffer from a bottom-of-pipe RHI command list in a typesafe way.")
	virtual void* RHIGetNativeCommandBuffer() { return nullptr; }

	virtual void RHIPostExternalCommandsReset() { }

private:
	// Pointer to the RHI command list that is replaying commands into this context.
	class FRHICommandListBase* ExecutingCmdList = nullptr;

public:
	// Returns the RHI command list that is currently replaying commands into this context.
	FRHICommandListBase& GetExecutingCommandList() const
	{
		check(ExecutingCmdList);
		return *ExecutingCmdList;
	}

	// Used within FRHICommandListBase::ActivatePipeline to setup a context for command execution.
	virtual void SetExecutingCommandList(FRHICommandListBase* InCmdList)
	{
		ExecutingCmdList = InCmdList;
	}

#if WITH_RHI_BREADCRUMBS
	//
	// Returns true if RHI breadcrumb strings should be emitted to platform GPU profiling APIs.
	// Platform RHI implementations should check for this inside RHIBeginBreadcrumbGPU and RHIEndBreadcrumbGPU.
	//
	inline bool ShouldEmitBreadcrumbs() const;
#endif

protected:
	// Used to accumulate draw call and primitive counts,
	// via the RHI_DRAW_CALL_INC / RHI_DRAW_CALL_STATS macros.
	UE::RHI::GPUProfiler::FEvent::FStats StatEvent {};
};

/** Context that is used to generate Upload commands. */
class IRHIUploadContext
{};

// Utility function to generate pre-transfer sync points to pass to CrossGPUTransferSignal and CrossGPUTransfer
RHI_API void RHIGenerateCrossGPUPreTransferFences(TConstArrayView<FTransferResourceParams> Params, TArray<FCrossGPUTransferFence*>& OutPreTransfer);

enum class EAccelerationStructureBuildMode
{
	// Perform a full acceleration structure build.
	Build,

	// Update existing acceleration structure, based on new vertex positions.
	// Index buffer must not change between initial build and update operations.
	// Only valid when geometry was created with FRayTracingGeometryInitializer::bAllowUpdate = true.
	Update,
};

struct FRayTracingGeometryBuildParams
{
	FRayTracingGeometryRHIRef Geometry;
	EAccelerationStructureBuildMode BuildMode = EAccelerationStructureBuildMode::Build;

	// Optional array of geometry segments that can be used to change per-segment vertex buffers.
	// Only fields related to vertex buffer are used. If empty, then geometry vertex buffers are not changed.
	TConstArrayView<FRayTracingGeometrySegment> Segments;
};

struct FRayTracingSceneBuildParams
{
	// Scene to be built. May be null if explicit instance buffer is provided.
	FRHIRayTracingScene* Scene = nullptr;

	// Acceleration structure will be written to this buffer. The buffer must be in BVHWrite state.
	FRHIBuffer* ResultBuffer = nullptr;
	uint32 ResultBufferOffset = 0;

	// Scratch buffer used to build Acceleration structure. Must be in UAV state.
	FRHIBuffer* ScratchBuffer = nullptr;
	uint32 ScratchBufferOffset = 0;

	// Buffer of native ray tracing instance descriptors. Must be in SRV state.
	FRHIBuffer* InstanceBuffer = nullptr;
	uint32 InstanceBufferOffset = 0;
	
	// Buffer of native ray tracing hit contribution buffer. Must be in SRV state.
	// Only used when RequiresSeparateHitGroupContributionsBuffer is set
	FRHIBuffer* HitGroupContributionsBuffer = nullptr;
	uint32 HitGroupContributionsBufferOffset = 0;

	// Buffer of actual number of instances used for indirect acceleration structure build (if supported)
	FRHIBuffer* InstanceCountBuffer = nullptr;
	uint32 InstanceCountBufferOffset = 0;

	uint32 NumInstances = 0;

	// Unique list of geometries referenced by all instances in this scene.
	// Any referenced geometry is kept alive while the scene is alive.
	TConstArrayView<FRHIRayTracingGeometry*> ReferencedGeometries;

	// Unique list of buffers referenced by instances in this scene.
	// Any referenced buffer is kept alive while the scene is alive.
	TConstArrayView<FRHIBuffer*> ReferencedBuffers;

	EAccelerationStructureBuildMode BuildMode = EAccelerationStructureBuildMode::Build;
};

struct FRayTracingClusterOperationParams
{
	FRayTracingClusterOperationInitializer Initializer;

	struct FResourceBuffers
	{
		static const uint32 InAdditionalResourceCount = 4;
		typedef TArray<FRHIBuffer*, TInlineAllocator<InAdditionalResourceCount>> FAdditionalResourcesArray;

		struct FInResources
		{
			uint64 ResultCountOffsetInBytes = 0;			// Offset (in bytes) to where the count is in the ResultCount resource
			uint64 DescriptorsOffsetInBytes = 0;			// Offset (in bytes) to where the descriptors array starts in the Descriptors resource
			uint64 ScratchOffsetInBytes = 0;				// Offset (in bytes) to where the scratch memory starts in the Scratch resource

			FRHIBuffer* ResultCount = nullptr;				// The number of AS to build, instantiate, or move
			FRHIBuffer* Descriptors = nullptr;				// Array of descriptors
			FRHIBuffer* Scratch = nullptr;					// Scratch memory

			FAdditionalResourcesArray AdditionalResources;	// List of additional resources that need to be resident for this operation
		} In;

		struct FInOutResources
		{
			uint64 AddressesOffsetInBytes = 0;				// Offset (in bytes) to where the addresses array starts in the Addresses resource

			FRHIBuffer* Addresses = nullptr;				// Addresseses of CLAS, CLAS Templates, or BLAS
		} InOut;

		struct FOutResources
		{
			uint64 SizesOffsetInBytes = 0;					// Offset (in bytes) to where the output sizes array starts in the Sizes resource
			uint64 AccelerationStructuresOffsetInBytes = 0;	// Offset (in bytes) to where the output acceleration structures array starts in the AccelerationStructures resource

			FRHIBuffer* Sizes = nullptr;					// Sizes (in bytes) of CLAS, CLAS Templates, or BLAS
			FRHIBuffer* AccelerationStructures = nullptr;	// CLAS, CLAS Template, or BLAS
		} Out;
	} Resources;
};

struct FCopyBufferRegionParams
{
	FRHIBuffer* DestBuffer;
	uint64 DstOffset;
	FRHIBuffer* SourceBuffer;
	uint64 SrcOffset;
	uint64 NumBytes;
};

/** The interface RHI command context. Sometimes the RHI handles these. On platforms that can processes command lists in parallel, it is a separate object. */
class IRHICommandContext : public IRHIComputeContext
{
public:
	virtual ~IRHICommandContext() = default;

	virtual ERHIPipeline GetPipeline() const override
	{
		return ERHIPipeline::Graphics;
	}

	virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) = 0;

	virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) = 0;

	// Useful when used with geometry shader (emit polygons to different viewports), otherwise SetViewPort() is simpler
	// @param Count >0
	// @param Data must not be 0
	virtual void RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data) = 0;

	/**
	* Rebuilds the depth target HTILE meta data (on supported platforms).
	* @param DepthTexture - the depth surface to resummarize.
	*/
	virtual void RHIResummarizeHTile(FRHITexture* DepthTexture)
	{
		/* empty default implementation */
	}

	virtual void RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery) = 0;

	virtual void RHIEndRenderQuery(FRHIRenderQuery* RenderQuery) = 0;

	virtual void RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBuffer, uint32 Offset) = 0;

	// @param MinX including like Win32 RECT
	// @param MinY including like Win32 RECT
	// @param MaxX excluding like Win32 RECT
	// @param MaxY excluding like Win32 RECT
	virtual void RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ) = 0;

	virtual void RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ)
	{
		/* empty default implementation */
	}

	// @param MinX including like Win32 RECT
	// @param MinY including like Win32 RECT
	// @param MaxX excluding like Win32 RECT
	// @param MaxY excluding like Win32 RECT
	virtual void RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY) = 0;

	virtual void RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState) = 0;

#if PLATFORM_USE_FALLBACK_PSO
	virtual void RHISetGraphicsPipelineState(const FGraphicsPipelineStateInitializer& PsoInit, uint32 StencilRef, bool bApplyAdditionalState) = 0;
#endif

	// Inherit the parent context's RHISet functions that take FRHIComputeShader arguments
	// Required to avoid warning C4263 : 'function' : member function does not override any base class virtual member function
	using IRHIComputeContext::RHISetShaderParameters;
	using IRHIComputeContext::RHISetBatchedShaderParameters;
	using IRHIComputeContext::RHISetShaderUnbinds;

	virtual void RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) = 0;

	void RHISetBatchedShaderParameters(FRHIGraphicsShader* InShader, FRHIBatchedShaderParameters& InBatchedParameters)
	{
		RHISetShaderParameters(
			InShader,
			InBatchedParameters.ParametersData,
			InBatchedParameters.Parameters,
			InBatchedParameters.ResourceParameters,
			InBatchedParameters.BindlessParameters);

		InBatchedParameters.Reset();
	}

	virtual void RHISetShaderUnbinds(FRHIGraphicsShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
	{
		checkf(false, TEXT("RHISetShaderUnbinds called when the active RHI hasn't overridden it and GRHIGlobals.NeedsShaderUnbinds is set."));
	}

	virtual void RHISetStencilRef(uint32 StencilRef) {}

	virtual void RHISetBlendFactor(const FLinearColor& BlendFactor) {}

	virtual void RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances) = 0;

	virtual void RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) = 0;

	// @param NumPrimitives need to be >0 
	virtual void RHIDrawIndexedPrimitive(FRHIBuffer* IndexBuffer, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances) = 0;

	virtual void RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) = 0;

	/**
	* Similar to RHIDrawIndexedPrimitiveIndirect, but allows many draw arguments to be provided at once.
	* GRHIGlobals.SupportsDrawIndirect must be checked to detect support on the current machine.
	* @ param IndexBuffer			Buffer containing primitive indices
	* @ param ArgumentsBuffer		Buffer containing FRHIDrawIndexedIndirectParameters structures
	* @ param ArgumentOffset		Offset in bytes of the first element in ArgumentsBuffer that will be used for drawing
	* @ param CountBuffer			Buffer containing uint32 count of valid draw arguments that should be consumed (may be nullptr, indicating that only MaxDrawArguments value should be used)
	* @ param CountBuffeOffset		Offset in bytes for the CountBuffer element that will be used to source the draw argument count
	* @ param MaxDrawArguments		How many draw arguments should be processed at most, i.e. NumDrawArguments = min(MaxDrawArguments, ValueFromCountBuffer)
	*/
	virtual void RHIMultiDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset, FRHIBuffer* CountBuffer, uint32 CountBuffeOffset, uint32 MaxDrawArguments)
	{
		checkNoEntry();
	}

	virtual void RHIDispatchMeshShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
	{
		/* empty default implementation */
	}

	virtual void RHIDispatchIndirectMeshShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset)
	{
		/* empty default implementation */
	}

	/**
	* Sets Depth Bounds range with the given min/max depth.
	* @param MinDepth	The minimum depth for depth bounds test
	* @param MaxDepth	The maximum depth for depth bounds test.
	*					The valid values for fMinDepth and fMaxDepth are such that 0 <= fMinDepth <= fMaxDepth <= 1
	*/
	virtual void RHISetDepthBounds(float MinDepth, float MaxDepth) = 0;

	virtual void RHIGpuHangCommandListCorruption()
	{
		/* empty default implementation */
	}

	virtual void RHISetShadingRate(EVRSShadingRate ShadingRate, EVRSRateCombiner Combiner)
	{
		/* empty default implementation */
	} 

	virtual void RHIBeginParallelRenderPass(TSharedPtr<FRHIParallelRenderPassInfo> InInfo, const TCHAR* InName)
	{
		/* empty default implementation */
	}
	
	virtual void RHIEndParallelRenderPass()
	{
		/* empty default implementation */
	}
	
	virtual void RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName) = 0;

	virtual void RHIEndRenderPass() = 0;

	virtual void RHINextSubpass()
	{
	}

	virtual void RHICopyTexture(FRHITexture* SourceTexture, FRHITexture* DestTexture, const FRHICopyTextureInfo& CopyInfo) = 0;

	virtual void RHICopyBufferRegion(FRHIBuffer* DestBuffer, uint64 DstOffset, FRHIBuffer* SourceBuffer, uint64 SrcOffset, uint64 NumBytes) = 0;

	virtual void RHISetBindingsOnShaderBindingTable(FRHIShaderBindingTable* SBT, FRHIRayTracingPipelineState* Pipeline, uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings, ERayTracingBindingType BindingType)
	{
		checkNoEntry();
	}

	virtual void RHICommitShaderBindingTable(FRHIShaderBindingTable* SBT, FRHIBuffer* InlineBindingDataBuffer)
	{
		checkNoEntry();
	}

protected:
	FRHIRenderPassInfo RenderPassInfo;
};



inline FBoundShaderStateRHIRef RHICreateBoundShaderState(
	FRHIVertexDeclaration* VertexDeclaration,
	FRHIVertexShader* VertexShader,
	FRHIPixelShader* PixelShader,
	FRHIGeometryShader* GeometryShader
);

inline FBoundShaderStateRHIRef RHICreateBoundShaderState(
	FRHIAmplificationShader* AmplificationShader,
	FRHIMeshShader* MeshShader,
	FRHIPixelShader* PixelShader
);


// Command Context for RHIs that do not support real Graphics/Compute Pipelines.
class IRHICommandContextPSOFallback : public IRHICommandContext
{
public:
	virtual void RHISetBoundShaderState(FRHIBoundShaderState* BoundShaderState) = 0;
	virtual void RHISetDepthStencilState(FRHIDepthStencilState* NewState, uint32 StencilRef) = 0;
	virtual void RHISetRasterizerState(FRHIRasterizerState* NewState) = 0;
	virtual void RHISetBlendState(FRHIBlendState* NewState, const FLinearColor& BlendFactor) = 0;
	virtual void RHIEnableDepthBoundsTest(bool bEnable) = 0;
	virtual void RHISetComputeShader(FRHIComputeShader* ComputeShader) = 0;

	/**
	* This will set most relevant pipeline state. Legacy APIs are expected to set corresponding disjoint state as well.
	* @param GraphicsShaderState - the graphics pipeline state
	*/
	virtual void RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState) override
	{
		FRHIGraphicsPipelineStateFallBack* FallbackGraphicsState = static_cast<FRHIGraphicsPipelineStateFallBack*>(GraphicsState);
		SetGraphicsPipelineStateFromInitializer(FallbackGraphicsState->Initializer, StencilRef, bApplyAdditionalState);
	}

#if PLATFORM_USE_FALLBACK_PSO
	virtual void RHISetGraphicsPipelineState(const FGraphicsPipelineStateInitializer& PsoInit, uint32 StencilRef, bool bApplyAdditionalState) override
	{
		SetGraphicsPipelineStateFromInitializer(PsoInit, StencilRef, bApplyAdditionalState);
	}
#endif

	virtual void RHISetComputePipelineState(FRHIComputePipelineState* ComputePipelineState)
	{
		if (FRHIComputePipelineStateFallback* FallbackState = static_cast<FRHIComputePipelineStateFallback*>(ComputePipelineState))
		{
			RHISetComputeShader(FallbackState->GetComputeShader());
		}
	}

private:
	RHI_API void SetGraphicsPipelineStateFromInitializer(const FGraphicsPipelineStateInitializer& PsoInit, uint32 StencilRef, bool bApplyAdditionalState);
};
