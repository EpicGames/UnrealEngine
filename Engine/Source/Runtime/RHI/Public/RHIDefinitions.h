// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHIDefinitions.h: Render Hardware Interface definitions
		(that don't require linking).
=============================================================================*/

#pragma once

#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "ProfilingDebugging/CsvProfilerConfig.h" // TODO Move defines into RHIDefinitions
#include "UObject/NameTypes.h"
#include "Experimental/ConcurrentLinearAllocator.h"

#ifndef USE_STATIC_SHADER_PLATFORM_ENUMS
#define USE_STATIC_SHADER_PLATFORM_ENUMS 0
#endif

/** RHI Logging. */
RHI_API DECLARE_LOG_CATEGORY_EXTERN(LogRHI, Log, VeryVerbose);

/** Alignment of the shader parameters struct is required to be 16-byte boundaries. */
#define SHADER_PARAMETER_STRUCT_ALIGNMENT 16

/** The alignment in bytes between elements of array shader parameters. */
#define SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT 16

/** The alignment in bytes for rhi buffers. */
#define RHI_RAW_VIEW_ALIGNMENT 16

// RHICreateUniformBuffer assumes C++ constant layout matches the shader layout when extracting float constants, yet the C++ struct contains pointers.  
// Enforce a min size of 64 bits on pointer types in uniform buffer structs to guarantee layout matching between languages.
#define SHADER_PARAMETER_POINTER_ALIGNMENT sizeof(uint64)
static_assert(sizeof(void*) <= SHADER_PARAMETER_POINTER_ALIGNMENT, "The alignment of pointer needs to match the largest pointer.");

// Support platforms which require indirect dispatch arguments to not cross memory boundaries
#ifndef PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE
	#define PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE	0
#endif

#ifndef USE_STATIC_SHADER_PLATFORM_INFO
#define USE_STATIC_SHADER_PLATFORM_INFO 0
#endif


#ifndef RHI_USE_RESOURCE_DEBUG_NAME
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define RHI_USE_RESOURCE_DEBUG_NAME	0
#else
	#define RHI_USE_RESOURCE_DEBUG_NAME	1
#endif
#endif

#ifndef RHI_USE_SYNC_POINT_DEBUG_NAME
#if (UE_BUILD_SHIPPING)
#define RHI_USE_SYNC_POINT_DEBUG_NAME	0
#else
#define RHI_USE_SYNC_POINT_DEBUG_NAME	1
#endif
#endif

#ifndef RHI_RAYTRACING
#define RHI_RAYTRACING 0
#endif

#ifdef RHI_NEW_GPU_PROFILER
#undef RHI_NEW_GPU_PROFILER
#endif
#define RHI_NEW_GPU_PROFILER UE_DEPRECATED_MACRO(5.8, "The legacy GPU profiler has been removed. RHI_NEW_GPU_PROFILER is deprecated, and always defined to 1. Remove any remaining uses of this macro.") 1

#ifndef HAS_GPU_STATS
#define HAS_GPU_STATS ((STATS || CSV_PROFILER_STATS) && (!UE_BUILD_SHIPPING))
#endif


/**
 * The type used to pass the specialization values around
 */
using FSpecializationContainerType = TArray<int32>;


/**
 * A type used only for printing a string for debugging/profiling.
 * Adds Number as a suffix to the printed string even if the base name includes a number, so may prints a string like: Base_1_1
 * This type will always store a numeric suffix explicitly inside itself and never in the name table so it will always be at least 12 bytes
 * regardless of the value of UE_FNAME_OUTLINE_NUMBER.
 * It is not comparable or convertible to other name types to encourage its use only for debugging and avoid using more storage than necessary
 * for the primary use cases of FName (names of objects, assets etc which are widely used and therefor deduped in the name table).
 */
class FDebugName
{
public:
	// Buffer size required for any null-terminated FDebugName string, i.e. [name] '_' [digits] '\0'
	static constexpr uint32 StringBufferSize = FName::StringBufferSize + 1 + 10;

	RHI_API FDebugName();
	RHI_API FDebugName(FName InName);
	RHI_API FDebugName(FName InName, int32 InNumber);

	RHI_API FDebugName& operator=(FName Other);

	RHI_API FString ToString() const;

	/**
	 * Converts the FDebugName to a string buffer, avoiding dynamic allocations.
	 * Returns the length of the string, excluding the null terminator.
	 */
	template <uint32 N>
	uint32 ToString(TCHAR(&Out)[N]) const
	{
		UE_STATIC_DEPRECATE(5.6, N < StringBufferSize,
			"FDebugName::ToString(TCHAR (&Out)[N]) requires a buffer of size of at least FDebugName::StringBufferSize. "
			"Use ToStringTruncate() if a smaller buffer is required and it is safe for the returned string to be truncated.");

		return ToStringInternal(Out, N);
	}

	/**
	 * Converts the FDebugName to a string buffer, avoiding dynamic allocations. Truncates the name if it does not fit in the specified output buffer.
	 * Use a buffer size of at least FDebugName::StringBufferSize to avoid truncation.
	 *
	 * Returns the length of the (possibly truncated) string, excluding the null terminator.
	 */
	template <uint32 N>
	uint32 ToStringTruncate(TCHAR(&Out)[N]) const
	{
		static_assert(N > 0, "Out buffer must have at least one element.");
		return ToStringInternal(Out, N);
	}

	bool IsNone() const { return Name.IsNone() && Number == NAME_NO_NUMBER_INTERNAL; }
	RHI_API void AppendString(FStringBuilderBase& Builder) const;

private:
	FName Name;
	uint32 Number;

	RHI_API uint32 ToStringInternal(TCHAR* Out, uint32 OutSize) const;
};

enum class ERHIInterfaceType
{
	Hidden,
	Null,
	D3D11,
	D3D12,
	Vulkan,
	Metal,
	Agx,
	OpenGL,
};

enum class ERHIFeatureSupport : uint8
{
	// The RHI feature is completely unavailable at runtime
	Unsupported,

	// The RHI feature can be available at runtime based on hardware or driver
	RuntimeDependent,

	// The RHI feature is guaranteed to be available at runtime.
	RuntimeGuaranteed,

	Num,
	NumBits = 2,
};

enum class UE_DEPRECATED(5.7, "ERHIBindlessSupport is not used anymore, it is now a boolean") ERHIBindlessSupport : uint8
{
	Unsupported,
	RayTracingOnly,
	AllShaderTypes,

	NumBits = 2
};

enum class ERHIStaticShaderBindingLayoutSupport : uint8
{
	Unsupported,
	RayTracingOnly,
	AllShaderTypes,

	NumBits = 2
};

enum EShaderFrequency : uint8
{
	SF_Vertex				= 0,
	SF_Mesh					= 1,
	SF_Amplification		= 2,
	SF_Pixel				= 3,
	SF_Geometry				= 4,
	SF_Compute				= 5,
	SF_RayGen				= 6,
	SF_RayMiss				= 7,
	SF_RayHitGroup			= 8,
	SF_RayCallable			= 9,
	SF_WorkGraphRoot		= 10,
	SF_WorkGraphComputeNode	= 11,

	SF_NumFrequencies	= 12,

	// Number of standard shader frequencies for graphics pipeline (excluding compute)
	SF_NumGraphicsFrequencies = 5,

	// Number of standard shader frequencies (including compute)
	SF_NumStandardFrequencies = 6,

	SF_NumBits			= 4,
};
static_assert(SF_NumFrequencies <= (1 << SF_NumBits), "SF_NumFrequencies will not fit on SF_NumBits");

inline bool IsValidGraphicsFrequency(EShaderFrequency InShaderFrequency)
{
	switch (InShaderFrequency)
	{
	case SF_Vertex:        return true;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case SF_Mesh:          return true;
	case SF_Amplification: return true;
#endif
	case SF_Pixel:         return true;
	case SF_Geometry:      return true;
	}
	return false;
}

inline bool IsValidPreRasterizationFrequency(EShaderFrequency InShaderFrequency)
{
	// Any graphics frequency that isn't pixel
	return IsValidGraphicsFrequency(InShaderFrequency) && (InShaderFrequency != SF_Pixel);
}

inline bool IsComputeShaderFrequency(EShaderFrequency ShaderFrequency)
{
	switch (ShaderFrequency)
	{
	case SF_Compute:
	case SF_RayGen:
	case SF_RayMiss:
	case SF_RayHitGroup:
	case SF_RayCallable:
		return true;
	}
	return false;
}

enum ERenderQueryType
{
	// e.g. WaitForFrameEventCompletion()
	RQT_Undefined,
	// Result is the number of samples that are not culled (divide by MSAACount to get pixels)
	RQT_Occlusion,
	// Result is current time in micro seconds = 1/1000 ms = 1/1000000 sec (not a duration).
	RQT_AbsoluteTime,
};

/** Maximum number of miplevels in a texture. */
enum { MAX_TEXTURE_MIP_COUNT = 15 };

/** Maximum number of static/skeletal mesh LODs */
enum { MAX_MESH_LOD_COUNT = 8 };

/** The maximum number of vertex elements which can be used by a vertex declaration. */
enum
{
	MaxVertexElementCount = 17,
	MaxVertexElementCount_NumBits = 5,
};
static_assert(MaxVertexElementCount <= (1 << MaxVertexElementCount_NumBits), "MaxVertexElementCount will not fit on MaxVertexElementCount_NumBits");

/** The alignment in bytes between elements of array shader parameters. */
enum { ShaderArrayElementAlignBytes = 16 };

/** The number of render-targets that may be simultaneously written to. */
enum
{
	MaxSimultaneousRenderTargets = 8,
	MaxSimultaneousRenderTargets_NumBits = 3,
};
static_assert(MaxSimultaneousRenderTargets <= (1 << MaxSimultaneousRenderTargets_NumBits), "MaxSimultaneousRenderTargets will not fit on MaxSimultaneousRenderTargets_NumBits");

enum class UE_DEPRECATED(5.8, "ERHIZBuffer is being removed as the assumption of using an Inverted Z buffer is so ingrained that it could not reasonably be changed.") ERHIZBuffer
{
	FarPlane = 0,
	NearPlane = 1,
	IsInverted = 1,
};


/**
* The RHI's currently enabled shading path.
*/
namespace ERHIShadingPath
{
	enum Type : int
	{
		Deferred,
		Forward,
		Mobile,
		Num
	};
}

enum ESamplerFilter
{
	SF_Point,
	SF_Bilinear,
	SF_Trilinear,
	SF_AnisotropicPoint,
	SF_AnisotropicLinear,

	ESamplerFilter_Num,
	ESamplerFilter_NumBits = 3,
};
static_assert(ESamplerFilter_Num <= (1 << ESamplerFilter_NumBits), "ESamplerFilter_Num will not fit on ESamplerFilter_NumBits");

enum ESamplerAddressMode
{
	AM_Wrap,
	AM_Clamp,
	AM_Mirror,
	/** Not supported on all platforms */
	AM_Border,

	ESamplerAddressMode_Num,
	ESamplerAddressMode_NumBits = 2,
};
static_assert(ESamplerAddressMode_Num <= (1 << ESamplerAddressMode_NumBits), "ESamplerAddressMode_Num will not fit on ESamplerAddressMode_NumBits");

enum ESamplerCompareFunction
{
	SCF_Never,
	SCF_Less
};

enum ERasterizerFillMode
{
	FM_Point,
	FM_Wireframe,
	FM_Solid,

	ERasterizerFillMode_Num,
	ERasterizerFillMode_NumBits = 2,
};
static_assert(ERasterizerFillMode_Num <= (1 << ERasterizerFillMode_NumBits), "ERasterizerFillMode_Num will not fit on ERasterizerFillMode_NumBits");

enum ERasterizerCullMode
{
	CM_None,
	CM_CW,
	CM_CCW,

	ERasterizerCullMode_Num,
	ERasterizerCullMode_NumBits = 2,
};
static_assert(ERasterizerCullMode_Num <= (1 << ERasterizerCullMode_NumBits), "ERasterizerCullMode_Num will not fit on ERasterizerCullMode_NumBits");

enum class ERasterizerDepthClipMode : uint8
{
	DepthClip,
	DepthClamp,

	Num,
	NumBits = 1,
};
static_assert(uint32(ERasterizerDepthClipMode::Num) <= (1U << uint32(ERasterizerDepthClipMode::NumBits)), "ERasterizerDepthClipMode::Num will not fit on ERasterizerDepthClipMode::NumBits");

enum EColorWriteMask
{
	CW_RED   = 0x01,
	CW_GREEN = 0x02,
	CW_BLUE  = 0x04,
	CW_ALPHA = 0x08,

	CW_NONE  = 0,
	CW_RGB   = CW_RED | CW_GREEN | CW_BLUE,
	CW_RGBA  = CW_RED | CW_GREEN | CW_BLUE | CW_ALPHA,
	CW_RG    = CW_RED | CW_GREEN,
	CW_BA    = CW_BLUE | CW_ALPHA,

	EColorWriteMask_NumBits = 4,
};

enum ECompareFunction
{
	CF_Less,
	CF_LessEqual,
	CF_Greater,
	CF_GreaterEqual,
	CF_Equal,
	CF_NotEqual,
	CF_Never,
	CF_Always,

	ECompareFunction_Num,
	ECompareFunction_NumBits = 3,

	// Utility enumerations
	CF_DepthNearOrEqual		= CF_GreaterEqual,
	CF_DepthNear			= CF_Greater,
	CF_DepthFartherOrEqual	= CF_LessEqual,
	CF_DepthFarther			= CF_Less,
};
static_assert(ECompareFunction_Num <= (1 << ECompareFunction_NumBits), "ECompareFunction_Num will not fit on ECompareFunction_NumBits");

enum EStencilMask
{
	SM_Default,
	SM_255,
	SM_1,
	SM_2,
	SM_4,
	SM_8,
	SM_16,
	SM_32,
	SM_64,
	SM_128,
	SM_Count
};

enum EStencilOp
{
	SO_Keep,
	SO_Zero,
	SO_Replace,
	SO_SaturatedIncrement,
	SO_SaturatedDecrement,
	SO_Invert,
	SO_Increment,
	SO_Decrement,

	EStencilOp_Num,
	EStencilOp_NumBits = 3,
};
static_assert(EStencilOp_Num <= (1 << EStencilOp_NumBits), "EStencilOp_Num will not fit on EStencilOp_NumBits");

enum EBlendOperation
{
	BO_Add,
	BO_Subtract,
	BO_Min,
	BO_Max,
	BO_ReverseSubtract,

	EBlendOperation_Num,
	EBlendOperation_NumBits = 3,
};
static_assert(EBlendOperation_Num <= (1 << EBlendOperation_NumBits), "EBlendOperation_Num will not fit on EBlendOperation_NumBits");

enum EBlendFactor
{
	BF_Zero,
	BF_One,
	BF_SourceColor,
	BF_InverseSourceColor,
	BF_SourceAlpha,
	BF_InverseSourceAlpha,
	BF_DestAlpha,
	BF_InverseDestAlpha,
	BF_DestColor,
	BF_InverseDestColor,
	BF_ConstantBlendFactor,
	BF_InverseConstantBlendFactor,
	BF_Source1Color,
	BF_InverseSource1Color,
	BF_Source1Alpha,
	BF_InverseSource1Alpha,

	EBlendFactor_Num,
	EBlendFactor_NumBits = 4,
};
static_assert(EBlendFactor_Num <= (1 << EBlendFactor_NumBits), "EBlendFactor_Num will not fit on EBlendFactor_NumBits");

enum EVertexElementType
{
	VET_None,
	VET_Float1,
	VET_Float2,
	VET_Float3,
	VET_Float4,
	VET_PackedNormal,	// FPackedNormal
	VET_UByte4,
	VET_UByte4N,
	VET_Color,
	VET_Short2,
	VET_Short4,
	VET_Short2N,		// 16 bit word normalized to (value/32767.0,value/32767.0,0,0,1)
	VET_Half2,			// 16 bit float using 1 bit sign, 5 bit exponent, 10 bit mantissa 
	VET_Half4,
	VET_Short4N,		// 4 X 16 bit word, normalized 
	VET_UShort2,
	VET_UShort4,
	VET_UShort2N,		// 16 bit word normalized to (value/65535.0,value/65535.0,0,0,1)
	VET_UShort4N,		// 4 X 16 bit word unsigned, normalized 
	VET_URGB10A2N,		// 10 bit r, g, b and 2 bit a normalized to (value/1023.0f, value/1023.0f, value/1023.0f, value/3.0f)
	VET_UInt,
	VET_MAX,

	VET_NumBits = 5,
};
static_assert(VET_MAX <= (1 << VET_NumBits), "VET_MAX will not fit on VET_NumBits");

enum ECubeFace : uint32
{
	CubeFace_PosX = 0,
	CubeFace_NegX,
	CubeFace_PosY,
	CubeFace_NegY,
	CubeFace_PosZ,
	CubeFace_NegZ,
	CubeFace_MAX
};

enum EUniformBufferUsage
{
	// the uniform buffer is temporary, used for a single draw call then discarded
	UniformBuffer_SingleDraw = 0,
	// the uniform buffer is used for multiple draw calls but only for the current frame
	UniformBuffer_SingleFrame,
	// the uniform buffer is used for multiple draw calls, possibly across multiple frames
	UniformBuffer_MultiFrame,
};

enum class EUniformBufferValidation
{
	None,
	ValidateResources
};

/** The USF binding type for a resource in a shader. */
enum class EShaderCodeResourceBindingType : uint8
{
	Invalid,

	SamplerState,

	// Texture1D: not used in the renderer.
	// Texture1DArray: not used in the renderer.
	Texture2D,
	Texture2DArray,
	Texture2DMS,
	Texture3D,
	// Texture3DArray: not used in the renderer.
	TextureCube,
	TextureCubeArray,
	TextureMetadata,

	Buffer,
	StructuredBuffer,
	ByteAddressBuffer,
	RaytracingAccelerationStructure,

	// RWTexture1D: not used in the renderer.
	// RWTexture1DArray: not used in the renderer.
	RWTexture2D,
	RWTexture2DArray,
	RWTexture3D,
	// RWTexture3DArray: not used in the renderer.
	RWTextureCube,
	// RWTextureCubeArray: not used in the renderer.
	RWTextureMetadata,

	RWBuffer,
	RWStructuredBuffer,
	RWByteAddressBuffer,

	RasterizerOrderedTexture2D,

	ResourceCollection,
	DescriptorRange,

	// Texture2DMultiView can represent a Texture2D or Texture2DArray depending on whether shaders were compiled to use mobile multi-view (VK_KHR_multiview)
	Texture2DMultiView,
	RWTexture2DMultiView,

	MAX
};

inline bool IsResourceBindingTypeSRV(EShaderCodeResourceBindingType Type)
{
	switch (Type)
	{
	case EShaderCodeResourceBindingType::Texture2D:
	case EShaderCodeResourceBindingType::Texture2DArray:
	case EShaderCodeResourceBindingType::Texture2DMultiView:
	case EShaderCodeResourceBindingType::Texture2DMS:
	case EShaderCodeResourceBindingType::TextureCube:
	case EShaderCodeResourceBindingType::TextureCubeArray:
	case EShaderCodeResourceBindingType::Texture3D:
	case EShaderCodeResourceBindingType::ByteAddressBuffer:
	case EShaderCodeResourceBindingType::StructuredBuffer:
	case EShaderCodeResourceBindingType::Buffer:
	case EShaderCodeResourceBindingType::RaytracingAccelerationStructure:
	case EShaderCodeResourceBindingType::ResourceCollection:
		return true;
	case EShaderCodeResourceBindingType::RWTexture2D:
	case EShaderCodeResourceBindingType::RWTexture2DArray:
	case EShaderCodeResourceBindingType::RWTexture2DMultiView:
	case EShaderCodeResourceBindingType::RWTextureCube:
	case EShaderCodeResourceBindingType::RWTexture3D:
	case EShaderCodeResourceBindingType::RWByteAddressBuffer:
	case EShaderCodeResourceBindingType::RWStructuredBuffer:
	case EShaderCodeResourceBindingType::RWBuffer:
	case EShaderCodeResourceBindingType::RasterizerOrderedTexture2D:
	case EShaderCodeResourceBindingType::SamplerState:
	case EShaderCodeResourceBindingType::TextureMetadata:
	case EShaderCodeResourceBindingType::RWTextureMetadata:
	case EShaderCodeResourceBindingType::DescriptorRange:
		return false;
	case EShaderCodeResourceBindingType::MAX:
	case EShaderCodeResourceBindingType::Invalid:
	default:
		ensureMsgf(0, TEXT("Missing or invalid SRV or UAV Type"));
	}

	return false;
}

/** The base type of a value in a shader parameter structure. */
enum EUniformBufferBaseType : uint8
{
	UBMT_INVALID,

	// Invalid type when trying to use bool, to have explicit error message to programmer on why
	// they shouldn't use bool in shader parameter structures.
	UBMT_BOOL,

	// Parameter types.
	UBMT_INT32,
	UBMT_UINT32,
	UBMT_FLOAT32,

	// RHI resources not tracked by render graph.
	UBMT_TEXTURE,
	UBMT_SRV,
	UBMT_UAV,
	UBMT_SAMPLER,

	// Resources tracked by render graph.
	UBMT_RDG_TEXTURE,
	UBMT_RDG_TEXTURE_ACCESS,
	UBMT_RDG_TEXTURE_ACCESS_ARRAY,
	UBMT_RDG_TEXTURE_SRV,
	UBMT_RDG_TEXTURE_NON_PIXEL_SRV,
	UBMT_RDG_TEXTURE_UAV,
	UBMT_RDG_BUFFER_ACCESS,
	UBMT_RDG_BUFFER_ACCESS_ARRAY,
	UBMT_RDG_BUFFER_SRV,
	UBMT_RDG_BUFFER_UAV,
	UBMT_RDG_UNIFORM_BUFFER,

	// Nested structure.
	UBMT_NESTED_STRUCT,

	// Structure that is nested on C++ side, but included on shader side.
	UBMT_INCLUDED_STRUCT,

	// GPU Indirection reference of struct, like is currently named Uniform buffer.
	UBMT_REFERENCED_STRUCT,

	// Structure dedicated to setup render targets for a rasterizer pass.
	UBMT_RENDER_TARGET_BINDING_SLOTS,

	UBMT_RESOURCE_COLLECTION,
	UBMT_DESCRIPTOR_RANGE,

	EUniformBufferBaseType_Num,
	EUniformBufferBaseType_NumBits = 5,
};
static_assert(EUniformBufferBaseType_Num <= (1 << EUniformBufferBaseType_NumBits), "EUniformBufferBaseType_Num will not fit on EUniformBufferBaseType_NumBits");

/** The list of flags declaring which binding models are allowed for a uniform buffer layout. */
enum class EUniformBufferBindingFlags : uint8
{
	/** If set, the uniform buffer can be bound as an RHI shader parameter on an RHI shader (i.e. RHISetShaderUniformBuffer). */
	Shader = 1 << 0,

	/** If set, the uniform buffer can be bound globally through a static slot (i.e. RHISetStaticUniformBuffers). */
	Static = 1 << 1,

	/** If set, the uniform buffer can be bound globally or per-shader, depending on the use case. Only one binding model should be
	 *  used at a time, and RHI validation will emit an error if both are used for a particular uniform buffer at the same time. This
	 *  is designed for difficult cases where a fixed single binding model would produce an unnecessary maintenance burden. Using this
	 *  disables some RHI validation errors for global bindings, so use with care.
	 */
	StaticAndShader = Static | Shader
};
ENUM_CLASS_FLAGS(EUniformBufferBindingFlags);

/** Flags for Uniform Buffers */
enum class ERHIUniformBufferFlags : uint8
{
	None                    = 0,

	/** Whether to force a real uniform buffer when using emulated uniform buffers */
	NoEmulatedUniformBuffer = 1 << 0,

	/** Signals if the uniform buffer members need to be included in shader reflection */
	NeedsReflectedMembers   = 1 << 1,

	/** Whether this layout may contain non-render-graph outputs (e.g. RHI UAVs). */
	HasNonGraphOutputs      = 1 << 2,

	/** This struct is a view into uniform buffer object, on platforms that support UBO */
	UniformView             = 1 << 3,

	/** When bindless is enabled, make sure this uniform buffer can be accessed bindlessly. */
	BindlessAccessible      = 1 << 4,
};
ENUM_CLASS_FLAGS(ERHIUniformBufferFlags);

/** Numerical type used to store the static slot indices. */
using FUniformBufferStaticSlot = uint8;

enum
{
	/** The maximum number of static slots allowed. */
	MAX_UNIFORM_BUFFER_STATIC_SLOTS = 255
};

/** Returns whether a static uniform buffer slot index is valid. */
inline bool IsUniformBufferStaticSlotValid(const FUniformBufferStaticSlot Slot)
{
	return Slot < MAX_UNIFORM_BUFFER_STATIC_SLOTS;
}

struct FRHIResourceTableEntry
{
public:
	static constexpr uint32 GetEndOfStreamToken()
	{
		return 0xffffffff;
	}

	static uint32 Create(uint16 UniformBufferIndex, uint16 ResourceIndex, uint16 BindIndex)
	{
		return ((UniformBufferIndex & RTD_Mask_UniformBufferIndex) << RTD_Shift_UniformBufferIndex) |
			((ResourceIndex & RTD_Mask_ResourceIndex) << RTD_Shift_ResourceIndex) |
			((BindIndex & RTD_Mask_BindIndex) << RTD_Shift_BindIndex);
	}

	static inline uint16 GetUniformBufferIndex(uint32 Data)
	{
		return (Data >> RTD_Shift_UniformBufferIndex) & RTD_Mask_UniformBufferIndex;
	}

	static inline uint16 GetResourceIndex(uint32 Data)
	{
		return (Data >> RTD_Shift_ResourceIndex) & RTD_Mask_ResourceIndex;
	}

	static inline uint16 GetBindIndex(uint32 Data)
	{
		return (Data >> RTD_Shift_BindIndex) & RTD_Mask_BindIndex;
	}

private:
	enum EResourceTableDefinitions
	{
		RTD_NumBits_UniformBufferIndex	= 8,
		RTD_NumBits_ResourceIndex		= 16,
		RTD_NumBits_BindIndex			= 8,

		RTD_Mask_UniformBufferIndex		= (1 << RTD_NumBits_UniformBufferIndex) - 1,
		RTD_Mask_ResourceIndex			= (1 << RTD_NumBits_ResourceIndex) - 1,
		RTD_Mask_BindIndex				= (1 << RTD_NumBits_BindIndex) - 1,

		RTD_Shift_BindIndex				= 0,
		RTD_Shift_ResourceIndex			= RTD_Shift_BindIndex + RTD_NumBits_BindIndex,
		RTD_Shift_UniformBufferIndex	= RTD_Shift_ResourceIndex + RTD_NumBits_ResourceIndex,
	};
	static_assert(RTD_NumBits_UniformBufferIndex + RTD_NumBits_ResourceIndex + RTD_NumBits_BindIndex <= sizeof(uint32)* 8, "RTD_* values must fit in 32 bits");
};

enum EResourceLockMode
{
	RLM_ReadOnly,
	RLM_WriteOnly,
	RLM_WriteOnly_NoOverwrite,
	RLM_Num
};

/** limited to 8 types in FReadSurfaceDataFlags */
// RCM_UNorm is the default
// RCM_MinMax means "leave the values alone" and is recommended as what you should use
// RCM_SNorm and RCM_MinMaxNorm seem to be unsupported
enum ERangeCompressionMode
{
	// 0 .. 1
	RCM_UNorm, // if you read values that go outside [0,1], they are scaled to fit inside [0,1]
	// -1 .. 1
	RCM_SNorm,
	// 0 .. 1 unless there are smaller values than 0 or bigger values than 1, then the range is extended to the minimum or the maximum of the values
	RCM_MinMaxNorm,
	// minimum .. maximum (each channel independent)
	RCM_MinMax, // read values without changing them
};

enum class EPrimitiveTopologyType : uint8
{
	Triangle,
	Patch,
	Line,
	Point,
	//Quad,

	Num,
	NumBits = 2,
};
static_assert((uint32)EPrimitiveTopologyType::Num <= (1 << (uint32)EPrimitiveTopologyType::NumBits), "EPrimitiveTopologyType::Num will not fit on EPrimitiveTopologyType::NumBits");

enum EPrimitiveType
{
	// Topology that defines a triangle N with 3 vertex extremities: 3*N+0, 3*N+1, 3*N+2.
	PT_TriangleList,

	// Topology that defines a triangle N with 3 vertex extremities: N+0, N+1, N+2.
	PT_TriangleStrip,

	// Topology that defines a line with 2 vertex extremities: 2*N+0, 2*N+1.
	PT_LineList,

	// Topology that defines a quad N with 4 vertex extremities: 4*N+0, 4*N+1, 4*N+2, 4*N+3.
	// Supported only if GRHISupportsQuadTopology == true.
	PT_QuadList,

	// Topology that defines a point N with a single vertex N.
	PT_PointList,

	// Topology that defines a screen aligned rectangle N with only 3 vertex corners:
	//    3*N + 0 is upper-left corner,
	//    3*N + 1 is upper-right corner,
	//    3*N + 2 is the lower-left corner.
	// Supported only if GRHISupportsRectTopology == true.
	PT_RectList,

	PT_Num,
	PT_NumBits = 3
};
static_assert(PT_Num <= (1 << 8), "EPrimitiveType doesn't fit in a byte");
static_assert(PT_Num <= (1 << PT_NumBits), "PT_NumBits is too small");

enum EVRSAxisShadingRate : uint8
{
	VRSASR_1X = 0x0,
	VRSASR_2X = 0x1,
	VRSASR_4X = 0x2,
};

enum EVRSShadingRate : uint8
{
	VRSSR_1x1  = (VRSASR_1X << 2) + VRSASR_1X,
	VRSSR_1x2  = (VRSASR_1X << 2) + VRSASR_2X,
	VRSSR_2x1  = (VRSASR_2X << 2) + VRSASR_1X,
	VRSSR_2x2  = (VRSASR_2X << 2) + VRSASR_2X,
	VRSSR_2x4  = (VRSASR_2X << 2) + VRSASR_4X,
	VRSSR_4x2  = (VRSASR_4X << 2) + VRSASR_2X,
	VRSSR_4x4  = (VRSASR_4X << 2) + VRSASR_4X,
	
	VRSSR_Last  = VRSSR_4x4
};

enum EVRSRateCombiner : uint8
{
	VRSRB_Passthrough,
	VRSRB_Override,
	VRSRB_Min,
	VRSRB_Max,
	VRSRB_Sum,
};

enum EVRSImageDataType : uint8
{
	VRSImage_NotSupported,		// Image-based Variable Rate Shading is not supported on the current device/platform.
	VRSImage_Palette,			// Image-based VRS uses a palette of discrete, enumerated values to describe shading rate per tile.
	VRSImage_Fractional,		// Image-based VRS uses a floating point value to describe shading rate in X/Y (e.g. 1.0f is full rate, 0.5f is half-rate, 0.25f is 1/4 rate, etc).
};

/**
 *	Resource usage flags - for vertex and index buffers.
 */
enum class EBufferUsageFlags : uint32
{
	None                    = 0,

	/** The buffer will be written to once. */
	Static                  = 1 << 0,

	/** The buffer will be written to occasionally, GPU read only, CPU write only.  The data lifetime is until the next update, or the buffer is destroyed. */
	Dynamic                 = 1 << 1,

	/** The buffer's data will have a lifetime of one frame.  It MUST be written to each frame, or a new one created each frame. */
	Volatile                = 1 << 2,

	/** Allows an unordered access view to be created for the buffer. */
	UnorderedAccess         = 1 << 3,

	/** Create a byte address buffer, which is basically a structured buffer with a uint32 type. */
	ByteAddressBuffer       = 1 << 4,

	/** Buffer that the GPU will use as a source for a copy. */
	SourceCopy              = 1 << 5,

	UNUSED_BIT_6            = 1 << 6,

	/** Create a buffer which contains the arguments used by DispatchIndirect or DrawIndirect. */
	DrawIndirect            = 1 << 7,

	/** 
	 * Create a buffer that can be bound as a shader resource. 
	 * This is only needed for buffer types which wouldn't ordinarily be used as a shader resource, like a vertex buffer.
	 */
	ShaderResource          = 1 << 8,

	/** Request that this buffer is directly CPU accessible. */
	KeepCPUAccessible       = 1 << 9,

	/** Buffer should go in fast vram (hint only). Requires BUF_Transient */
	FastVRAM                = 1 << 10,

	/** Buffer is used by NNE.  DirectML requires NNE resources to be in single device memory heaps when multi-GPU is active. */
	NNE						= 1 << 11,

	/** Create a buffer that can be shared with an external RHI or process. */
	Shared                  = 1 << 12,

	/**
	 * Buffer contains opaque ray tracing acceleration structure data.
	 * Resources with this flag can't be bound directly to any shader stage and only can be used with ray tracing APIs.
	 * This flag is mutually exclusive with all other buffer flags except Static and ReservedResource.
	*/
	AccelerationStructure   = 1 << 13,

	VertexBuffer            = 1 << 14,
	IndexBuffer             = 1 << 15,
	StructuredBuffer        = 1 << 16,

	/** Buffer memory is allocated independently for multiple GPUs, rather than shared via driver aliasing */
	MultiGPUAllocate		= 1 << 17,

	/**
	 * Tells the render graph to not bother transferring across GPUs in multi-GPU scenarios.  Useful for cases where
	 * a buffer is read back to the CPU (such as streaming request buffers), or written to each frame by CPU (such
	 * as indirect arg buffers), and the other GPU doesn't actually care about the data.
	*/
	MultiGPUGraphIgnore		= 1 << 18,
	
	/** Allows buffer to be used as a scratch buffer for building ray tracing acceleration structure,
	 * which implies unordered access. Only changes the buffer alignment and can be combined with other flags.
	**/
	RayTracingScratch = (1 << 19) | UnorderedAccess,

	/** The buffer is a placeholder for streaming, and does not contain an underlying GPU resource. */
	NullResource = 1 << 20,

	/** Buffer can be used as uniform buffer on platforms that do support uniform buffer objects. */
	UniformBuffer = 1 << 21,

	/**
	* EXPERIMENTAL: Allow the buffer to be created as a reserved (AKA tiled/sparse/virtual) resource internally, without physical memory backing.
	* May not be used with Dynamic and other buffer flags that prevent the resource from being allocated in local GPU memory.
	*/
	ReservedResource = 1 << 22,

	// Helper bit-masks
	AnyDynamic = (Dynamic | Volatile),
};
ENUM_CLASS_FLAGS(EBufferUsageFlags);

#define BUF_None                   EBufferUsageFlags::None
#define BUF_Static                 EBufferUsageFlags::Static
#define BUF_Dynamic                EBufferUsageFlags::Dynamic
#define BUF_Volatile               EBufferUsageFlags::Volatile
#define BUF_UnorderedAccess        EBufferUsageFlags::UnorderedAccess
#define BUF_ByteAddressBuffer      EBufferUsageFlags::ByteAddressBuffer
#define BUF_SourceCopy             EBufferUsageFlags::SourceCopy
#define BUF_StreamOutput           EBufferUsageFlags::StreamOutput
#define BUF_DrawIndirect           EBufferUsageFlags::DrawIndirect
#define BUF_ShaderResource         EBufferUsageFlags::ShaderResource
#define BUF_KeepCPUAccessible      EBufferUsageFlags::KeepCPUAccessible
#define BUF_FastVRAM               EBufferUsageFlags::FastVRAM
#define BUF_Transient              EBufferUsageFlags::Transient
#define BUF_Shared                 EBufferUsageFlags::Shared
#define BUF_AccelerationStructure  EBufferUsageFlags::AccelerationStructure
#define BUF_RayTracingScratch	   EBufferUsageFlags::RayTracingScratch
#define BUF_VertexBuffer           EBufferUsageFlags::VertexBuffer
#define BUF_IndexBuffer            EBufferUsageFlags::IndexBuffer
#define BUF_StructuredBuffer       EBufferUsageFlags::StructuredBuffer
#define BUF_AnyDynamic             EBufferUsageFlags::AnyDynamic
#define BUF_MultiGPUAllocate       EBufferUsageFlags::MultiGPUAllocate
#define BUF_MultiGPUGraphIgnore    EBufferUsageFlags::MultiGPUGraphIgnore
#define BUF_NullResource           EBufferUsageFlags::NullResource
#define BUF_UniformBuffer          EBufferUsageFlags::UniformBuffer
#define BUF_ReservedResource       EBufferUsageFlags::ReservedResource

enum class EGpuVendorId : uint32
{
	Unknown		= 0xffffffff,
	NotQueried	= 0,

	Amd			= 0x1002,
	ImgTec		= 0x1010,
	Nvidia		= 0x10DE, 
	Arm			= 0x13B5, 
	Broadcom	= 0x14E4,
	Qualcomm	= 0x5143,
	Intel		= 0x8086,
	Apple		= 0x106B,
	Vivante		= 0x7a05,
	VeriSilicon	= 0x1EB1,
	SamsungAMD  = 0x144D,
	Microsoft   = 0x1414,

	Kazan		= 0x10003,	// VkVendorId
	Codeplay	= 0x10004,	// VkVendorId
	Mesa		= 0x10005,	// VkVendorId
};

/** An enumeration of the different RHI reference types. */
enum ERHIResourceType : uint8
{
	RRT_None,

	RRT_SamplerState,
	RRT_RasterizerState,
	RRT_DepthStencilState,
	RRT_BlendState,
	RRT_VertexDeclaration,
	RRT_VertexShader,
	RRT_MeshShader,
	RRT_AmplificationShader,
	RRT_PixelShader,
	RRT_GeometryShader,
	RRT_RayTracingShader,
	RRT_ComputeShader,
	RRT_GraphicsPipelineState,
	RRT_ComputePipelineState,
	RRT_RayTracingPipelineState,
	RRT_BoundShaderState,
	RRT_UniformBufferLayout,
	RRT_UniformBuffer,
	RRT_Buffer,
	RRT_Texture,
	RRT_Texture2D      UE_DEPRECATED(5.7,      "RRT_Texture2D is unused. Remove any remaining uses of it. There is a single RHI texture type which always uses RRT_Texture. RHI texture references use RRT_TextureReference."),
	RRT_Texture2DArray UE_DEPRECATED(5.7, "RRT_Texture2DArray is unused. Remove any remaining uses of it. There is a single RHI texture type which always uses RRT_Texture. RHI texture references use RRT_TextureReference."),
	RRT_Texture3D      UE_DEPRECATED(5.7,      "RRT_Texture3D is unused. Remove any remaining uses of it. There is a single RHI texture type which always uses RRT_Texture. RHI texture references use RRT_TextureReference."),
	RRT_TextureCube    UE_DEPRECATED(5.7,    "RRT_TextureCube is unused. Remove any remaining uses of it. There is a single RHI texture type which always uses RRT_Texture. RHI texture references use RRT_TextureReference."),
	RRT_TextureReference,
	RRT_TimestampCalibrationQuery UE_DEPRECATED(5.8, "RRT_TimestampCalibrationQuery is unused. Remove any remaining uses of it."),
	RRT_GPUFence,
	RRT_RenderQuery,
	RRT_RenderQueryPool,
	RRT_Viewport,
	RRT_UnorderedAccessView,
	RRT_ShaderResourceView,
	RRT_RayTracingAccelerationStructure UE_DEPRECATED(5.7, "RRT_RayTracingAccelerationStructure is deprecated. Use RRT_RayTracingGeometry or RRT_RayTracingScene instead."),
	RRT_RayTracingGeometry,
	RRT_RayTracingScene,
	RRT_RayTracingShaderBindingTable,
	RRT_StagingBuffer,
	RRT_CustomPresent,
	RRT_ShaderLibrary,
	RRT_ShaderBundle,
	RRT_WorkGraphShader,
	RRT_WorkGraphPipelineState,
	RRT_StreamSourceSlot,
	RRT_ResourceCollection,
	RRT_DescriptorRange,

	RRT_Num
};

/** Describes the dimension of a texture. */
enum class ETextureDimension : uint8
{
	Texture2D,
	Texture2DArray,
	Texture3D,
	TextureCube,
	TextureCubeArray
};

/** Flags used for texture creation */
enum class ETextureCreateFlags : uint64
{
    None                              = 0,

    // Texture can be used as a render target
    RenderTargetable                  = 1ull << 0,
    // Texture can be used as a resolve target
    ResolveTargetable                 = 1ull << 1,
    // Texture can be used as a depth-stencil target.
    DepthStencilTargetable            = 1ull << 2,
    // Texture can be used as a shader resource.
    ShaderResource                    = 1ull << 3,
    // Texture is encoded in sRGB gamma space
    SRGB                              = 1ull << 4,
    // Texture data is writable by the CPU
    CPUWritable                       = 1ull << 5,
    // Texture will be created with an un-tiled format
    NoTiling                          = 1ull << 6,
    // Texture will be used for video decode
    VideoDecode                       = 1ull << 7,
	// Texture expected to be updated by the CPU every frame, caches staging buffers on D3D12 to potentially improve performance and save memory
    Dynamic                           = 1ull << 8,
    // Texture will be used as a render pass attachment that will be read from
    InputAttachmentRead               = 1ull << 9,
    /** Texture represents a foveation attachment */
    Foveation                         = 1ull << 10,
    // Prefer 3D internal surface tiling mode for volume textures when possible
    Tiling3D                          = 1ull << 11,
    // This texture has no GPU or CPU backing. It only exists in tile memory on TBDR GPUs (i.e., mobile).
    Memoryless                        = 1ull << 12,

    //UNUSED_BIT_13                   = 1ull << 13,

    // The texture can be partially allocated in fastvram
    FastVRAMPartialAlloc              = 1ull << 14,
    // Do not create associated shader resource view, only applicable to D3D11 and D3D12
    DisableSRVCreation                = 1ull << 15,
    // Do not allow Delta Color Compression (DCC) to be used with this texture
    DisableDCC                        = 1ull << 16,
    // UnorderedAccessView (DX11 only)
    // Warning: Causes additional synchronization between draw calls when using a render target allocated with this flag, use sparingly
    // See: GCNPerformanceTweets.pdf Tip 37
    UAV                               = 1ull << 17,
    // Render target texture that will be displayed on screen (back buffer)
    Presentable                       = 1ull << 18,
    // Texture data is accessible by the CPU
    CPUReadback                       = 1ull << 19,
    // Texture was processed offline (via a texture conversion process for the current platform)
    OfflineProcessed                  = 1ull << 20,
    // Texture needs to go in fast VRAM if available (HINT only)
    FastVRAM                          = 1ull << 21,
    // by default the texture is not showing up in the list - this is to reduce clutter, using the FULL option this can be ignored
    HideInVisualizeTexture            = 1ull << 22,
    // Texture should be created in virtual memory, with no physical memory allocation made
    // You must make further calls to RHIVirtualTextureSetFirstMipInMemory to allocate physical memory
    // and RHIVirtualTextureSetFirstMipVisible to map the first mip visible to the GPU
    Virtual                           = 1ull << 23,
    // Creates a RenderTargetView for each array slice of the texture
    // Warning: if this was specified when the resource was created, you can't use SV_RenderTargetArrayIndex to route to other slices!
    TargetArraySlicesIndependently    = 1ull << 24,
    // Texture that may be shared with DX9 or other devices
    Shared                            = 1ull << 25,
    // RenderTarget will not use full-texture fast clear functionality.
    NoFastClear                       = 1ull << 26,
    // Texture is a depth stencil resolve target
    DepthStencilResolveTarget         = 1ull << 27,
    // Flag used to indicted this texture is a streamable 2D texture, and should be counted towards the texture streaming pool budget.
    Streamable                        = 1ull << 28,
    // Render target will not FinalizeFastClear; Caches and meta data will be flushed, but clearing will be skipped (avoids potentially trashing metadata)
    NoFastClearFinalize               = 1ull << 29,
	/** Texture needs to support atomic operations */
	Atomic64Compatible                = 1ull << 30,
    // Workaround for 128^3 volume textures getting bloated 4x due to tiling mode on some platforms.
    ReduceMemoryWithTilingMode        = 1ull << 31,
    /** Texture needs to support atomic operations */
    AtomicCompatible                  = 1ull << 33,
	/** Texture should be allocated for external access. Vulkan only */
	External                		  = 1ull << 34,
	/** Don't automatically transfer across GPUs in multi-GPU scenarios.  For example, if you are transferring it yourself manually. */
	MultiGPUGraphIgnore				  = 1ull << 35,
	/**
	* EXPERIMENTAL: Allow the texture to be created as a reserved (AKA tiled/sparse/virtual) resource internally, without physical memory backing. 
	* May not be used with Dynamic and other buffer flags that prevent the resource from being allocated in local GPU memory.
	*/
	ReservedResource                  = 1ull << 37,
	/** EXPERIMENTAL: Used with ReservedResource flag to immediately allocate and commit memory on creation. May use N small physical memory allocations instead of a single large one. */
	ImmediateCommit                   = 1ull << 38,

	/** Don't lump this texture with streaming memory when tracking total texture allocation sizes */
	ForceIntoNonStreamingMemoryTracking = 1ull << 39,

	/** Textures marked with this are meant to be immediately evicted after creation for intentionally crashing the GPU with a page fault. */
	Invalid                           = 1ull << 40,

	// Allow lossy framebuffer compression if supported by the GPU
	LossyCompression                  = 1ull << 41,
	LossyCompressionLowBitrate        = 1ull << 42
};
ENUM_CLASS_FLAGS(ETextureCreateFlags);

// Compatibility defines
#define TexCreate_None                           ETextureCreateFlags::None
#define TexCreate_RenderTargetable               ETextureCreateFlags::RenderTargetable
#define TexCreate_ResolveTargetable              ETextureCreateFlags::ResolveTargetable
#define TexCreate_DepthStencilTargetable         ETextureCreateFlags::DepthStencilTargetable
#define TexCreate_ShaderResource                 ETextureCreateFlags::ShaderResource
#define TexCreate_SRGB                           ETextureCreateFlags::SRGB
#define TexCreate_CPUWritable                    ETextureCreateFlags::CPUWritable
#define TexCreate_NoTiling                       ETextureCreateFlags::NoTiling
#define TexCreate_VideoDecode                    ETextureCreateFlags::VideoDecode
#define TexCreate_Dynamic                        ETextureCreateFlags::Dynamic
#define TexCreate_InputAttachmentRead            ETextureCreateFlags::InputAttachmentRead
#define TexCreate_Foveation                      ETextureCreateFlags::Foveation
#define TexCreate_3DTiling                       ETextureCreateFlags::Tiling3D
#define TexCreate_Memoryless                     ETextureCreateFlags::Memoryless
#define TexCreate_GenerateMipCapable             ETextureCreateFlags::GenerateMipCapable
#define TexCreate_FastVRAMPartialAlloc           ETextureCreateFlags::FastVRAMPartialAlloc
#define TexCreate_DisableSRVCreation             ETextureCreateFlags::DisableSRVCreation
#define TexCreate_DisableDCC                     ETextureCreateFlags::DisableDCC
#define TexCreate_UAV                            ETextureCreateFlags::UAV
#define TexCreate_Presentable                    ETextureCreateFlags::Presentable
#define TexCreate_CPUReadback                    ETextureCreateFlags::CPUReadback
#define TexCreate_OfflineProcessed               ETextureCreateFlags::OfflineProcessed
#define TexCreate_FastVRAM                       ETextureCreateFlags::FastVRAM
#define TexCreate_HideInVisualizeTexture         ETextureCreateFlags::HideInVisualizeTexture
#define TexCreate_Virtual                        ETextureCreateFlags::Virtual
#define TexCreate_TargetArraySlicesIndependently ETextureCreateFlags::TargetArraySlicesIndependently
#define TexCreate_Shared                         ETextureCreateFlags::Shared
#define TexCreate_NoFastClear                    ETextureCreateFlags::NoFastClear
#define TexCreate_DepthStencilResolveTarget      ETextureCreateFlags::DepthStencilResolveTarget
#define TexCreate_Streamable                     ETextureCreateFlags::Streamable
#define TexCreate_NoFastClearFinalize            ETextureCreateFlags::NoFastClearFinalize
#define TexCreate_ReduceMemoryWithTilingMode     ETextureCreateFlags::ReduceMemoryWithTilingMode
#define TexCreate_Transient                      ETextureCreateFlags::Transient
#define TexCreate_AtomicCompatible               ETextureCreateFlags::AtomicCompatible
#define TexCreate_External                       ETextureCreateFlags::External
#define TexCreate_MultiGPUGraphIgnore            ETextureCreateFlags::MultiGPUGraphIgnore
#define TexCreate_ReservedResource               ETextureCreateFlags::ReservedResource
#define TexCreate_ImmediateCommit                ETextureCreateFlags::ImmediateCommit
#define TexCreate_Invalid                        ETextureCreateFlags::Invalid
#define TexCreate_LossyCompression               ETextureCreateFlags::LossyCompression
#define TexCreate_LossyCompressionLowBitrate     ETextureCreateFlags::LossyCompressionLowBitrate

enum EAsyncComputePriority
{
	AsyncComputePriority_Default = 0,
	AsyncComputePriority_High,
};

/**
 * Async texture reallocation status, returned by RHIGetReallocateTexture2DStatus().
 */
enum ETextureReallocationStatus
{
	TexRealloc_Succeeded = 0,
	TexRealloc_Failed,
	TexRealloc_InProgress,
};

/**
 * Action to take when a render target is set.
 */
enum class ERenderTargetLoadAction : uint8
{
	// Untouched contents of the render target are undefined. Any existing content is not preserved.
	ENoAction,

	// Existing contents are preserved.
	ELoad,

	// The render target is cleared to the fast clear value specified on the resource.
	EClear,

	Num,
	NumBits = 2,
};
static_assert((uint32)ERenderTargetLoadAction::Num <= (1 << (uint32)ERenderTargetLoadAction::NumBits), "ERenderTargetLoadAction::Num will not fit on ERenderTargetLoadAction::NumBits");

/**
 * Action to take when a render target is unset or at the end of a pass. 
 */
enum class ERenderTargetStoreAction : uint8
{
	// Contents of the render target emitted during the pass are not stored back to memory.
	ENoAction,

	// Contents of the render target emitted during the pass are stored back to memory.
	EStore,

	// Contents of the render target emitted during the pass are resolved using a box filter and stored back to memory.
	EMultisampleResolve,

	Num,
	NumBits = 2,
};
static_assert((uint32)ERenderTargetStoreAction::Num <= (1 << (uint32)ERenderTargetStoreAction::NumBits), "ERenderTargetStoreAction::Num will not fit on ERenderTargetStoreAction::NumBits");

/**
 * Common render target use cases
 */
enum class ESimpleRenderTargetMode
{
	// These will all store out color and depth
	EExistingColorAndDepth,							// Color = Existing, Depth = Existing
	EUninitializedColorAndDepth,					// Color = ????, Depth = ????
	EUninitializedColorExistingDepth,				// Color = ????, Depth = Existing
	EUninitializedColorClearDepth,					// Color = ????, Depth = Default
	EClearColorExistingDepth,						// Clear Color = whatever was bound to the rendertarget at creation time. Depth = Existing
	EClearColorAndDepth,							// Clear color and depth to bound clear values.
	EExistingContents_NoDepthStore,					// Load existing contents, but don't store depth out.  depth can be written.
	EExistingColorAndClearDepth,					// Color = Existing, Depth = clear value
	EExistingColorAndDepthAndClearStencil,			// Color = Existing, Depth = Existing, Stencil = clear

	// If you add an item here, make sure to add it to DecodeRenderTargetMode() as well!
};

enum class EClearDepthStencil
{
	Depth,
	Stencil,
	DepthStencil,
};

/**
 * Hint to the driver on how to load balance async compute work.  On some platforms this may be a priority, on others actually masking out parts of the GPU for types of work.
 */
enum class EAsyncComputeBudget
{
	ELeast_0,			//Least amount of GPU allocated to AsyncCompute that still gets 'some' done.
	EGfxHeavy_1,		//Gfx gets most of the GPU.
	EBalanced_2,		//Async compute and Gfx share GPU equally.
	EComputeHeavy_3,	//Async compute can use most of the GPU
	EAll_4,				//Async compute can use the entire GPU.
};

/**
* Hint to control how to balance GPU resources between graphics and synchronous compute workloads.
* On some platforms, compute shaders may run with higher occupancy when they don't need to share resources with graphics.
* Compute-heavy mode may prevent async compute queue workloads from running concurrently with main queue work.
*/
enum class ESyncComputeBudget
{
	Default,		// Use default behavior chosen by the current RHI
	Balanced,		// Balance GPU resources to allow overlapped graphics and compute shader execution
	ComputeHeavy,	// Give more of the GPU resources to run synchronous compute workloads
};

enum class ERHIDescriptorHeapType : uint8
{
	Standard,
	Sampler,
	RenderTarget,
	DepthStencil,
	Count,
	Invalid = MAX_uint8
};

enum class ERHIDescriptorType : uint8
{
	BufferSRV,
	BufferUAV,

	TypedBufferSRV,
	TypedBufferUAV,

	TextureSRV,
	TextureUAV,

	CBV,
	Sampler,

	AccelerationStructure,

	Invalid
};

enum class ERHIDescriptorTypeMask : uint16
{
	None = 0,

	BufferSRV = (1 << static_cast<int32>(ERHIDescriptorType::BufferSRV)),
	BufferUAV = (1 << static_cast<int32>(ERHIDescriptorType::BufferUAV)),

	TypedBufferSRV = (1 << static_cast<int32>(ERHIDescriptorType::TypedBufferSRV)),
	TypedBufferUAV = (1 << static_cast<int32>(ERHIDescriptorType::TypedBufferUAV)),

	TextureSRV = (1 << static_cast<int32>(ERHIDescriptorType::TextureSRV)),
	TextureUAV = (1 << static_cast<int32>(ERHIDescriptorType::TextureUAV)),

	CBV = (1 << static_cast<int32>(ERHIDescriptorType::CBV)),
	Sampler = (1 << static_cast<int32>(ERHIDescriptorType::Sampler)),

	AccelerationStructure = (1 << static_cast<int32>(ERHIDescriptorType::AccelerationStructure)),

	SRV = (BufferSRV | TypedBufferSRV | TextureSRV),
	UAV = (BufferUAV | TypedBufferUAV | TextureUAV),

	Standard = (SRV | UAV | CBV | AccelerationStructure),
};
ENUM_CLASS_FLAGS(ERHIDescriptorTypeMask);

inline constexpr ERHIDescriptorTypeMask RHIDescriptorTypeMaskFromType(ERHIDescriptorType InType)
{
	if (InType == ERHIDescriptorType::Invalid)
	{
		return ERHIDescriptorTypeMask::None;
	}
	return static_cast<ERHIDescriptorTypeMask>(1 << static_cast<int32>(InType));
}

struct FRHIDescriptorHandle
{
	FRHIDescriptorHandle() = default;

	FRHIDescriptorHandle(ERHIDescriptorType InType, uint32 InIndex)
		: Index(InIndex)
		, Type(InType)
	{
	}

	uint32 GetIndex() const
	{
		return Index;
	}

	ERHIDescriptorType GetType() const
	{
		return Type;
	}

	ERHIDescriptorTypeMask GetTypeMask() const
	{
		return RHIDescriptorTypeMaskFromType(GetType());
	}

	bool IsValid() const
	{
		return Index != MAX_uint32 && Type != ERHIDescriptorType::Invalid;
	}

private:
	uint32             Index{ MAX_uint32 };
	ERHIDescriptorType Type{ ERHIDescriptorType::Invalid };
};

// Configuration levels for Bindless. Each value should include more shaders than the previous.
enum class ERHIBindlessConfiguration
{
	Disabled,

	// Only RayTracing shaders
	RayTracing,

	// RayTracing + Materials + shader types that have opted-in for bindless
	Minimal,

	// All shaders
	All
};

inline bool IsBindlessDisabled(ERHIBindlessConfiguration Configuration)
{
	return Configuration == ERHIBindlessConfiguration::Disabled;
}

inline bool IsBindlessEnabledForOnlyRayTracing(ERHIBindlessConfiguration Configuration)
{
	return Configuration == ERHIBindlessConfiguration::RayTracing;
}

inline bool IsBindlessEnabledForRayTracing(ERHIBindlessConfiguration Configuration)
{
	return Configuration >= ERHIBindlessConfiguration::RayTracing;
}

inline bool IsBindlessEnabledForAnyGraphics(ERHIBindlessConfiguration Configuration)
{
	return Configuration >= ERHIBindlessConfiguration::Minimal;
}

inline bool IsBindlessFullyEnabled(ERHIBindlessConfiguration Configuration)
{
	return Configuration == ERHIBindlessConfiguration::All;
}

struct FRHIBindlessDescriptorAllocationInfo
{
	ERHIDescriptorTypeMask Types{};
	uint64 Count = 0;
	uint64 Capacity = 0;
	uint64 MaxCapacity = 0;
};

enum class EColorSpaceAndEOTF
{
	EUnknown = 0,

	EColorSpace_Rec709  = 1,		// Color Space Uses Rec 709  Primaries
	EColorSpace_Rec2020 = 2,		// Color Space Uses Rec 2020 Primaries
	EColorSpace_DCIP3   = 3,		// Color Space Uses DCI-P3   Primaries
	EEColorSpace_MASK   = 0xf,

	EEOTF_Linear		= 1 << 4,   // Transfer Function Uses Linear Encoding
	EEOTF_sRGB			= 2 << 4,	// Transfer Function Uses sRGB Encoding
	EEOTF_PQ			= 3 << 4,	// Transfer Function Uses PQ Encoding
	EEOTF_MASK			= 0xf << 4,

	ERec709_sRGB		= EColorSpace_Rec709  | EEOTF_sRGB,
	ERec709_Linear		= EColorSpace_Rec709  | EEOTF_Linear,
	
	ERec2020_PQ			= EColorSpace_Rec2020 | EEOTF_PQ,
	ERec2020_Linear		= EColorSpace_Rec2020 | EEOTF_Linear,
	
	EDCIP3_PQ			= EColorSpace_DCIP3 | EEOTF_PQ,
	EDCIP3_Linear		= EColorSpace_DCIP3 | EEOTF_Linear,
};

enum class ERHIShaderBundleMethod
{
	// Dispatch shader bundles using the emulation path.
	Emulation = 0,

	// Dispatch shader bundles using native platform features.
	Native = 1,
};

enum class ERHITransitionCreateFlags
{
	None = 0,

	// Disables fencing between pipelines during the transition.
	NoFence = 1 << 0,

	// Indicates the transition will have no useful work between the Begin/End calls,
	// so should use a partial flush rather than a fence as this is more optimal.
	NoSplit = 1 << 1,

	// Indicates that the transition is allowed to happen during a RenderPass
	AllowDuringRenderPass = 1 << 2,

	// This is an advanced mode that allows issuing a transition using a single pipeline as the previous pipelines in a transition
	// for a resource that was previously transitioned to ERHIPipeline::All. It assumes that a manual fence was issued to synchronize
	// the prior async compute work. It also assumes that tracked access for a resource (using SetTrackedAccess) is up to date.
	AllowDecayPipelines = 1 << 3
};

ENUM_CLASS_FLAGS(ERHITransitionCreateFlags);

enum class EResourceTransitionFlags : uint8
{
	None                = 0,

	MaintainCompression = 1 << 0, // Specifies that the transition should not decompress the resource, allowing us to read a compressed resource directly in its compressed state.
	IgnoreAfterState    = 1 << 1, // The transition will be patched for Unknown if needed but the After State will be ignored by the tracker

	Last = IgnoreAfterState,
	Mask = (Last << 1) - 1
};
ENUM_CLASS_FLAGS(EResourceTransitionFlags);

enum class ERequestedGPUCrash : uint8
{
	None = 0,
	Type_Hang = 1 << 0,
	Type_PageFault = 1 << 1,
	Type_PlatformBreak = 1 << 2,
	Type_Assert = 1 << 3,
	Type_CmdListCorruption = 1 << 4,

	Queue_Direct = 1 << 5,
	Queue_Compute = 1 << 6,
};
ENUM_CLASS_FLAGS(ERequestedGPUCrash);

/** Returns whether the shader parameter type references an RDG texture. */
inline bool IsRDGTextureReferenceShaderParameterType(EUniformBufferBaseType BaseType)
{
	return
		BaseType == UBMT_RDG_TEXTURE ||
		BaseType == UBMT_RDG_TEXTURE_SRV ||
		BaseType == UBMT_RDG_TEXTURE_NON_PIXEL_SRV ||
		BaseType == UBMT_RDG_TEXTURE_UAV ||
		BaseType == UBMT_RDG_TEXTURE_ACCESS ||
		BaseType == UBMT_RDG_TEXTURE_ACCESS_ARRAY;
}

/** Returns whether the shader parameter type references an RDG buffer. */
inline bool IsRDGBufferReferenceShaderParameterType(EUniformBufferBaseType BaseType)
{
	return
		BaseType == UBMT_RDG_BUFFER_SRV ||
		BaseType == UBMT_RDG_BUFFER_UAV ||
		BaseType == UBMT_RDG_BUFFER_ACCESS ||
		BaseType == UBMT_RDG_BUFFER_ACCESS_ARRAY;
}

/** Returns whether the shader parameter type is for RDG access and not actually for shaders. */
inline bool IsRDGResourceAccessType(EUniformBufferBaseType BaseType)
{
	return
		BaseType == UBMT_RDG_TEXTURE_ACCESS ||
		BaseType == UBMT_RDG_TEXTURE_ACCESS_ARRAY ||
		BaseType == UBMT_RDG_BUFFER_ACCESS ||
		BaseType == UBMT_RDG_BUFFER_ACCESS_ARRAY;
}

/** Returns whether the shader parameter type is a reference onto a RDG resource. */
inline bool IsRDGResourceReferenceShaderParameterType(EUniformBufferBaseType BaseType)
{
	return IsRDGTextureReferenceShaderParameterType(BaseType) || IsRDGBufferReferenceShaderParameterType(BaseType) || BaseType == UBMT_RDG_UNIFORM_BUFFER;
}

inline bool IsShaderParameterTypeReadOnlyRHIResource(EUniformBufferBaseType BaseType)
{
	return
		BaseType == UBMT_TEXTURE ||
		BaseType == UBMT_SRV ||
		BaseType == UBMT_SAMPLER ||
		BaseType == UBMT_RESOURCE_COLLECTION ||
		BaseType == UBMT_DESCRIPTOR_RANGE;
}

inline bool IsShaderParameterTypeRHIResource(EUniformBufferBaseType BaseType)
{
	return
		IsShaderParameterTypeReadOnlyRHIResource(BaseType) ||
		BaseType == UBMT_UAV;
}

/** Returns whether the shader parameter type needs to be passdown to RHI through FRHIUniformBufferLayout when creating an uniform buffer. */
inline bool IsShaderParameterTypeForUniformBufferLayout(EUniformBufferBaseType BaseType)
{
	return
		// RHI resource referenced in shader parameter structures.
		IsShaderParameterTypeRHIResource(BaseType) ||

		// RHI is able to access RHI resources from RDG.
		IsRDGResourceReferenceShaderParameterType(BaseType) ||

		// Render graph uses FRHIUniformBufferLayout to walk pass' parameters.
		BaseType == UBMT_REFERENCED_STRUCT ||
		BaseType == UBMT_RENDER_TARGET_BINDING_SLOTS;
}

/** Returns whether the shader parameter type in FRHIUniformBufferLayout is actually ignored by the RHI. */
inline bool IsShaderParameterTypeIgnoredByRHI(EUniformBufferBaseType BaseType)
{
	return
		// Render targets bindings slots needs to be in FRHIUniformBufferLayout for render graph, but the RHI does not actually need to know about it.
		BaseType == UBMT_RENDER_TARGET_BINDING_SLOTS ||

		// Custom access states are used by the render graph.
		IsRDGResourceAccessType(BaseType) ||

		// #yuriy_todo: RHI is able to dereference uniform buffer in root shader parameter structures
		BaseType == UBMT_REFERENCED_STRUCT ||
		BaseType == UBMT_RDG_UNIFORM_BUFFER;
}

inline EGpuVendorId RHIConvertToGpuVendorId(uint32 VendorId)
{
	switch ((EGpuVendorId)VendorId)
	{
	case EGpuVendorId::NotQueried:
		return EGpuVendorId::NotQueried;

	case EGpuVendorId::Amd:
	case EGpuVendorId::Kazan:
	case EGpuVendorId::Codeplay:
	case EGpuVendorId::Mesa:
	case EGpuVendorId::ImgTec:
	case EGpuVendorId::Nvidia:
	case EGpuVendorId::Arm:
	case EGpuVendorId::Broadcom:
	case EGpuVendorId::Qualcomm:
	case EGpuVendorId::Intel:
	case EGpuVendorId::Apple:
	case EGpuVendorId::Vivante:
	case EGpuVendorId::VeriSilicon:
	case EGpuVendorId::SamsungAMD:
	case EGpuVendorId::Microsoft:
		return (EGpuVendorId)VendorId;

	case EGpuVendorId::Unknown:
	default:
		return EGpuVendorId::Unknown;
	}
}

inline bool IsGeometryPipelineShaderFrequency(EShaderFrequency Frequency)
{
	switch (Frequency)
	{
	case SF_Mesh:
	case SF_Amplification:
		return true;
	case SF_Vertex:
	case SF_Pixel:
	case SF_Geometry:
	case SF_Compute:
	case SF_RayGen:
	case SF_RayMiss:
	case SF_RayHitGroup:
	case SF_RayCallable:
	case SF_WorkGraphRoot:
	case SF_WorkGraphComputeNode:
	case SF_NumFrequencies:
	default:
		return false;
	}
}

inline bool IsRayTracingShaderFrequency(EShaderFrequency Frequency)
{
	switch (Frequency)
	{
	case SF_RayGen:
	case SF_RayMiss:
	case SF_RayHitGroup:
	case SF_RayCallable:
		return true;
	case SF_Vertex:
	case SF_Mesh:
	case SF_Amplification:
	case SF_Pixel:
	case SF_Geometry:
	case SF_Compute:
	case SF_WorkGraphRoot:
	case SF_WorkGraphComputeNode:
	case SF_NumFrequencies:
	default:
		return false;
	}
}

inline bool IsWorkGraphShaderFrequency(EShaderFrequency Frequency)
{
	switch (Frequency)
	{
	case SF_WorkGraphRoot:
	case SF_WorkGraphComputeNode:
		return true;
	case SF_Vertex:
	case SF_Mesh:
	case SF_Amplification:
	case SF_Pixel:
	case SF_Geometry:
	case SF_Compute:
	case SF_RayGen:
	case SF_RayMiss:
	case SF_RayHitGroup:
	case SF_RayCallable:
	case SF_NumFrequencies:
	default:
		return false;
	}
}

UE_DEPRECATED(5.7, "GetRHIResourceType(ETextureDimension Dimension) is deprecated. There is a single RHI texture type which always uses RRT_Texture now, which makes this function pointless. Remove any remaining uses of this function.")
inline ERHIResourceType GetRHIResourceType(ETextureDimension /*Dimension*/)
{
	return ERHIResourceType::RRT_Texture;
}

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	#define GEOMETRY_SHADER(GeometryShader)	(GeometryShader)
#else
	#define GEOMETRY_SHADER(GeometryShader)	nullptr
#endif

/** Screen Resolution */
struct FScreenResolutionRHI
{
	uint32	Width;
	uint32	Height;
	uint32	RefreshRate;
};

struct FShaderCodeValidationStride
{
	uint16 BindPoint;
	uint16 Stride;
};

struct FShaderCodeValidationType
{
	uint16 BindPoint;
	EShaderCodeResourceBindingType Type;
};

struct FShaderCodeValidationUBSize
{
	uint16 BindPoint;
	uint32 Size;
};

/** Struct to provide details of swap chain flips */
struct FRHIFlipDetails
{
	uint64 PresentIndex;
	double FlipTimeInSeconds;
	double VBlankTimeInSeconds;
	uint64 VBlankTimeInCycles;
	float Hz;

	FRHIFlipDetails()
		: PresentIndex(0)
		, FlipTimeInSeconds(0)
		, VBlankTimeInSeconds(0)
		, VBlankTimeInCycles(0)
		, Hz(0.f)
	{
	}

	FRHIFlipDetails(uint64 InPresentIndex, double InFlipTimeInSeconds, double InVBlankTimeInSeconds, uint64 InVBlankTimeInCycles, float InHz)
		: PresentIndex(InPresentIndex)
		, FlipTimeInSeconds(InFlipTimeInSeconds)
		, VBlankTimeInSeconds(InVBlankTimeInSeconds)
		, VBlankTimeInCycles(InVBlankTimeInCycles)
		, Hz(InHz)
	{
	}
};
