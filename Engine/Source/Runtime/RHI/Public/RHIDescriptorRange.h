// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIResources.h"

//
// RHI Descriptor Range - Experimental feature, these types may completely change in the future.
//
// Abstraction around allocating a contiguous range of descriptors and loading them in shaders. The descriptor range will manage updating the platform specific
// bindless descriptor heap with its resources. When bound to a shader, the offset into the bindless descriptor heap will be available in the shader.
// 
// Type Mask
//   Due to limitations on certain platforms, not all resource/view types can be in the same descriptor range. The types that can be used together are exposed
//   via 'GRHIGlobals.DescriptorTypeGroups'. Each element in the DescriptorTypeGroups array corresponds to a set of descriptor types (via mask) that can be
//   used together. Given how nuanced some of the descriptor types are, this may create friction in some uses of descriptor ranges.
// 
//   To make type mask support easier, consider using different descriptor ranges for read-only vs read-write resources.
// 
// Resource Lifetimes
//   The Descriptor Range will keep the resources assigned to it alive until the descriptor range is destroyed.
// 
// Support checks for this feature:
//   Shader support: FDataDrivenShaderPlatformInfo::GetSupportsDescriptorRange
//   Shader define:  PLATFORM_SUPPORTS_DESCRIPTOR_RANGE
//   Runtime:        GRHIGlobals.bSupportsDescriptorRange
// (These must be checked in addition to the normal bindless checks)
// 
// Creation:
//   FRHIDescriptorRangeDesc Desc{ Count, TypeMask };
//   TArray<FRHIDescriptorResource> Resources = ...;
//   FDescriptorRangeRHIRef DescriptorRange = RHICmdList.CreateDescriptorRange(Desc, Resources);
// 
// Updating:
//   TArray<FRHIDescriptorResource> Resources = ...;
//   RHICmdList.UpdateDescriptorRange(DescriptorRange, Offset, Resources);
// 
// Shader binding:
//   SHADER_PARAMETER_DESCRIPTOR_RANGE(FDescriptorRange, DescriptorRange)
// 
// Shader access:
//   Texture2D MyTexture = GetDescriptorRangeSRV(Texture2D, DescriptorRange, Offset);
// 

struct FRHIDescriptorRangeDesc
{
	// Number of descriptors in the range.
	uint32 Count = 0;

	// Mask of different descriptor types in the range. Types cannot cross groups exposed in GRHIGlobals.DescriptorTypeGroups
	ERHIDescriptorTypeMask TypeMask = ERHIDescriptorTypeMask::None;
};

// The data that is bound in the shaders.
struct FRHIDescriptorRangeShaderData
{
	uint32 StartIndex;
	uint32 Count;
};
static_assert(sizeof(FRHIDescriptorRangeShaderData) == sizeof(void*), "FRHIDescriptorRangeShaderData must be exactly the same size as a pointer");

// Container for descriptor sources. Sources are Textures, SRVs and UAVs.
// Samplers may be added in the future.
struct FRHIDescriptorResource
{
	enum class EType : uint8
	{
		None,
		Texture,
		ResourceView,
		UnorderedAccessView,
	};

	FRHIDescriptorResource() = default;
	FRHIDescriptorResource(EType InType, FRHIResource* InResource)
		: Resource(InResource)
		, Type(InType)
	{
	}
	FRHIDescriptorResource(FRHITexture* InTexture)
		: FRHIDescriptorResource(FRHIDescriptorResource::EType::Texture, InTexture)
	{
	}
	FRHIDescriptorResource(FRHIShaderResourceView* InView)
		: FRHIDescriptorResource(FRHIDescriptorResource::EType::ResourceView, InView)
	{
	}
	FRHIDescriptorResource(FRHIUnorderedAccessView* InUAV)
		: FRHIDescriptorResource(FRHIDescriptorResource::EType::UnorderedAccessView, InUAV)
	{
	}

	bool operator==(const FRHIDescriptorResource& Other) const
	{
		return Resource == Other.Resource
			&& Type == Other.Type;
	}

	FRHIResource* Resource = nullptr;
	EType         Type = EType::None;
};

class FRHIDescriptorRange : public FRHIResource
{
public:
	const FRHIDescriptorRangeDesc& GetDesc() const
	{
		return Desc;
	}

	bool IsValid() const
	{
		return GetDesc().TypeMask != ERHIDescriptorTypeMask::None;
	}

	virtual FRHIDescriptorRangeShaderData GetShaderData() const = 0;

	// Returns the resource and type at the given index as an FRHIDescriptorResource.
	FRHIDescriptorResource GetResource(int32 Index) const
	{
		checkSlow(Index >= 0 && Index < Resources.Num());
		return FRHIDescriptorResource(ResourceTypes[Index], Resources[Index].GetReference());
	}

	int32 GetNumResources() const
	{
		return Resources.Num();
	}

	TConstArrayView<FRHIDescriptorResource::EType> GetResourceTypes() const
	{
		return MakeConstArrayView(ResourceTypes);
	}

protected:
	RHI_API FRHIDescriptorRange(const FRHIDescriptorRangeDesc& InDesc, TConstArrayView<FRHIDescriptorResource> InResources);
	RHI_API ~FRHIDescriptorRange();

	// Non-copyable due to manual AddRef/Release on resources.
	FRHIDescriptorRange(const FRHIDescriptorRange&) = delete;
	FRHIDescriptorRange& operator=(const FRHIDescriptorRange&) = delete;

	// Updates a single resource slot. Handles AddRef/Release for the old and new resource.
	RHI_API void UpdateResource(int32 Index, FRHIDescriptorResource InResource);
	RHI_API void UpdateResources(int32 StartIndex, TConstArrayView<FRHIDescriptorResource> InResources);

	const FRHIDescriptorRangeDesc Desc;

	// Resources assigned to this range. Sized to Desc.Count.
	TArray<TRefCountPtr<FRHIResource>> Resources;

	// Parallel array storing the descriptor type for each slot. Sized to Desc.Count.
	TArray<FRHIDescriptorResource::EType> ResourceTypes;
};
