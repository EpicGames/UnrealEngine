// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "RHIDefinitions.h"

#ifndef USE_STATIC_FEATURE_LEVEL_ENUMS
#define USE_STATIC_FEATURE_LEVEL_ENUMS 0
#endif

/**
 * The RHI's feature level indicates what level of support can be relied upon.
 * Note: these are named after graphics API's like ES3 but a feature level can be used with a different API (eg ERHIFeatureLevel::ES3.1 on D3D11)
 * As long as the graphics API supports all the features of the feature level (eg no ERHIFeatureLevel::SM5 on OpenGL ES3.1)
 */
namespace ERHIFeatureLevel
{
	enum Type : int
	{
		/** Feature level defined by the core capabilities of OpenGL ES2. Deprecated */
		ES2_REMOVED,

		/** Feature level defined by the core capabilities of OpenGL ES3.1 & Metal/Vulkan. */
		ES3_1,

		/**
		 * Feature level defined by the capabilities of DX10 Shader Model 4.
		 * SUPPORT FOR THIS FEATURE LEVEL HAS BEEN ENTIRELY REMOVED.
		 */
		SM4_REMOVED,

		/**
		 * Feature level defined by the capabilities of DX11 Shader Model 5.
		 *   Compute shaders with shared memory, group sync, UAV writes, integer atomics
		 *   Indirect drawing
		 *   Pixel shaders with UAV writes
		 *   Cubemap arrays
		 *   Read-only depth or stencil views (eg read depth buffer as SRV while depth test and stencil write)
		 * Tessellation is not considered part of Feature Level SM5 and has a separate capability flag.
		 */
		SM5,

		/**
		 * Feature level defined by the capabilities of DirectX 12 hardware feature level 12_2 with Shader Model 6.5
		 *   Raytracing Tier 1.1
		 *   Mesh and Amplification shaders
		 *   Variable rate shading
		 *   Sampler feedback
		 *   Resource binding tier 3
		 */
		SM6,

		Num
	};
};

struct FGenericStaticFeatureLevel
{
	inline FGenericStaticFeatureLevel(const ERHIFeatureLevel::Type InFeatureLevel) : FeatureLevel(InFeatureLevel) {}
	inline FGenericStaticFeatureLevel(const TEnumAsByte<ERHIFeatureLevel::Type> InFeatureLevel) : FeatureLevel(InFeatureLevel) {}

	inline operator ERHIFeatureLevel::Type() const
	{
		return FeatureLevel;
	}

	inline bool operator == (const ERHIFeatureLevel::Type Other) const
	{
		return Other == FeatureLevel;
	}

	inline bool operator != (const ERHIFeatureLevel::Type Other) const
	{
		return Other != FeatureLevel;
	}

	inline bool operator <= (const ERHIFeatureLevel::Type Other) const
	{
		return FeatureLevel <= Other;
	}

	inline bool operator < (const ERHIFeatureLevel::Type Other) const
	{
		return FeatureLevel < Other;
	}

	inline bool operator >= (const ERHIFeatureLevel::Type Other) const
	{
		return FeatureLevel >= Other;
	}

	inline bool operator > (const ERHIFeatureLevel::Type Other) const
	{
		return FeatureLevel > Other;
	}

private:
	ERHIFeatureLevel::Type FeatureLevel;
};

#if USE_STATIC_FEATURE_LEVEL_ENUMS
#include COMPILED_PLATFORM_HEADER(StaticFeatureLevel.inl)
#else
using FStaticFeatureLevel = FGenericStaticFeatureLevel;
#endif

// The maximum feature level available on this system
extern RHI_API ERHIFeatureLevel::Type GMaxRHIFeatureLevel;
