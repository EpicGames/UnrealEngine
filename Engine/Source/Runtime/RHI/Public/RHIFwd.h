// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/RefCounting.h"

// RHI_WANT_RESOURCE_INFO should be controlled by the RHI module.
#ifndef RHI_WANT_RESOURCE_INFO
#define RHI_WANT_RESOURCE_INFO 0
#endif

// RHI_FORCE_DISABLE_RESOURCE_INFO can be defined anywhere else, like in GlobalDefinitions.
#ifndef RHI_FORCE_DISABLE_RESOURCE_INFO
#define RHI_FORCE_DISABLE_RESOURCE_INFO 0
#endif

#define RHI_ENABLE_RESOURCE_INFO (RHI_WANT_RESOURCE_INFO && !RHI_FORCE_DISABLE_RESOURCE_INFO)

// Basic Types

namespace ERHIFeatureLevel { enum Type : int; }
enum EShaderPlatform : uint16;
enum ECubeFace : uint32;

enum EPixelFormat : uint8;
enum class EPixelFormatChannelFlags : uint8;

enum class EBufferUsageFlags : uint32;
enum class ETextureCreateFlags : uint64;

// Command Lists
class FRHICommandListBase;
class FRHIComputeCommandList;
class FRHICommandList;
class FRHICommandListImmediate;
class FRHISubCommandList;

// Contexts
class IRHIComputeContext;
class IRHICommandContext;
class IRHIUploadContext;
class FRHIContextArray;

class FRHIResourceReplaceInfo;
class FRHIResourceReplaceBatcher;

struct FSamplerStateInitializerRHI;
struct FRasterizerStateInitializerRHI;
struct FDepthStencilStateInitializerRHI;
class FBlendStateInitializerRHI;

// Resources
class FRHIAmplificationShader;
class FRHIBlendState;
class FRHIBoundShaderState;
class FRHIBuffer;
class FRHIComputePipelineState;
class FRHIComputeShader;
class FRHICustomPresent;
class FRHIDepthStencilState;
class FRHIDescriptorRange;
class FRHIGeometryShader;
class FRHIGPUFence;
class FRHIGraphicsPipelineState;
class FRHIMeshShader;
class FRHIPixelShader;
class FRHIRasterizerState;
class FRHIRayTracingGeometry;
class FRHIRayTracingPipelineState;
class FRHIRayTracingScene;
class FRHIShaderBindingTable;
class FRHIRayTracingShader;
class FRHIRenderQuery;
class FRHIRenderQueryPool;
class FRHIResource;
class FRHIResourceCollection;
class FRHISamplerState;
class FRHIShader;
class FRHIShaderData;
class FRHIShaderLibrary;
class FRHIShaderResourceView;
class FRHIShaderBundle;
class FRHIStagingBuffer;
class FRHITexture;
class FRHITextureReference;
class FRHIUniformBuffer;
class FRHIUnorderedAccessView;
class FRHIVertexDeclaration;
class FRHIVertexShader;
class FRHIViewableResource;
class FRHIViewport;
class FRHIWorkGraphPipelineState;
class FRHIWorkGraphShader;
class FRHIStreamSourceSlot;

struct FRHIUniformBufferLayout;

// Pointers

using FAmplificationShaderRHIRef       = TRefCountPtr<FRHIAmplificationShader>;
using FBlendStateRHIRef                = TRefCountPtr<FRHIBlendState>;
using FBoundShaderStateRHIRef          = TRefCountPtr<FRHIBoundShaderState>;
using FBufferRHIRef                    = TRefCountPtr<FRHIBuffer>;
using FComputePipelineStateRHIRef      = TRefCountPtr<FRHIComputePipelineState>;
using FComputeShaderRHIRef             = TRefCountPtr<FRHIComputeShader>;
using FCustomPresentRHIRef             = TRefCountPtr<FRHICustomPresent>;
using FDepthStencilStateRHIRef         = TRefCountPtr<FRHIDepthStencilState>;
using FDescriptorRangeRHIRef           = TRefCountPtr<FRHIDescriptorRange>;
using FGeometryShaderRHIRef            = TRefCountPtr<FRHIGeometryShader>;
using FGPUFenceRHIRef                  = TRefCountPtr<FRHIGPUFence>;
using FGraphicsPipelineStateRHIRef     = TRefCountPtr<FRHIGraphicsPipelineState>;
using FMeshShaderRHIRef                = TRefCountPtr<FRHIMeshShader>;
using FPixelShaderRHIRef               = TRefCountPtr<FRHIPixelShader>;
using FRasterizerStateRHIRef           = TRefCountPtr<FRHIRasterizerState>;
using FRayTracingGeometryRHIRef        = TRefCountPtr<FRHIRayTracingGeometry>;
using FRayTracingPipelineStateRHIRef   = TRefCountPtr<FRHIRayTracingPipelineState>;
using FRayTracingSceneRHIRef           = TRefCountPtr<FRHIRayTracingScene>;
using FRayTracingShaderRHIRef          = TRefCountPtr<FRHIRayTracingShader>;
using FShaderBindingTableRHIRef        = TRefCountPtr<FRHIShaderBindingTable>;
using FRenderQueryPoolRHIRef           = TRefCountPtr<FRHIRenderQueryPool>;
using FRenderQueryRHIRef               = TRefCountPtr<FRHIRenderQuery>;
using FRHIResourceCollectionRef        = TRefCountPtr<FRHIResourceCollection>;
using FRHIShaderLibraryRef             = TRefCountPtr<FRHIShaderLibrary>;
using FRHIShaderResourceViewRef        = TRefCountPtr<FRHIShaderResourceView>;
using FSamplerStateRHIRef              = TRefCountPtr<FRHISamplerState>;
using FShaderResourceViewRHIRef        = TRefCountPtr<FRHIShaderResourceView>;
using FShaderBundleRHIRef              = TRefCountPtr<FRHIShaderBundle>;
using FStagingBufferRHIRef             = TRefCountPtr<FRHIStagingBuffer>;
using FTextureReferenceRHIRef          = TRefCountPtr<FRHITextureReference>;
using FTextureRHIRef                   = TRefCountPtr<FRHITexture>;
using FUniformBufferLayoutRHIRef       = TRefCountPtr<const FRHIUniformBufferLayout>;
using FUniformBufferRHIRef             = TRefCountPtr<FRHIUniformBuffer>;
using FUnorderedAccessViewRHIRef       = TRefCountPtr<FRHIUnorderedAccessView>;
using FVertexDeclarationRHIRef         = TRefCountPtr<FRHIVertexDeclaration>;
using FVertexShaderRHIRef              = TRefCountPtr<FRHIVertexShader>;
using FViewportRHIRef                  = TRefCountPtr<FRHIViewport>;
using FWorkGraphPipelineStateRHIRef    = TRefCountPtr<FRHIWorkGraphPipelineState>;
using FStreamSourceSlotRHIRef          = TRefCountPtr<FRHIStreamSourceSlot>;
using FWorkGraphShaderRHIRef           = TRefCountPtr<FRHIWorkGraphShader>;
