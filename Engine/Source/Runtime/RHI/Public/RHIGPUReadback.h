// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
  RHIGPUReadback.h: classes for managing fences and staging buffers for
  asynchronous GPU memory updates and readbacks with minimal stalls and no
  RHI thread flushes
=============================================================================*/

#pragma once

#include "RHI.h"
#include "DynamicRHI.h"
#include "RHICommandList.h"
#include "MultiGPU.h"

/**
 * FRHIGPUMemoryReadback: Represents a memory readback request scheduled with CopyToStagingBuffer
 * Wraps a staging buffer with a FRHIGPUFence for synchronization.
 */
class FRHIGPUMemoryReadback
{
public:

	FRHIGPUMemoryReadback(FName RequestName)
	{
		Fence = RHICreateGPUFence(RequestName);
		LastLockGPUIndex = 0;
	}

	virtual ~FRHIGPUMemoryReadback() = default;

	/** Indicates if the data is in place and ready to be read. */
	inline bool IsReady()
	{
		return !Fence || (Fence->NumPendingWriteCommands.GetValue() == 0 && Fence->Poll());
	}

	/** Indicates if the data is in place and ready to be read on a subset of GPUs. */
	inline bool IsReady(FRHIGPUMask GPUMask)
	{
		return !Fence || Fence->Poll(GPUMask);
	}

	/** Reset the ready state */
	inline void ResetReady()
	{
		if (Fence)
		{
			Fence->Clear();
		}
	}

	void Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const
	{
		if (Fence)
		{
			Fence->Wait(RHICmdList, GPUMask);
		}
	}

	/**
	 * Copy the current state of the resource to the readback data.
	 * @param RHICmdList The command list to enqueue the copy request on.
	 * @param SourceBuffer The buffer holding the source data.
	 * @param NumBytes The number of bytes to copy. If 0, this will copy the entire buffer.
	 */
	virtual void EnqueueCopy(FRHICommandList& RHICmdList, FRHIBuffer* SourceBuffer, uint32 NumBytes = 0)
	{
		unimplemented();
	}

	virtual void EnqueueCopy(FRHICommandList& RHICmdList, FRHITexture* SourceTexture, const FIntVector& SourcePosition, uint32 SourceSlice, const FIntVector& Size)
	{
		unimplemented();
	}

	void EnqueueCopy(FRHICommandList& RHICmdList, FRHITexture* SourceTexture, FResolveRect Rect = FResolveRect(), uint32 SourceSlice = 0)
	{
		FIntVector SourcePosition, Size;

		if (Rect.IsValid())
		{
			SourcePosition = FIntVector(Rect.X1, Rect.Y1, 0);
			Size = FIntVector(Rect.X2 - Rect.X1, Rect.Y2 - Rect.Y1, 1);
		}
		else
		{
			SourcePosition = FIntVector::ZeroValue;
			Size = FIntVector::ZeroValue;
		}
		
		EnqueueCopy(RHICmdList, SourceTexture, SourcePosition, SourceSlice, Size);
	}

	/**
	 * Returns the CPU accessible pointer that backs this staging buffer.
	 * @param NumBytes The maximum number of bytes the host will read from this pointer.
	 * @returns A CPU accessible pointer to the backing buffer.
	 */
	UE_DEPRECATED(5.8, "FRHIGPUMemoryReadback::Lock is deprecated. Please use FRHIGPUBufferReadback::Lock or FRHIGPUTextureReadback::Lock directly.")
	void* Lock(uint32 NumBytes)
	{
		return nullptr;
	}

	/**
	 * Signals that the host is finished reading from the backing buffer.
	 */
	UE_DEPRECATED(5.8, "FRHIGPUMemoryReadback::Unlock is deprecated. Please use FRHIGPUBufferReadback::Unlock or FRHIGPUTextureReadback::Unlock directly.")
	void Unlock()
	{
	}

	inline const FRHIGPUMask& GetLastCopyGPUMask() const { return LastCopyGPUMask; }

	FName GetName() const { return Fence->GetFName(); }

protected:

	FGPUFenceRHIRef Fence;
	FRHIGPUMask LastCopyGPUMask;

	// We need to separately track which GPU buffer was last locked.  It's possible for a new copy operation to
	// be enqueued (writing to LastCopyGPUMask) while the buffer is technically locked, with the unlock and enqueued
	// copy on the GPU itself happening later, during pass execution in FRDGBuilder::Execute (for example, this
	// happens with Nanite streaming).  It's not unsafe, because the operations are occurring in order on both the
	// render thread and later pass Execute, but our locking logic needs to handle that scenario.
	uint32 LastLockGPUIndex;
};

/** Buffer readback implementation. */
class FRHIGPUBufferReadback final : public FRHIGPUMemoryReadback
{
public:

	RHI_API FRHIGPUBufferReadback(FName RequestName);
	 
	RHI_API void EnqueueCopy(FRHICommandList& RHICmdList, FRHIBuffer* SourceBuffer, uint32 NumBytes = 0) override;
	RHI_API void* Lock(uint32 NumBytes);
	RHI_API void Unlock();
	RHI_API uint64 GetGPUSizeBytes() const;

private:

	// RHI staging buffers are single GPU -- need to be branched when using multiple GPUs
	FStagingBufferRHIRef DestinationStagingBuffers[MAX_NUM_GPUS];
};


/** Texture readback implementation. */
class FRHIGPUTextureReadback final : public FRHIGPUMemoryReadback
{
public:
	RHI_API FRHIGPUTextureReadback(FName RequestName);

	using FRHIGPUMemoryReadback::EnqueueCopy;

	RHI_API void EnqueueCopy(FRHICommandList& RHICmdList, FRHITexture* SourceTexture, const FIntVector& SourcePosition, uint32 SourceSlice, const FIntVector& Size) override;

	RHI_API void* Lock(int32& OutRowPitchInPixels, int32* OutBufferHeight = nullptr);
	RHI_API void Unlock();

	RHI_API uint64 GetGPUSizeBytes() const;

	FTextureRHIRef DestinationStagingTextures[MAX_NUM_GPUS];
};
