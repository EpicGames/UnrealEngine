// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/IntVector.h"
#include "Math/Vector2D.h"
#include "PixelFormat.h"
#include "RHIDefinitions.h"
#include "RHIFeatureLevel.h"
#include "RHIShaderPlatform.h"

#if WITH_EDITOR
#include "Misc/Optional.h"
#endif

/** The minimum Z value in clip space for the RHI. This is a constant value to always match D3D clip-space. */
inline constexpr float GMinClipZ = 0.0f;

/** The sign to apply to the Y axis of projection matrices. This is a constant value to always match D3D clip-space. */
inline constexpr float GProjectionSignY = 1.0f;

/** Info for supporting the vertex element types */
class FVertexElementTypeSupportInfo
{
public:
	FVertexElementTypeSupportInfo() { for (int32 i = 0; i < VET_MAX; i++) ElementCaps[i] = true; }
	bool IsSupported(EVertexElementType ElementType) { return ElementCaps[ElementType]; }
	void SetSupported(EVertexElementType ElementType, bool bIsSupported) { ElementCaps[ElementType] = bIsSupported; }
private:
	/** cap bit set for each VET. One-to-one mapping based on EVertexElementType */
	bool ElementCaps[VET_MAX];
};

// Wrapper for GRHI## global variables, allows values to be overridden for mobile preview modes.
template <typename TValueType>
class TRHIGlobal
{
public:
	TRHIGlobal() = default;
	TRHIGlobal(const TValueType& InValue) : Value(InValue) {}

	TRHIGlobal& operator=(const TValueType& InValue)
	{
		Value = InValue;
		return *this;
	}

#if WITH_EDITOR
	inline void SetPreviewOverride(const TValueType& InValue)
	{
		PreviewValue = InValue;
	}

	inline operator TValueType() const
	{
		return PreviewValue.IsSet() ? GetPreviewValue() : Value;
	}
#else
	inline operator TValueType() const { return Value; }
#endif

private:
	TValueType Value{};
#if WITH_EDITOR
	TOptional<TValueType> PreviewValue;
	TValueType GetPreviewValue() const { return PreviewValue.GetValue(); }
#endif
};

#if WITH_EDITOR
template<>
inline int32 TRHIGlobal<int32>::GetPreviewValue() const
{
	// ensure the preview values are subsets of RHI functionality.
	return FMath::Min(PreviewValue.GetValue(), Value);
}

template<>
inline bool TRHIGlobal<bool>::GetPreviewValue() const
{
	// ensure the preview values are subsets of RHI functionality.
	return PreviewValue.GetValue() && Value;
}
#endif

struct FRHIGlobals
{
	RHI_API FRHIGlobals();
	RHI_API ~FRHIGlobals();

	/** True if the render hardware has been initialized. */
	bool IsRHIInitialized = false;

	/** Optimal number of persistent thread groups to fill the GPU. */
	int32 PersistentThreadGroupCount = 0;

	/** The maximum number of mip-maps that a texture can contain. */
	int32 MaxTextureMipCount = MAX_TEXTURE_MIP_COUNT;

	/** true if this platform has quad buffer stereo support. */
	bool SupportsQuadBufferStereo = false;

	/** true if the RHI supports textures that may be bound as both a render target and a shader resource. */
	bool SupportsRenderDepthTargetableShaderResources = true;

	/** true if the RHI supports Draw Indirect */
	bool SupportsDrawIndirect = true;

	/** true if the RHI supports Multi Draw Indirect, a variant of Draw Indirect with variable number of sub-commands generated by GPU */
	bool SupportsMultiDrawIndirect = false;

	/** Whether the RHI can send commands to the device context from multiple threads. Used in the GPU readback to avoid stalling the RHI threads. */
	bool SupportsMultithreading = false;

	/** Whether RHIGetRenderQueryResult can be safely called off the render thread. */
	bool SupportsAsyncGetRenderQueryResult = false;

	struct FGpuInfo
	{
		/**
		 * only set if RHI has the information (after init of the RHI and only if RHI has that information, never changes after that)
		 * e.g. "NVIDIA GeForce GTX 670"
		 */
		FString AdapterName;
		FString AdapterInternalDriverVersion;
		FString AdapterUserDriverVersion;
		FString AdapterDriverDate;
		bool AdapterDriverOnDenyList = false;
		uint32 DeviceId = 0;
		uint32 DeviceRevision = 0;

		/** true if the GPU is AMD's Pre-GCN architecture */
		bool IsAMDPreGCNArchitecture = false;

		// 0 means not defined yet, use functions like IsRHIDeviceAMD() to access
		uint32 VendorId = 0;

		// Vendor-specific generation number of the GPU. 0 if unknown.
		uint32 AdapterGeneration = 0;

		// Amount of VRAM (currently implemented for D3D11/D3D12).
		uint64 DedicatedVideoMemory = 0;
	} GpuInfo;

	// true if the RHI supports Pixel Shader UAV
	bool SupportsPixelShaderUAVs = true;

	// true if the RHI supports Vertex Shader UAV
	bool SupportsVertexShaderUAVs = false;

	/** true if PF_G8 render targets are supported */
	TRHIGlobal<bool> SupportsRenderTargetFormat_PF_G8 = true;

	/** true if PF_FloatRGBA render targets are supported */
	TRHIGlobal<bool> SupportsRenderTargetFormat_PF_FloatRGBA = true;

	/** true if mobile framebuffer fetch is supported */
	bool SupportsShaderFramebufferFetch = false;

	/** true if mobile framebuffer fetch is supported from MRT's*/
	bool SupportsShaderMRTFramebufferFetch = false;

	/** true if mobile framebuffer fetch can be used for programmable blending, does not imply that framebuffer fetch is supported*/
	bool SupportsShaderFramebufferFetchProgrammableBlending = true;

	/** true if mobile pixel local storage is supported */
	bool SupportsPixelLocalStorage = false;

	/** true if mobile depth & stencil fetch is supported */
	bool SupportsShaderDepthStencilFetch = false;

	/** true if RQT_AbsoluteTime is supported by RHICreateRenderQuery */
	bool SupportsTimestampRenderQueries = false;

	/** true if RQT_AbsoluteTime is supported by RHICreateRenderQuery */
	bool SupportsGPUTimestampBubblesRemoval = false;

	/** true if RHIGetGPUFrameCycles removes CPu generated bubbles. */
	bool SupportsFrameCyclesBubblesRemoval = false;

	/** true if RHIGetGPUUsage() is supported. */
	bool SupportsGPUUsage = false;

	/** true if the GPU supports hidden surface removal in hardware. */
	bool HardwareHiddenSurfaceRemoval = false;

	/** true if the RHI supports asynchronous creation of texture resources */
	bool SupportsAsyncTextureCreation = false;

	/** true if the RHI supports quad topology (PT_QuadList). */
	bool SupportsQuadTopology = false;

	/** true if the RHI supports rectangular topology (PT_RectList). */
	bool SupportsRectTopology = false;

	/** true if the RHI supports primitive shaders. */
	bool SupportsPrimitiveShaders = false;

	/** true if the RHI supports 64 bit uint atomics. */
	bool SupportsAtomicUInt64 = false;

	/** true if the RHI supports 64 bit uint atomics using DX12 SM6.6 - TEMP / DEPRECATED - DO NOT USE */
	bool SupportsDX12AtomicUInt64 = false;

	/** true if the RHI supports optimal low level pipeline state sort keys. */
	bool SupportsPipelineStateSortKey = false;

	/** Temporary. When OpenGL is running in a separate thread, it cannot yet do things like initialize shaders that are first discovered in a rendering task. It is doable, it just isn't done. */
	bool SupportsParallelRenderingTasksWithSeparateRHIThread = true;

	/** If an RHI is so slow, that it is the limiting factor for the entire frame, we can kick early to try to give it as much as possible. */
	bool RHIThreadNeedsKicking = false;

	/** The maximum number of in-flight GPU queries the current RHI can handle without stalling and waiting for the GPU. Used to tune the occlusion culler to avoid stalls. */
	int32 MaximumInFlightQueries = MAX_int32;

	/** Some RHIs can only do visible or not occlusion queries. */
	bool SupportsExactOcclusionQueries = true;

	/** True if and only if the GPU support rendering to volume textures (2D Array, 3D). Some OpenGL 3.3 cards support SM4, but can't render to volume textures. */
	bool SupportsVolumeTextureRendering = true;

	/** True if the RHI supports separate blend states per render target. */
	bool SupportsSeparateRenderTargetBlendState = false;

	/** True if the RHI supports dual src blending. */
	bool SupportsDualSrcBlending = true;

	/** True if the RHI has artifacts with atlased CSM depths. */
	bool NeedsUnatlasedCSMDepthsWorkaround = false;

	/** Whether to initialize 3D textures using a bulk data (or through a mip update if false). */
	bool SupportsTexture3D = true;

	/** Whether RHI supports anisotropic filtering for 3D textures. */
	bool SupportsTexture3DAnisotropicFiltering = true;

	/** true if bulk data should be used with 3d textures */
	bool UseTexture3DBulkData = false;

	/** true if the RHI supports mobile multi-view */
	bool SupportsMobileMultiView = false;

	/** true if the RHI supports image external */
	bool SupportsImageExternal = false;

	/** true if the RHI supports 256bit MRT */
	bool SupportsWideMRT = true;

	/** True if the RHI and current hardware supports supports depth bounds testing */
	bool SupportsDepthBoundsTest = false;

	/** True if the RHI supports explicit access to depth target HTile meta data. */
	bool SupportsExplicitHTile = false;

	/** True if the RHI supports explicit access to MSAA target FMask meta data. */
	bool SupportsExplicitFMask = false;

	/** True if the RHI supports resummarizing depth target HTile meta data. */
	bool SupportsResummarizeHTile = false;

	/** True if the RHI supports depth target unordered access views. */
	bool SupportsDepthUAV = false;

	/** True if the RHI and current hardware supports efficient AsyncCompute (by default we assume false and later we can enable this for more hardware) */
	bool SupportsEfficientAsyncCompute = false;

	/** True if the RHI supports aliasing transient resources on the async compute pipe. */
	bool SupportsAsyncComputeTransientAliasing = false;

	/** True if the RHI supports getting the result of occlusion queries when on a thread other than the render thread */
	bool SupportsParallelOcclusionQueries = false;

	/** true if the RHI requires a valid RT bound during UAV scatter operation inside the pixel shader */
	bool RequiresRenderTargetForPixelShaderUAVs = false;

	/** true if the RHI supports unordered access view format aliasing */
	bool SupportsUAVFormatAliasing = false;

	/** true if the RHI supports texture views (data aliasing) */
	bool SupportsTextureViews = true;

	/** true if the pointer returned by Lock is a persistent direct pointer to gpu memory */
	bool SupportsDirectGPUMemoryLock = false;

	/** true if the multi-threaded shader creation is supported by (or desirable for) the RHI. */
	bool SupportsMultithreadedShaderCreation = true;

	/** Does the RHI support parallel resource commands (i.e. create / lock / unlock) on non-immediate command list APIs, recorded off the render thread. */
	bool SupportsMultithreadedResources = false;

	/** Does this RHI need to wait for deletion of resources due to ref counting. */
	bool NeedsExtraDeletionLatency = false;

	/** Allow opt-out default RHI resource deletion latency for streaming textures */
	bool ForceNoDeletionLatencyForStreamingTextures = false;

	/** The maximum size allowed for a computeshader dispatch. */
	TRHIGlobal<int32> MaxComputeDispatchDimension = ((1 << 16) - 1);

	/** If true, then avoiding loading shader code and instead force the "native" path, which sends a library and a hash instead. */
	bool LazyShaderCodeLoading = false;

	/** If true, then it is possible to turn on LazyShaderCodeLoading. */
	bool SupportsLazyShaderCodeLoading = false;

	/** true if the RHI supports UpdateFromBufferTexture method */
	bool SupportsUpdateFromBufferTexture = false;

	/** The maximum size to allow for the shadow depth buffer in the X dimension.  This must be larger or equal to GMaxShadowDepthBufferSizeY. */
	TRHIGlobal<int32> MaxShadowDepthBufferSizeX = 2048;

	/** The maximum size to allow for the shadow depth buffer in the Y dimension. */
	TRHIGlobal<int32> MaxShadowDepthBufferSizeY = 2048;

	/** The maximum size allowed for 2D textures in both dimensions. */
	TRHIGlobal<int32> MaxTextureDimensions = 2048;

	/** The maximum number of elements allowed for Typed Buffers. */
	TRHIGlobal<uint64> MaxViewDimensionForTypedBuffer = (1ULL << 32);

	/** The maximum size(in Bytes) for non Typed Buffers. */
	TRHIGlobal<uint64> MaxViewSizeBytesForNonTypedBuffer = (1ULL << 32);

	/** The maximum size allowed for a contant buffer. */
	TRHIGlobal<int64> MaxConstantBufferByteSize = (1 << 27);

	/** The maximum size allowed for Shared Compute Memory. */
	TRHIGlobal<int64> MaxComputeSharedMemory = (1 << 15);

	/** The maximum size allowed for 3D textures in all three dimensions. */
	TRHIGlobal<int32> MaxVolumeTextureDimensions = 2048;

	/** The maximum size allowed for cube textures. */
	TRHIGlobal<int32> MaxCubeTextureDimensions = 2048;

	/** Whether RW texture buffers are supported */
	bool SupportsRWTextureBuffers = true;

	/** Whether a raw (ByteAddress) buffer view can be created for any buffer, regardless of its EBufferUsageFlags::ByteAddressBuffer flag. */
	bool SupportsRawViewsForAnyBuffer = false;

	/** Whether depth or stencil can individually be set to CopySrc/Dest access. */
	bool SupportsSeparateDepthStencilCopyAccess = true;

	/** Support using async thread for texture stream out operations */
	bool SupportsAsyncTextureStreamOut = false;

	/** The Maximum number of layers in a 1D or 2D texture array. */
	int32 MaxTextureArrayLayers = 256;

	int32 MaxTextureSamplers = 16;

	/** The guaranteed minimum number of simultaneously bound UAV's across all stages in a single pipeline */
	static constexpr int32 MinGuaranteedSimultaneousUAVs = 8;

	/** The maximum number of simultaneously bound UAV's across all stages in a single pipeline */
	int32 MaxSimultaneousUAVs = MinGuaranteedSimultaneousUAVs;

	/** The maximum work group invocations allowed for compute shader. */
	TRHIGlobal<int32> MaxWorkGroupInvocations = 1024;

	/** true if we are running with the NULL RHI */
	bool UsingNullRHI = false;

	/**
	 *	The size to check against for Draw*UP call vertex counts.
	 *	If greater than this value, the draw call will not occur.
	 */
	int32 DrawUPVertexCheckCount = MAX_int32;
	/**
	 *	The size to check against for Draw*UP call index counts.
	 *	If greater than this value, the draw call will not occur.
	 */
	int32 DrawUPIndexCheckCount = MAX_int32;

	/** true for each VET that is supported. One-to-one mapping with EVertexElementType */
	FVertexElementTypeSupportInfo VertexElementTypeSupport;

	/** Whether an intentional GPU crash has been scheduled. */
	ERequestedGPUCrash TriggerGPUCrash = ERequestedGPUCrash::None;

	/** Non-empty if we are performing a gpu trace. Also says where to place trace file. */
	FString GPUTraceFileName;

	/** True if the RHI supports texture streaming */
	bool SupportsTextureStreaming = false;

	/** Amount of memory allocated by streaming textures. In kilobytes. */
	volatile uint64 StreamingTextureMemorySizeInKB = 0;

	/** Amount of memory allocated by non streaming textures. In kilobytes. */
	volatile uint64 NonStreamingTextureMemorySizeInKB = 0;

	/** Current texture streaming pool size, in bytes. 0 means unlimited. */
	int64 TexturePoolSize = 0 * 1024 * 1024;

	/** In percent. If non-zero, the texture pool size is a percentage of TotalGraphicsMemory. */
	int32 PoolSizeVRAMPercentage = 0;

	/** Amount of local video memory demoted to system memory. In bytes. */
	uint64 DemotedLocalMemorySize = 0;

	/** Amount of memory allocated by buffers */
	volatile uint64 BufferMemorySize = 0;

	/** Amount of memory allocated by uniform buffers */
	volatile uint64 UniformBufferMemorySize = 0;

	/** Whether or not the RHI can handle a non-zero BaseVertexIndex - extra SetStreamSource calls will be needed if this is false */
	bool SupportsBaseVertexIndex = true;

	/** Whether or not the RHI can handle a non-zero FirstInstance to DrawIndexedPrimitive and friends - extra SetStreamSource calls will be needed if this is false */
	bool SupportsFirstInstance = false;

	/** Whether or not the RHI can handle dynamic resolution or not. */
	bool SupportsDynamicResolution = false;

	struct FRayTracing
	{
		/**
		* Whether or not the RHI supports ray tracing on current hardware (acceleration structure building and new ray tracing-specific shader types).
		* SupportsRayTracingShaders and SupportsInlineRayTracing must also be checked before dispatching ray tracing workloads.
		*/
		bool Supported = false;

		/**
		* Whether or not the RHI supports ray tracing raygen, miss and hit shaders (i.e. full ray tracing pipeline).
		* The RHI may support inline ray tracing from compute shaders, but not the full pipeline.
		*/
		bool SupportsShaders = false;

		/** Whether or not the RHI supports adding new shaders to an existing RT PSO. */
		bool SupportsPSOAdditions = false;

		/** Whether or not the RHI supports indirect ray tracing dispatch commands. */
		bool SupportsDispatchIndirect = false;

		/** Whether or not the RHI supports async building ray tracing acceleration structures. */
		bool SupportsAsyncBuildAccelerationStructure = false;

		/** Whether or not the RHI supports indirect building of ray tracing top level acceleration structures. */
		bool SupportsBuildTopLevelAccelerationStructureIndirect = false;
		
		/** Whether or not the RHI supports async ray trace dispatch calls. */
		bool SupportsAsyncRayTraceDispatch = false;

		/** Whether or not the RHI supports NVIDIA Cluster Operations **/
		bool SupportsClusterOps = false;

		/** Whether or not the RHI supports the AMD Hit Token extension. */
		bool SupportsAMDHitToken = false;

		/** Whether or not the RHI supports inline ray tracing in compute shaders, without a full ray tracing pipeline. */
		bool SupportsInlineRayTracing = false;
		
		/** Whether or not the RHI requires a SBT for inline ray tracing in compute shaders to fetch geometry information */
		bool RequiresInlineRayTracingSBT = false;
		
		/** Whether or not the RHI supports inlined callbacks */
		bool SupportsInlinedCallbacks = false;

		/** Whether or not the RHI supports persistent SBTs */
		bool SupportsPersistentSBTs = false;

		/** Wether an extra uniform buffer parameter is required when loose parameters are present, or if they are stored directly in the shader record. */
		bool SupportsLooseParamsInShaderRecord = false;

		/** Whether or not the RHI supports compacting acceleration structures */
		bool SupportsAccelerationStructureCompaction = false;

		/** Whether or not the RHI supports serializing acceleration structures to a file */
		bool SupportsSerializeAccelerationStructure = false;
		
		/** Whether or not the RHI supports a separate hit group buffer when building the acceleration structure */
		bool RequiresSeparateHitGroupContributionsBuffer = false;

		/** Whether or not the RHI supports linear swept sphere primitives **/
		bool SupportsLinearSweptSpheres = false;

		/** Required alignment for ray tracing acceleration structures. */
		uint32 AccelerationStructureAlignment = 0;

		/** Required alignment for ray tracing cluster-level acceleration structures. */
		uint32 ClusterAccelerationStructureAlignment = 0;

		/** Required alignment for ray tracing cluster-level acceleration structure templates. */
		uint32 ClusterAccelerationStructureTemplateAlignment = 0;

		/** Required alignment for ray tracing scratch buffers. */
		uint32 ScratchBufferAlignment = 0;

		/** Required alignment for ray tracing shader binding table buffer. */
		uint32 ShaderTableAlignment = 0;

		/** Size of an individual element in the ray tracing instance buffer. This defines the required stride and alignment of structured buffers of instances. */
		uint32 InstanceDescriptorSize = 0;

	} RayTracing;

	/** Whether or not the RHI supports shader wave operations (shader model 6.0). */
	bool SupportsWaveOperations = false;

	/** Whether or not the current GPU is integrated into to CPU */
	bool DeviceIsIntegrated = false;

	/** Whether or not the current device has a Unified Memory Architecture (UMA) - Memory Shared across GPU and CPU but separate Virtual Addresses*/
	bool DeviceIsUMASeparateVA = false;

	/** Whether or not the current device UMA is Cache Coherent between the GPU and the CPU */
	bool DeviceIsCacheCoherentUMA = false;

	/** Whether or not the current device supports GPU Uploads Heaps (ReBar) */
	bool DeviceSupportsGPUUploadHeaps = false;

	/**
	* Specifies the minimum and maximum number of lanes in the SIMD wave that this GPU can support. I.e. 32 on NVIDIA, 64 on AMD.
	* Valid values are in range [4..128] (as per SM 6.0 specification) or 0 if unknown.
	* Rendering code must always check SupportsWaveOperations in addition to wave min/max size.
	*/
	int32 MinimumWaveSize = 0;
	int32 MaximumWaveSize = 0;

	/** Whether or not the RHI supports 16bit VALU and resource loads (HLSL shader model 6.2). */
	bool SupportsNative16BitOps = false;

	/** Whether or not the RHI supports an RHI thread. */
	bool SupportsRHIThread = false;

	/* as above, but we run the commands on arbitrary task threads */
	bool SupportsRHIOnTaskThread = false;

	/** Whether or not the RHI supports parallel RHIThread executes / translates
	Requirements:
	* RHICreateBoundShaderState & RHICreateGraphicsPipelineState is threadsafe and GetCachedBoundShaderState must not be used. GetCachedBoundShaderState_Threadsafe has a slightly different protocol.
	***/
	bool SupportsParallelRHIExecute = false;
	
	/** 
	  * If true, RHI submit tasks (those that call RHISubmitCommandLists) are allowed to run concurrently with subsequent RHI translation tasks.
	  * This is only possible with RHIs that correctly pipeline viewport presentation, and don't implicitly submit GPU work from within RHI translate tasks.
	  * 
	  * If false, future RHI translate tasks cannot start until all prior RHI submit tasks have completed. Submit tasks run in isolation on the RHI thread.
	  */
	bool SupportsConcurrentTranslateAndSubmit = false;

	struct
	{
		/** Enables support for platforms that use render passes to split command list translation across multiple threads. For platforms that require
		   dependencies on Parent/Child command lists i.e. requiring translation of children to be complete before finishing translation of the
		   parent, should use ParentWaitForChildTranslate and/or ChildWaitForParentBegin */
		bool Supported = false;

		/** When true, signals to the renderer that the platform RHI requires a call to RHIGetParallelCommandContext to retrieve RHI contexts for use with parallel render passes.
		    Otherwise, regular graphics contexts are used. */
		bool UsesCustomContexts = false;

		/** When true, indicates that the platform RHI requires that the child command list translations in a parallel render pass are completed before the remainder of the parent command list can be translated. */
		bool ParentWaitForChildTranslate = false;

		/** When true, indicates that the platform RHI requires that child command list translations in a parallel render pass do not start until the parent command list translation has reached the BeginParallelRenderPass command. */
		bool ChildWaitForParentBegin = false;

	} ParallelRenderPasses;

	/** Whether or not the RHI can perform MSAA sample load. */
	bool SupportsMSAADepthSampleAccess = false;

	/** Whether or not the RHI can render to the backbuffer with a custom depth/stencil surface bound. */
	bool SupportsBackBufferWithCustomDepthStencil = true;

	/** Whether or not HDR is currently enabled */
	bool IsHDREnabled = false;

	/** Whether the present adapter/display offers HDR output capabilities. */
	bool SupportsHDROutput = false;

	/** The maximum number of groups that can be dispatched in each dimensions. */
	FIntVector MaxDispatchThreadGroupsPerDimension = FIntVector::ZeroValue;

	struct FVariableRateShading
	{
		/** Whether or not the RHI can support per-draw Variable Rate Shading. */
		bool SupportsPipeline = false;

		/** Whether or not the Variable Rate Shading can be done at larger (2x4 or 4x2 or 4x4) sizes. */
		bool SupportsLargerSizes = false;

		/** Whether or not the RHI can support image-based Variable Rate Shading. */
		bool SupportsAttachment = false;

		/** Whether or not the RHI can support complex combiner operatations between per-draw (pipeline) VRS and image VRS. */
		bool SupportsComplexCombinerOps = false;

		/** Whether or not the RHI can support shading rate attachments as array textures. */
		bool SupportsAttachmentArrayTextures = false;

		/** Maximum tile width in a screen space texture that can be used to drive Variable Rate Shading. */
		int32 ImageTileMaxWidth = 0;

		/** Maximum tile height in a screen space texture that can be used to drive Variable Rate Shading. */
		int32 ImageTileMaxHeight = 0;

		/** Minimum tile width in a screen space texture that can be used to drive Variable Rate Shading. */
		int32 ImageTileMinWidth = 0;

		/** Minimum tile height in a screen space texture that can be used to drive Variable Rate Shading. */
		int32 ImageTileMinHeight = 0;

		/** Data type contained in a shading-rate image for image-based Variable Rate Shading. */
		EVRSImageDataType ImageDataType = VRSImage_NotSupported;

		/** Image format for the shading rate image for image-based Variable Rate Shading. */
		EPixelFormat ImageFormat = PF_Unknown;

		/** Whether Variable Rate Shading deferred shading rate texture update is supported. */
		bool SupportsLateUpdate = false;

		/** Whether or not the RHI can support offsets for image-based Variable Rate Shading. Currently requires VK_EXT_fragment_density_map_offset to be supported and enabled. */
		bool SupportsImageOffsets = false;
		
		/** Current offsets of the shading rate image, in pixels. NOTE: This MUST be quantized by the amount specified in ImageOffsetGranularity below! */
		FIntPoint ImageOffsets[2];
		
		/** The granularity of the offset, in pixels, required by the RHI. This is queried at runtime, and must be used to enforce granularity for the ImageOffsets above. */
		FIntPoint ImageOffsetGranularity;

	} VariableRateShading;

	/** Format used for the backbuffer when outputting to a HDR display. */
	EPixelFormat HDRDisplayOutputFormat = PF_FloatRGBA;

	/** Counter incremented once on each frame present. Used to support game thread synchronization with swap chain frame flips. */
	uint64 PresentCounter = 1;

	/** True if the RHI supports setting the render target array index from any shader stage */
	bool SupportsArrayIndexFromAnyShader = false;

	/** True if the pipeline file cache can be used with this RHI */
	bool SupportsPipelineFileCache = false;

	/** True if the RHI support PSO precaching */
	bool SupportsPSOPrecaching = false;

	/** True if the RHI supports setting the stencil ref at pixel granularity from the pixel shader */
	bool SupportsStencilRefFromPixelShader = false;

	/** True if the RHI supports raster order views. */
	bool SupportsRasterOrderViews = false;

	/** Whether current RHI supports overestimated conservative rasterization. */
	bool SupportsConservativeRasterization = false;

	/** Whether current RHI supports shader root constants. */
	bool SupportsShaderRootConstants = false;

	struct FShaderBundles
	{
		/** Whether current RHI supports native shader bundle dispatch. */
		bool SupportsDispatch = false;

		/** Whether current RHI supports shader bundle dispatch using work graphs. */
		bool SupportsWorkGraphDispatch = false;

		/** Whether current RHI supports shader bundle graphics dispatch using work graphs. */
		bool SupportsWorkGraphGraphicsDispatch = false;

		/** Whether current RHI supports shader bundle dispatch and RHI parallel translate. */
		bool SupportsParallel = false;

		/** Whether the current RHI requires shared bindless parameters. */
		bool RequiresSharedBindlessParameters = false;

	} ShaderBundles;

	/** true if the RHI supports Mesh and Amplification shaders with tier0 capability */
	bool SupportsMeshShadersTier0 = false;

	/** true if the RHI supports Mesh and Amplification shaders with tier1 capability */
	bool SupportsMeshShadersTier1 = false;

	/** Whether current RHI supports work graphs with tier1_0 capability. */
	bool SupportsShaderWorkGraphsTier1 = false;

	/** Whether current RHI supports work graphs with tier1_1 capability. */
	bool SupportsShaderWorkGraphsTier1_1 = false;

	/**
	* True if the RHI supports reading system timer in shaders via GetShaderTimestamp().
	* Individual shaders must be compiled with an appropriate vendor extension and check PLATFORM_SUPPORTS_SHADER_TIMESTAMP.
	*/
	bool SupportsShaderTimestamp = false;

	bool SupportsEfficientUploadOnResourceCreation = false;

	/** true if the RHI supports RLM_WriteOnly_NoOverwrite */
	bool SupportsMapWriteNoOverwrite = false;

	/** Tables of all MSAA sample offset for all MSAA supported. Use GetMSAASampleOffsets() to read it. */
	FVector2f DefaultMSAASampleOffsets[1 + 2 + 4 + 8 + 16];

	/** True if the RHI supports pipeline precompiling from any thread. */
	bool SupportsAsyncPipelinePrecompile = true;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "BindlessSupport is not used anymore, it is now a bool in bSupportsBindless")
	ERHIBindlessSupport BindlessSupport = ERHIBindlessSupport::Unsupported;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Whether dynamic (bindless) resources are supported. Can be set even if bindless is not enabled. */
	bool bSupportsBindless = false;

	/** Whether bindless descriptor ranges are supported. Can be set even if bindless is not enabled. */
	bool bSupportsDescriptorRange = false;

	/** Array of descriptor type masks showing which descriptor types can be allocated together. */
	TArray<ERHIDescriptorTypeMask> DescriptorTypeGroups;

	struct FReservedResources
	{
		/**
		* True if the RHI supports reserved (AKA tiled, virtual or sparse) resources and operations related to them.
		* Buffers and 2D textures (without mips) can be created with ReservedResource flag.
		*/
		bool Supported = false;

		/**
		* True if the RHI supports creating volume textures with ReservedResource flag.
		*/
		bool SupportsVolumeTextures = false;

		/**
		* Smallest mip dimension of reserved texture arrays must be greater or equal to this value.
		* Tiled/reserved resources with both more than one array slice and any mipmap that
		* has a dimension less than a tile in extent are not supported by some hardware.
		* This is a conservative value chosen by the engine, independent of the texture format for simplicity.
		*/
		int32 TextureArrayMinimumMipDimension = 256;

		/**
		* Size that corresponds to a minimum unit of physical memory that may be mapped to a region of a reserved resource.
		* High-level code should aim to allocate reserved resources such that their size is a multiple of this tile size.
		* Guaranteed to be the same value on all platforms, regardless of the native virtual memory page size.
		*/
		static constexpr int32 TileSizeInBytes = 65536;

		/** Current size of the virtual address space allocated for all reserved buffers and textures */
		volatile int64 VirtualSize = 0;

	} ReservedResources;

	/** Table for finding out which shader platform corresponds to a given feature level for this RHI. */
	EShaderPlatform ShaderPlatformForFeatureLevel[ERHIFeatureLevel::Num];
	
	/** True if the RHI has initialized a device with the debug layer enabled. */
	bool IsDebugLayerEnabled = false;

	/** True if the RHI needs shader unbinds (SetShaderUnbinds). RHIs that don't need them can avoid creating extra commands. */
	bool NeedsShaderUnbinds = false;

	/** True if the RHI supports shaders with barycentrics */
	bool SupportsBarycentricsSemantic = false;

	/** True if RHI supports MSAA resolve with a custom shader */
	bool SupportsMSAAShaderResolve = false;

	/** Whether Depth Stencil MSAA Resolve Targets are supported. */
	bool SupportsDepthStencilResolve = false;

	/** True if RHI supports Linear texture format in 3D/Cube/Array texture */
	bool SupportLinearTextureVolumeFormat = true;

	/** true if the RHI supports shader execution reordering. */
	bool SupportsShaderExecutionReordering = false;

	/** True if RHI needs the transitions from the high level for COPYSRC/COPYDEST*/
	bool NeedsExtraTransitions = false;
	
	/** True if RHI needs to track states when transitioning between Discard. */
	bool NeedsTransientDiscardStateTracking = false;

	/** True if RHI is unable to service transitioning to ERHIAccess::Discard from all pipes to async compute. */
	bool NeedsTransientDiscardOnGraphicsWorkaround = false;

	/** True if lossy/fixed-rate compression of render targets is supporterd. */
	bool SupportsLossyFramebufferCompression = false;

	/** True if RHI supports supports using any slice or all slices of a texture array as render targets without creating the texture with the TargetArraySlicesIndependently flag. */
	bool SupportsBindingTexArrayPerSlice = false;

	/** True if RHI needs to manually apply an RHI transition when a resource is flag as SRVGraphicsNonPixel. */
	bool NeedsSRVGraphicsNonPixelWorkaround = false;

	/** True if RHI expects RDG to produce dedicated SRVGraphicsPixel/SRVGraphicsNonPixel access for texture srv parameters (instead of using the SRVGraphics mask everywhere). */
	bool SupportsDedicatedSRVGraphicsNonPixelAccess = false;

	/** True if the platform supports split barriers. */
	bool SupportsSplitBarriers = false;

	/** Preferred format of FShaderCode (binary code or microcode). Only used for preview platforms with more than one shader code being supported such as D3D12 (supports DXIL and DXBC). */
	FName PreferredPreviewShaderCodeFormat = NAME_None;
};

extern RHI_API FRHIGlobals GRHIGlobals;

//
// Deprecated old-style names

#define GIsRHIInitialized                                      GRHIGlobals.IsRHIInitialized
#define GRHIPersistentThreadGroupCount                         GRHIGlobals.PersistentThreadGroupCount
#define GMaxTextureMipCount                                    GRHIGlobals.MaxTextureMipCount
#define GSupportsQuadBufferStereo                              GRHIGlobals.SupportsQuadBufferStereo
#define GSupportsRenderDepthTargetableShaderResources          GRHIGlobals.SupportsRenderDepthTargetableShaderResources
#define GRHISupportsDrawIndirect                               GRHIGlobals.SupportsDrawIndirect
#define GRHISupportsMultiDrawIndirect                          GRHIGlobals.SupportsMultiDrawIndirect
#define GRHISupportsMultithreading                             GRHIGlobals.SupportsMultithreading
#define GRHISupportsAsyncGetRenderQueryResult                  GRHIGlobals.SupportsAsyncGetRenderQueryResult
#define GRHIAdapterName                                        GRHIGlobals.GpuInfo.AdapterName
#define GRHIAdapterInternalDriverVersion                       GRHIGlobals.GpuInfo.AdapterInternalDriverVersion
#define GRHIAdapterUserDriverVersion                           GRHIGlobals.GpuInfo.AdapterUserDriverVersion
#define GRHIAdapterDriverDate                                  GRHIGlobals.GpuInfo.AdapterDriverDate
#define GRHIAdapterDriverOnDenyList                            GRHIGlobals.GpuInfo.AdapterDriverOnDenyList
#define GRHIDeviceId                                           GRHIGlobals.GpuInfo.DeviceId
#define GRHIDeviceRevision                                     GRHIGlobals.GpuInfo.DeviceRevision
#define GRHIVendorId                                           GRHIGlobals.GpuInfo.VendorId
#define GRHISupportsPixelShaderUAVs                            GRHIGlobals.SupportsPixelShaderUAVs
#define GRHIDeviceIsAMDPreGCNArchitecture                      GRHIGlobals.GpuInfo.IsAMDPreGCNArchitecture
#define GSupportsRenderTargetFormat_PF_G8                      GRHIGlobals.SupportsRenderTargetFormat_PF_G8
#define GSupportsRenderTargetFormat_PF_FloatRGBA               GRHIGlobals.SupportsRenderTargetFormat_PF_FloatRGBA
#define GSupportsShaderFramebufferFetch                        GRHIGlobals.SupportsShaderFramebufferFetch
#define GSupportsShaderFramebufferFetchProgrammableBlending    GRHIGlobals.SupportsShaderFramebufferFetchProgrammableBlending
#define GSupportsShaderMRTFramebufferFetch                     GRHIGlobals.SupportsShaderMRTFramebufferFetch
#define GSupportsPixelLocalStorage                             GRHIGlobals.SupportsPixelLocalStorage
#define GSupportsShaderDepthStencilFetch                       GRHIGlobals.SupportsShaderDepthStencilFetch
#define GSupportsTimestampRenderQueries                        GRHIGlobals.SupportsTimestampRenderQueries
#define GRHISupportsGPUTimestampBubblesRemoval                 GRHIGlobals.SupportsGPUTimestampBubblesRemoval
#define GRHISupportsFrameCyclesBubblesRemoval                  GRHIGlobals.SupportsFrameCyclesBubblesRemoval
#define GRHISupportsGPUUsage                                   GRHIGlobals.SupportsGPUUsage
#define GHardwareHiddenSurfaceRemoval                          GRHIGlobals.HardwareHiddenSurfaceRemoval
#define GRHISupportsAsyncTextureCreation                       GRHIGlobals.SupportsAsyncTextureCreation
#define GRHISupportsQuadTopology                               GRHIGlobals.SupportsQuadTopology
#define GRHISupportsRectTopology                               GRHIGlobals.SupportsRectTopology
#define GRHISupportsPrimitiveShaders                           GRHIGlobals.SupportsPrimitiveShaders
#define GRHISupportsAtomicUInt64                               GRHIGlobals.SupportsAtomicUInt64
#define GRHISupportsDX12AtomicUInt64                           GRHIGlobals.SupportsDX12AtomicUInt64
#define GRHISupportsPipelineStateSortKey                       GRHIGlobals.SupportsPipelineStateSortKey
#define GSupportsParallelRenderingTasksWithSeparateRHIThread   GRHIGlobals.SupportsParallelRenderingTasksWithSeparateRHIThread
#define GRHIThreadNeedsKicking                                 GRHIGlobals.RHIThreadNeedsKicking
#define GRHIMaximumInFlightQueries                             GRHIGlobals.MaximumInFlightQueries
#define GRHISupportsExactOcclusionQueries                      GRHIGlobals.SupportsExactOcclusionQueries
#define GSupportsVolumeTextureRendering                        GRHIGlobals.SupportsVolumeTextureRendering
#define GSupportsSeparateRenderTargetBlendState                GRHIGlobals.SupportsSeparateRenderTargetBlendState
#define GSupportsDualSrcBlending                               GRHIGlobals.SupportsDualSrcBlending
#define GRHINeedsUnatlasedCSMDepthsWorkaround                  GRHIGlobals.NeedsUnatlasedCSMDepthsWorkaround
#define GSupportsTexture3D                                     GRHIGlobals.SupportsTexture3D
#define GUseTexture3DBulkDataRHI                               GRHIGlobals.UseTexture3DBulkData
#define GSupportsMobileMultiView                               GRHIGlobals.SupportsMobileMultiView
#define GSupportsImageExternal                                 GRHIGlobals.SupportsImageExternal
#define GSupportsWideMRT                                       GRHIGlobals.SupportsWideMRT
#define GSupportsDepthBoundsTest                               GRHIGlobals.SupportsDepthBoundsTest
#define GRHISupportsExplicitHTile                              GRHIGlobals.SupportsExplicitHTile
#define GRHISupportsExplicitFMask                              GRHIGlobals.SupportsExplicitFMask
#define GRHISupportsResummarizeHTile                           GRHIGlobals.SupportsResummarizeHTile
#define GRHISupportsDepthUAV                                   GRHIGlobals.SupportsDepthUAV
#define GSupportsEfficientAsyncCompute                         GRHIGlobals.SupportsEfficientAsyncCompute
#define GSupportsParallelOcclusionQueries                      GRHIGlobals.SupportsParallelOcclusionQueries
#define GRHIRequiresRenderTargetForPixelShaderUAVs             GRHIGlobals.RequiresRenderTargetForPixelShaderUAVs
#define GRHISupportsUAVFormatAliasing                          GRHIGlobals.SupportsUAVFormatAliasing
#define GRHISupportsTextureViews                               GRHIGlobals.SupportsTextureViews
#define GRHISupportsDirectGPUMemoryLock                        GRHIGlobals.SupportsDirectGPUMemoryLock
#define GRHISupportsMultithreadedShaderCreation                GRHIGlobals.SupportsMultithreadedShaderCreation
#define GRHISupportsMultithreadedResources                     GRHIGlobals.SupportsMultithreadedResources
#define GRHINeedsExtraDeletionLatency                          GRHIGlobals.NeedsExtraDeletionLatency
#define GRHIForceNoDeletionLatencyForStreamingTextures         GRHIGlobals.ForceNoDeletionLatencyForStreamingTextures
#define GMaxComputeDispatchDimension                           GRHIGlobals.MaxComputeDispatchDimension
#define GRHILazyShaderCodeLoading                              GRHIGlobals.LazyShaderCodeLoading
#define GRHISupportsLazyShaderCodeLoading                      GRHIGlobals.SupportsLazyShaderCodeLoading
#define GRHISupportsUpdateFromBufferTexture                    GRHIGlobals.SupportsUpdateFromBufferTexture
#define GMaxShadowDepthBufferSizeX                             GRHIGlobals.MaxShadowDepthBufferSizeX
#define GMaxShadowDepthBufferSizeY                             GRHIGlobals.MaxShadowDepthBufferSizeY
#define GMaxTextureDimensions                                  GRHIGlobals.MaxTextureDimensions
#define GRHIMaxConstantBufferByteSize                          GRHIGlobals.MaxConstantBufferByteSize
#define GMaxComputeSharedMemory                                GRHIGlobals.MaxComputeSharedMemory
#define GMaxVolumeTextureDimensions                            GRHIGlobals.MaxVolumeTextureDimensions
#define GRHISupportsRWTextureBuffers                           GRHIGlobals.SupportsRWTextureBuffers
#define GRHISupportsRawViewsForAnyBuffer                       GRHIGlobals.SupportsRawViewsForAnyBuffer
#define GRHISupportsSeparateDepthStencilCopyAccess             GRHIGlobals.SupportsSeparateDepthStencilCopyAccess
#define GRHISupportAsyncTextureStreamOut                       GRHIGlobals.SupportsAsyncTextureStreamOut
#define GMaxCubeTextureDimensions                              GRHIGlobals.MaxCubeTextureDimensions
#define GMaxTextureArrayLayers                                 GRHIGlobals.MaxTextureArrayLayers
#define GMaxTextureSamplers                                    GRHIGlobals.MaxTextureSamplers
#define GMaxWorkGroupInvocations                               GRHIGlobals.MaxWorkGroupInvocations
#define GUsingNullRHI                                          GRHIGlobals.UsingNullRHI
#define GDrawUPVertexCheckCount                                GRHIGlobals.DrawUPVertexCheckCount
#define GDrawUPIndexCheckCount                                 GRHIGlobals.DrawUPIndexCheckCount
#define GVertexElementTypeSupport                              GRHIGlobals.VertexElementTypeSupport
#define GTriggerGPUCrash                                       GRHIGlobals.TriggerGPUCrash
#define GGPUTraceFileName                                      GRHIGlobals.GPUTraceFileName
#define GRHISupportsTextureStreaming                           GRHIGlobals.SupportsTextureStreaming
#define GTexturePoolSize                                       GRHIGlobals.TexturePoolSize
#define GPoolSizeVRAMPercentage                                GRHIGlobals.PoolSizeVRAMPercentage
#define GDemotedLocalMemorySize                                GRHIGlobals.DemotedLocalMemorySize
#define GRHISupportsBaseVertexIndex                            GRHIGlobals.SupportsBaseVertexIndex
#define GRHISupportsFirstInstance                              GRHIGlobals.SupportsFirstInstance
#define GRHISupportsDynamicResolution                          GRHIGlobals.SupportsDynamicResolution
#define GRHISupportsRayTracing                                 GRHIGlobals.RayTracing.Supported
#define GRHISupportsRayTracingShaders                          GRHIGlobals.RayTracing.SupportsShaders
#define GRHISupportsRayTracingPSOAdditions                     GRHIGlobals.RayTracing.SupportsPSOAdditions
#define GRHISupportsRayTracingDispatchIndirect                 GRHIGlobals.RayTracing.SupportsDispatchIndirect
#define GRHISupportsRayTracingAsyncBuildAccelerationStructure  GRHIGlobals.RayTracing.SupportsAsyncBuildAccelerationStructure
#define GRHISupportsRayTracingAMDHitToken                      GRHIGlobals.RayTracing.SupportsAMDHitToken
#define GRHISupportsInlineRayTracing                           GRHIGlobals.RayTracing.SupportsInlineRayTracing
#define GRHIRayTracingAccelerationStructureAlignment           GRHIGlobals.RayTracing.AccelerationStructureAlignment
#define GRHIRayTracingScratchBufferAlignment                   GRHIGlobals.RayTracing.ScratchBufferAlignment
#define GRHIRayTracingShaderTableAlignment                     GRHIGlobals.RayTracing.ShaderTableAlignment
#define GRHIRayTracingInstanceDescriptorSize                   GRHIGlobals.RayTracing.InstanceDescriptorSize
#define GRHISupportsWaveOperations                             GRHIGlobals.SupportsWaveOperations
#define GRHIDeviceIsIntegrated                                 GRHIGlobals.DeviceIsIntegrated
#define GRHIDeviceIsUMASeparateVA                              GRHIGlobals.DeviceIsUMASeparateVA
#define GRHIDeviceIsCacheCoherentUMA                           GRHIGlobals.DeviceIsCacheCoherentUMA
#define GRHIDeviceSupportsGPUUploadHeaps                       GRHIGlobals.DeviceSupportsGPUUploadHeaps
#define GRHIMinimumWaveSize                                    GRHIGlobals.MinimumWaveSize
#define GRHIMaximumWaveSize                                    GRHIGlobals.MaximumWaveSize
#define GRHISupportsRHIThread                                  GRHIGlobals.SupportsRHIThread
#define GRHISupportsRHIOnTaskThread                            GRHIGlobals.SupportsRHIOnTaskThread
#define GRHISupportsParallelRHIExecute                         GRHIGlobals.SupportsParallelRHIExecute
#define GRHISupportsParallelRHIExecute                         GRHIGlobals.SupportsParallelRHIExecute
#define GRHISupportsMSAADepthSampleAccess                      GRHIGlobals.SupportsMSAADepthSampleAccess
#define GRHISupportsBackBufferWithCustomDepthStencil           GRHIGlobals.SupportsBackBufferWithCustomDepthStencil
#define GRHIIsHDREnabled                                       GRHIGlobals.IsHDREnabled
#define GRHISupportsHDROutput                                  GRHIGlobals.SupportsHDROutput
#define GRHIMaxDispatchThreadGroupsPerDimension                GRHIGlobals.MaxDispatchThreadGroupsPerDimension
#define GRHISupportsPipelineVariableRateShading                GRHIGlobals.VariableRateShading.SupportsPipeline
#define GRHISupportsLargerVariableRateShadingSizes             GRHIGlobals.VariableRateShading.SupportsLargerSizes
#define GRHISupportsAttachmentVariableRateShading              GRHIGlobals.VariableRateShading.SupportsAttachment
#define GRHISupportsComplexVariableRateShadingCombinerOps      GRHIGlobals.VariableRateShading.SupportsComplexCombinerOps
#define GRHISupportsVariableRateShadingAttachmentArrayTextures GRHIGlobals.VariableRateShading.SupportsAttachmentArrayTextures
#define GRHIVariableRateShadingImageTileMaxWidth               GRHIGlobals.VariableRateShading.ImageTileMaxWidth
#define GRHIVariableRateShadingImageTileMaxHeight              GRHIGlobals.VariableRateShading.ImageTileMaxHeight
#define GRHIVariableRateShadingImageTileMinWidth               GRHIGlobals.VariableRateShading.ImageTileMinWidth
#define GRHIVariableRateShadingImageTileMinHeight              GRHIGlobals.VariableRateShading.ImageTileMinHeight
#define GRHIVariableRateShadingImageDataType                   GRHIGlobals.VariableRateShading.ImageDataType
#define GRHIVariableRateShadingImageFormat                     GRHIGlobals.VariableRateShading.ImageFormat
#define GRHISupportsLateVariableRateShadingUpdate              GRHIGlobals.VariableRateShading.SupportsLateUpdate
#define GRHISupportsVariableRateShadingImageOffsets			   GRHIGlobals.VariableRateShading.SupportsImageOffsets
#define GRHIVariableRateShadingImageOffsets					   GRHIGlobals.VariableRateShading.ImageOffsets
#define GRHIVariableRateShadingImageOffsetGranularity		   GRHIGlobals.VariableRateShading.ImageOffsetGranularity
#define GRHIHDRDisplayOutputFormat                             GRHIGlobals.HDRDisplayOutputFormat
#define GRHIPresentCounter                                     GRHIGlobals.PresentCounter
#define GRHISupportsArrayIndexFromAnyShader                    GRHIGlobals.SupportsArrayIndexFromAnyShader
#define GRHISupportsPipelineFileCache                          GRHIGlobals.SupportsPipelineFileCache
#define GRHISupportsPSOPrecaching                              GRHIGlobals.SupportsPSOPrecaching
#define GRHISupportsStencilRefFromPixelShader                  GRHIGlobals.SupportsStencilRefFromPixelShader
#define GRHISupportsRasterOrderViews                           GRHIGlobals.SupportsRasterOrderViews
#define GRHISupportsConservativeRasterization                  GRHIGlobals.SupportsConservativeRasterization
#define GRHISupportsShaderRootConstants                        GRHIGlobals.SupportsShaderRootConstants
#define GRHISupportsShaderBundleDispatch                       GRHIGlobals.ShaderBundles.SupportsDispatch
#define GRHISupportsShaderBundleWorkGraphDispatch              GRHIGlobals.ShaderBundles.SupportsWorkGraphDispatch
#define GRHISupportsShaderBundleParallel                       GRHIGlobals.ShaderBundles.SupportsParallel
#define GRHISupportsMeshShadersTier0                           GRHIGlobals.SupportsMeshShadersTier0
#define GRHISupportsMeshShadersTier1                           GRHIGlobals.SupportsMeshShadersTier1
#define GRHISupportsShaderWorkGraphsTier1                      GRHIGlobals.SupportsShaderWorkGraphsTier1
#define GRHISupportsShaderTimestamp                            GRHIGlobals.SupportsShaderTimestamp
#define GRHISupportsEfficientUploadOnResourceCreation          GRHIGlobals.SupportsEfficientUploadOnResourceCreation
#define GRHISupportsMapWriteNoOverwrite                        GRHIGlobals.SupportsMapWriteNoOverwrite
#define GRHIDefaultMSAASampleOffsets                           GRHIGlobals.DefaultMSAASampleOffsets
#define GRHISupportsAsyncPipelinePrecompile                    GRHIGlobals.SupportsAsyncPipelinePrecompile
#define GRHIBindlessSupport                                    GRHIGlobals.BindlessSupport
#define GShaderPlatformForFeatureLevel                         GRHIGlobals.ShaderPlatformForFeatureLevel
#define GRHIIsDebugLayerEnabled                                GRHIGlobals.IsDebugLayerEnabled
#define GRHISupportsMSAAShaderResolve						   GRHIGlobals.SupportsMSAAShaderResolve
#define GRHISupportsDepthStencilResolve                        GRHIGlobals.SupportsDepthStencilResolve
#define GRHISupportsLossyFramebufferCompression                GRHIGlobals.SupportsLossyFramebufferCompression
#define GRHISupportsBindingTexArrayPerSlice					   GRHIGlobals.SupportsBindingTexArrayPerSlice
#define GRHINeedsSRVGraphicsNonPixelWorkaround 				   GRHIGlobals.NeedsSRVGraphicsNonPixelWorkaround

// Utility Getters

inline uint64 GetMaxConstantBufferByteSize()
{
	return GRHIGlobals.MaxConstantBufferByteSize;
}

inline uint64 GetMaxComputeSharedMemory()
{
	return GRHIGlobals.MaxComputeSharedMemory;
}

inline uint32 GetMax2DTextureDimension()
{
	return GRHIGlobals.MaxTextureDimensions;
}

inline uint32 GetMaxCubeTextureDimension()
{
	return GRHIGlobals.MaxCubeTextureDimensions;
}

inline uint32 GetMaxTextureArrayLayers()
{
	return GRHIGlobals.MaxTextureArrayLayers;
}

inline uint32 GetMaxTextureSamplers()
{
	return GRHIGlobals.MaxTextureSamplers;
}

inline uint32 GetMaxWorkGroupInvocations()
{
	return GRHIGlobals.MaxWorkGroupInvocations;
}

/** Get the shader platform associated with the supplied feature level on this machine */
inline EShaderPlatform GetFeatureLevelShaderPlatform(const FStaticFeatureLevel InFeatureLevel)
{
	return InFeatureLevel == ERHIFeatureLevel::Num ? EShaderPlatform::SP_NumPlatforms: GRHIGlobals.ShaderPlatformForFeatureLevel[InFeatureLevel];
}

inline EShaderPlatform GetFeatureLevelShaderPlatform_Checked(const FStaticFeatureLevel InFeatureLevel)
{
	checkf(!IsRunningCookCommandlet(), TEXT("Calling GetFeatureLevelShaderPlatform while inside a cook commandlet is not correct, as it will return the cookers ShaderPlatform and not the ShaderPlatform you are cooking for"));
	return GetFeatureLevelShaderPlatform(InFeatureLevel);
}
