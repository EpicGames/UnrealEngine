// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHICommandList.h"

struct FRHILockTracker
{
	struct FTextureLockParams
	{
		FTextureLockParams() = default;

		FTextureLockParams(const FRHILockedTextureDesc& InDesc, EResourceLockMode InLockMode, void* InData, bool bInbDirectLock)
			: Desc(InDesc)
			, LockMode(InLockMode)
			, Data(InData)
			, bDirectLock(bInbDirectLock)
		{
		}

		FRHILockedTextureDesc Desc;
		EResourceLockMode     LockMode = RLM_WriteOnly;
		void*                 Data = nullptr;

		//did we call the normal flushing/updating lock?
		bool                  bDirectLock = false;
	};

	TArray<FTextureLockParams, TInlineAllocator<16>> OutstandingTextureLocks;

	void Lock(const FRHILockTextureArgs& InArguments, void* InData, bool bInDirectBufferWrite)
	{
		const FRHILockedTextureDesc& Desc = InArguments;
#if DO_CHECK
		for (const FTextureLockParams& OutstandingLock : OutstandingTextureLocks)
		{
			check(Desc != OutstandingLock.Desc || (OutstandingLock.bDirectLock && bInDirectBufferWrite));
		}
#endif
		OutstandingTextureLocks.Emplace(Desc, InArguments.LockMode, InData, bInDirectBufferWrite);
	}

	inline FTextureLockParams Unlock(const FRHILockedTextureDesc& InDesc)
	{
		for (int32 Index = 0; Index < OutstandingTextureLocks.Num(); Index++)
		{
			if (OutstandingTextureLocks[Index].Desc == InDesc)
			{
				FTextureLockParams Result = OutstandingTextureLocks[Index];
				OutstandingTextureLocks.RemoveAtSwap(Index, EAllowShrinking::No);
				return Result;
			}
		}
		RaiseMismatchError();
		return FTextureLockParams();
	}




	struct FLockParams
	{
		void* RHIBuffer;
		void* Buffer;
		uint32 BufferSize;
		uint32 Offset;
		EResourceLockMode LockMode;
		bool bDirectLock; //did we call the normal flushing/updating lock?
		bool bCreateLock; //did we lock to immediately initialize a newly created buffer?

		inline FLockParams(void* InRHIBuffer, void* InBuffer, uint32 InOffset, uint32 InBufferSize, EResourceLockMode InLockMode, bool bInbDirectLock, bool bInCreateLock)
			: RHIBuffer(InRHIBuffer)
			, Buffer(InBuffer)
			, BufferSize(InBufferSize)
			, Offset(InOffset)
			, LockMode(InLockMode)
			, bDirectLock(bInbDirectLock)
			, bCreateLock(bInCreateLock)
		{
		}
	};

	struct FUnlockFenceParams
	{
		FUnlockFenceParams(void* InRHIBuffer, FGraphEventRef InUnlockEvent)
			: RHIBuffer(InRHIBuffer)
			, UnlockEvent(InUnlockEvent)
		{

		}
		void* RHIBuffer;
		FGraphEventRef UnlockEvent;
	};

	TArray<FLockParams, TInlineAllocator<16> > OutstandingLocks;
	TArray<FUnlockFenceParams, TInlineAllocator<16> > OutstandingUnlocks;

	FRHILockTracker()
	{}

	inline void Lock(void* RHIBuffer, void* Buffer, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode, bool bInDirectBufferWrite = false, bool bInCreateLock = false)
	{
#if DO_CHECK
		for (auto& Parms : OutstandingLocks)
		{
			check((Parms.RHIBuffer != RHIBuffer) || (Parms.bDirectLock && bInDirectBufferWrite) || Parms.Offset != Offset);
		}
#endif
		OutstandingLocks.Add(FLockParams(RHIBuffer, Buffer, Offset, SizeRHI, LockMode, bInDirectBufferWrite, bInCreateLock));
	}
	inline FLockParams Unlock(void* RHIBuffer, uint32 Offset = 0)
	{
		for (int32 Index = 0; Index < OutstandingLocks.Num(); Index++)
		{
			if (OutstandingLocks[Index].RHIBuffer == RHIBuffer && OutstandingLocks[Index].Offset == Offset)
			{
				FLockParams Result = OutstandingLocks[Index];
				OutstandingLocks.RemoveAtSwap(Index, EAllowShrinking::No);
				return Result;
			}
		}
		RaiseMismatchError();
		return FLockParams(nullptr, nullptr, 0, 0, RLM_WriteOnly, false, false);
	}

	template<class TIndexOrVertexBufferPointer>
	inline void AddUnlockFence(TIndexOrVertexBufferPointer* Buffer, FRHICommandListImmediate& RHICmdList, const FLockParams& LockParms)
	{
		if (LockParms.LockMode != RLM_WriteOnly || !(Buffer->GetUsage() & BUF_Volatile))
		{
			OutstandingUnlocks.Emplace(Buffer, RHICmdList.RHIThreadFence(true));
		}
	}

	inline void WaitForUnlock(void* RHIBuffer)
	{
		for (int32 Index = 0; Index < OutstandingUnlocks.Num(); Index++)
		{
			if (OutstandingUnlocks[Index].RHIBuffer == RHIBuffer)
			{
				FRHICommandListExecutor::WaitOnRHIThreadFence(OutstandingUnlocks[Index].UnlockEvent);
				OutstandingUnlocks.RemoveAtSwap(Index, EAllowShrinking::No);
				return;
			}
		}
	}

	inline void FlushCompleteUnlocks()
	{
		uint32 Count = OutstandingUnlocks.Num();
		for (uint32 Index = 0; Index < Count; Index++)
		{
			if (OutstandingUnlocks[Index].UnlockEvent->IsComplete())
			{
				OutstandingUnlocks.RemoveAt(Index, 1);
				--Count;
				--Index;
			}
		}
	}

	RHI_API void RaiseMismatchError();
};

extern RHI_API FRHILockTracker GRHILockTracker;
