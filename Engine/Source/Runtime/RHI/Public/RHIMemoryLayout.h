// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"
#include "RHIFeatureLevel.h"
#include "Serialization/MemoryLayout.h"

DECLARE_INTRINSIC_TYPE_LAYOUT(EVertexElementType);
DECLARE_INTRINSIC_TYPE_LAYOUT(EUniformBufferBaseType);
DECLARE_INTRINSIC_TYPE_LAYOUT(EUniformBufferBindingFlags);
DECLARE_INTRINSIC_TYPE_LAYOUT(ERHIUniformBufferFlags);
DECLARE_INTRINSIC_TYPE_LAYOUT(ERHIFeatureLevel::Type);
