// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/EnumRange.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "RHIGlobals.h"

enum class ERHIPipeline : uint8
{
	Graphics = 1 << 0,
	AsyncCompute = 1 << 1,

	None = 0,
	All = Graphics | AsyncCompute,
	Num = 2
};
ENUM_CLASS_FLAGS(ERHIPipeline)

inline constexpr bool IsSingleRHIPipeline(ERHIPipeline Pipelines)
{
	return Pipelines != ERHIPipeline::None && FMath::IsPowerOfTwo(static_cast<std::underlying_type_t<ERHIPipeline>>(Pipelines));
}

inline constexpr uint32 GetRHIPipelineIndex(ERHIPipeline Pipeline)
{
	switch (Pipeline)
	{
	default:
	case ERHIPipeline::Graphics:
		return 0;
	case ERHIPipeline::AsyncCompute:
		return 1;
	}
}

inline constexpr uint32 GetRHIPipelineCount()
{
	return uint32(ERHIPipeline::Num);
}

inline ERHIPipeline GetEnabledRHIPipelines()
{
	return GRHIGlobals.SupportsEfficientAsyncCompute
		? ERHIPipeline::All
		: ERHIPipeline::Graphics;
}

/** Array of pass handles by RHI pipeline, with overloads to help with enum conversion. */
template <typename ElementType>
class TRHIPipelineArray : public TStaticArray<ElementType, GetRHIPipelineCount()>
{
	using Base = TStaticArray<ElementType, GetRHIPipelineCount()>;
public:
	using Base::Base;

	inline ElementType& operator[](int32 Index)
	{
		return Base::operator[](Index);
	}

	inline const ElementType& operator[](int32 Index) const
	{
		return Base::operator[](Index);
	}

	inline ElementType& operator[](ERHIPipeline Pipeline)
	{
		return Base::operator[](GetRHIPipelineIndex(Pipeline));
	}

	inline const ElementType& operator[](ERHIPipeline Pipeline) const
	{
		return Base::operator[](GetRHIPipelineIndex(Pipeline));
	}
};