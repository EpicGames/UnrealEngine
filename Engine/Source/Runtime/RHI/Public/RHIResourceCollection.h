// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIResources.h"
#include "RHITextureReference.h"

struct FRHIResourceCollectionMember
{
	enum class EType : uint8
	{
		Texture,
		TextureReference,
		ShaderResourceView,
		Sampler,
	};

	FRHIResourceCollectionMember() = default;
	FRHIResourceCollectionMember(EType InType, FRHIResource* InResource)
		: Resource(InResource)
		, Type(InType)
	{
	}
	FRHIResourceCollectionMember(FRHITexture* InTexture)
		: FRHIResourceCollectionMember(FRHIResourceCollectionMember::EType::Texture, InTexture)
	{
	}
	FRHIResourceCollectionMember(FRHITextureReference* InTextureReference)
		: FRHIResourceCollectionMember(FRHIResourceCollectionMember::EType::TextureReference, InTextureReference)
	{
	}
	FRHIResourceCollectionMember(FRHIShaderResourceView* InView)
		: FRHIResourceCollectionMember(FRHIResourceCollectionMember::EType::ShaderResourceView, InView)
	{
	}
	FRHIResourceCollectionMember(FRHISamplerState* InSamplerState)
		: FRHIResourceCollectionMember(FRHIResourceCollectionMember::EType::Sampler, InSamplerState)
	{
	}

	FRHIResource* Resource = nullptr;
	EType         Type = EType::Texture;
};

class FRHIResourceCollection : public FRHIResource
{
public:
	RHI_API FRHIResourceCollection(TConstArrayView<FRHIResourceCollectionMember> InMembers);
	FRHIResourceCollection(const FRHIResourceCollection&) = delete;
	RHI_API ~FRHIResourceCollection();

	RHI_API void UpdateMember(int32 Index, FRHIResourceCollectionMember InMember);
	RHI_API void UpdateMembers(int32 StartIndex, TConstArrayView<FRHIResourceCollectionMember> NewMembers);

	RHI_API virtual FRHIDescriptorHandle GetBindlessHandle() const;

	TConstArrayView<FRHIResourceCollectionMember> GetMembers() const
	{
		return MakeConstArrayView(Members);
	}

private:
	TArray<FRHIResourceCollectionMember> Members;
};
