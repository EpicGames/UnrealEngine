// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIFwd.h"
#include "RHIResources.h"

#include "Containers/Array.h"
#include "Misc/TVariant.h"

class FRHIResourceReplaceInfo
{
public:
	template <typename TType>
	struct TPair
	{
		TType* Dst;
		TType* Src;

		TPair(TType* Dst, TType* Src)
			: Dst(Dst)
			, Src(Src)
		{}
	};

	typedef TPair<FRHIBuffer            > FBuffer;
	typedef TPair<FRHIRayTracingGeometry> FRTGeometry;

	typedef TVariant<FBuffer, FRTGeometry> TStorage;

	enum class EType : uint8
	{
		Buffer     = TStorage::IndexOfType<FBuffer    >(),
		RTGeometry = TStorage::IndexOfType<FRTGeometry>(),
	};

	EType GetType() const { return EType(Storage.GetIndex()); }

	FBuffer     const& GetBuffer    () const { return Storage.Get<FBuffer    >(); }
	FRTGeometry const& GetRTGeometry() const { return Storage.Get<FRTGeometry>(); }

	FRHIResourceReplaceInfo(FRHIBuffer* Dst, FRHIBuffer* Src)
		: Storage(TInPlaceType<FBuffer>(), Dst, Src)
	{}

	FRHIResourceReplaceInfo(FRHIRayTracingGeometry* Dst, FRHIRayTracingGeometry* Src)
		: Storage(TInPlaceType<FRTGeometry>(), Dst, Src)
	{}

private:
	TStorage Storage;
};

class FRHIResourceReplaceBatcher
{
	FRHICommandListBase& RHICmdList;
	TArray<FRHIResourceReplaceInfo> Infos;

public:
	FRHIResourceReplaceBatcher(FRHICommandListBase& RHICmdList, uint32 InitialCapacity = 0)
		: RHICmdList(RHICmdList)
	{
		if (InitialCapacity > 0)
		{
			Infos.Reserve(InitialCapacity);
		}
	}

	inline ~FRHIResourceReplaceBatcher();

	template <typename... TArgs>
	void EnqueueReplace(TArgs&&... Args)
	{
		Infos.Emplace(Forward<TArgs>(Args)...);
	}
};
