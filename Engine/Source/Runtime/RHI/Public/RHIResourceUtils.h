// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHICommandList.h"
#include "RHIResources.h"
#include "Containers/ResourceArray.h"

namespace UE::RHIResourceUtils
{
	static FBufferRHIRef CreateBufferZeroed(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& Desc)
	{
		return RHICmdList.CreateBuffer(FRHIBufferCreateDesc(Desc).SetInitActionZeroData());
	}

	template<typename TElementType>
	static FBufferRHIRef CreateBufferWithArray(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& Desc, TConstArrayView<TElementType> Array)
	{
		FResourceArrayUploadArrayView UploadView(Array);
		return RHICmdList.CreateBuffer(FRHIBufferCreateDesc(Desc).SetInitActionResourceArray(&UploadView));
	}

	template<typename TElementType>
	static FBufferRHIRef CreateBufferWithArray(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& Desc, const TArray<TElementType>& Array)
	{
		return CreateBufferWithArray(RHICmdList, Desc, TConstArrayView<TElementType>(Array));
	}

	template<typename TElementType, size_t TCount>
	static FBufferRHIRef CreateBufferWithArray(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& Desc, const TElementType (&Array)[TCount])
	{
		return CreateBufferWithArray(RHICmdList, Desc, TConstArrayView<TElementType>(Array, TCount));
	}

	template<typename TElementType>
	static FBufferRHIRef CreateBufferWithValue(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& Desc, const TElementType& Value)
	{
		return CreateBufferWithArray(RHICmdList, Desc, TConstArrayView<TElementType>(&Value, 1));
	}

	static FBufferRHIRef CreateBufferFromArray(FRHICommandListBase& RHICmdList, const TCHAR* Name, EBufferUsageFlags UsageFlags, uint32 InStride, const void* InData, uint32 InSizeInBytes)
	{
		FResourceArrayUploadArrayView UploadView(InData, InSizeInBytes);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(Name, InSizeInBytes, InStride, UsageFlags)
			.DetermineInitialState()
			.SetInitActionResourceArray(&UploadView);

		return RHICmdList.CreateBuffer(CreateDesc);
	}

	template<typename TElementType>
	static FBufferRHIRef CreateBufferFromArray(FRHICommandListBase& RHICmdList, const TCHAR* Name, EBufferUsageFlags UsageFlags, uint32 InStride, ERHIAccess InitialState, TConstArrayView<TElementType> Array)
	{
		FResourceArrayUploadArrayView UploadView(Array);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(Name, UploadView.GetResourceDataSize(), InStride, UsageFlags)
			.SetInitialState(InitialState)
			.SetInitActionResourceArray(&UploadView);

		return RHICmdList.CreateBuffer(CreateDesc);
	}

	template<typename TElementType>
	static FBufferRHIRef CreateBufferFromArray(FRHICommandListBase& RHICmdList, const TCHAR* Name, EBufferUsageFlags UsageFlags, ERHIAccess InitialState, TConstArrayView<TElementType> Array)
	{
		return CreateBufferFromArray(RHICmdList, Name, UsageFlags, Array.GetTypeSize(), InitialState, Array);
	}

	template<typename TElementType>
	static FBufferRHIRef CreateBufferFromArray(FRHICommandListBase& RHICmdList, const TCHAR* Name, EBufferUsageFlags UsageFlags, TConstArrayView<TElementType> Array)
	{
		return CreateBufferFromArray(RHICmdList, Name, UsageFlags, RHIGetDefaultResourceState(UsageFlags, false), Array);
	}

	template<typename TElementType>
	static FBufferRHIRef CreateVertexBufferFromArray(FRHICommandListBase& RHICmdList, const TCHAR* Name, EBufferUsageFlags ExtraFlags, TConstArrayView<TElementType> Array)
	{
		const EBufferUsageFlags Usage = EBufferUsageFlags::VertexBuffer | ExtraFlags;
		const ERHIAccess InitialState = RHIGetDefaultResourceState(Usage, false);
		return CreateBufferFromArray<TElementType>(RHICmdList, Name, Usage, 0, InitialState, Array);
	}

	template<typename TElementType>
	static FBufferRHIRef CreateVertexBufferFromArray(FRHICommandListBase& RHICmdList, const TCHAR* Name, TConstArrayView<TElementType> Array)
	{
		return CreateVertexBufferFromArray<TElementType>(RHICmdList, Name, EBufferUsageFlags::None, Array);
	}

	template<typename TElementType>
	static FBufferRHIRef CreateIndexBufferFromArray(FRHICommandListBase& RHICmdList, const TCHAR* Name, EBufferUsageFlags ExtraFlags, TConstArrayView<TElementType> Array)
	{
		const EBufferUsageFlags Usage = EBufferUsageFlags::IndexBuffer | ExtraFlags;
		const ERHIAccess InitialState = RHIGetDefaultResourceState(Usage, false);
		return CreateBufferFromArray<TElementType>(RHICmdList, Name, Usage, InitialState, Array);
	}

	template<typename TElementType>
	static FBufferRHIRef CreateIndexBufferFromArray(FRHICommandListBase& RHICmdList, const TCHAR* Name, TConstArrayView<TElementType> Array)
	{
		return CreateIndexBufferFromArray<TElementType>(RHICmdList, Name, EBufferUsageFlags::None, Array);
	}
}
