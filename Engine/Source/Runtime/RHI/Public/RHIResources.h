// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Misc/Crc.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Math/Color.h"
#include "Containers/StaticArray.h"
#include "HAL/ThreadSafeCounter.h"
#include "Hash/ShaderHash.h"
#include "Templates/RefCounting.h"
#include "PixelFormat.h"
#include "Async/TaskGraphFwd.h"
#include "RHIFwd.h"
#include "RHITransition.h"
#include "MultiGPU.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/IntVector.h"


#include <atomic>

class FHazardPointerCollection;
class FRHIComputeCommandList;
class FRHICommandListImmediate;
class FRHITextureReference;
class FRHIShaderBindingLayout;
class FResourceBulkDataInterface;
class FResourceArrayInterface;
struct FResourceArrayUploadInterface;
class FRayTracingPipelineStateInitializer;

struct FClearValueBinding;
struct FRHIResourceInfo;
struct FGenerateMipsStruct;
enum class EClearBinding;

typedef TArray<FGraphEventRef, TInlineAllocator<4> > FGraphEventArray;

struct FRHISizeAndStride
{
	uint64 Size;
	uint32 Stride;
};

/** The base type of RHI resources. */
class FRHIResource
{
public:
	RHI_API FRHIResource(ERHIResourceType InResourceType);

protected:
	// RHI resources should only be destructed via the deletion queue,
	// so this is protected to prevent others from 'delete'ing these directly.
	RHI_API virtual ~FRHIResource();

private:
	// Separate function to avoid force inlining this everywhere. Helps both for code size and performance.
	RHI_API void MarkForDelete() const;

	friend class FDynamicRHI;
	friend class FRHICommandListExecutor;
	static RHI_API void DeleteResources(TArray<FRHIResource*> const& Resources);
	static RHI_API void GatherResourcesToDelete(TArray<FRHIResource*>& OutResources, bool bIncludeExtendedLifetimeResources);

public:
	inline uint32 AddRef() const
	{
		int32 NewValue = AtomicFlags.AddRef(std::memory_order_acquire);
		checkSlow(NewValue > 0); 
		return uint32(NewValue);
	}

	inline uint32 Release() const
	{
		int32 NewValue = AtomicFlags.Release(std::memory_order_release);
		check(NewValue >= 0);

		if (NewValue == 0)
		{
			MarkForDelete();
		}
		checkSlow(NewValue >= 0);
		return uint32(NewValue);
	}

	inline uint32 GetRefCount() const
	{
		int32 CurrentValue = AtomicFlags.GetNumRefs(std::memory_order_relaxed);
		checkSlow(CurrentValue >= 0); 
		return uint32(CurrentValue);
	}

	bool IsValid() const
	{
		return AtomicFlags.IsValid(std::memory_order_relaxed);
	}

	void DisableLifetimeExtension()
	{
		ensureMsgf(IsValid(), TEXT("Resource is already marked for deletion. This call is a no-op. DisableLifetimeExtension must be called while still holding a live reference."));
		bAllowExtendLifetime = false;
	}

	inline ERHIResourceType GetType() const { return ResourceType; }

	inline FName GetOwnerName() const
	{
#if RHI_ENABLE_RESOURCE_INFO
		return OwnerName;
#else
		return NAME_None;
#endif
	}

	inline void SetOwnerName(FName InOwnerName)
	{
#if RHI_ENABLE_RESOURCE_INFO
		OwnerName = InOwnerName;
#endif
	}

#if RHI_ENABLE_RESOURCE_INFO
	// Get resource info if available.
	// Should return true if the ResourceInfo was filled with data.
	RHI_API virtual bool GetResourceInfo(FRHIResourceInfo& OutResourceInfo) const;

	static void BeginTrackingResource(FRHIResource* InResource);
	static void EndTrackingResource(FRHIResource* InResource);
	static void StartTrackingAllResources();
	static void StopTrackingAllResources();
#endif

private:
	class FAtomicFlags
	{
		static constexpr uint32 MarkedForDeleteBit    = 1 << 30;
		static constexpr uint32 DeletingBit           = 1 << 31;
		static constexpr uint32 NumRefsMask           = ~(MarkedForDeleteBit | DeletingBit);

		std::atomic_uint Packed = { 0 };

	public:
		int32 AddRef(std::memory_order MemoryOrder)
		{
			uint32 OldPacked = Packed.fetch_add(1, MemoryOrder);
			checkf((OldPacked & DeletingBit) == 0, TEXT("Resource is being deleted."));
			int32  NumRefs = (OldPacked & NumRefsMask) + 1;
			checkf(NumRefs < NumRefsMask, TEXT("Reference count has overflowed."));
			return NumRefs;
		}

		int32 Release(std::memory_order MemoryOrder)
		{
			uint32 OldPacked = Packed.fetch_sub(1, MemoryOrder);
			checkf((OldPacked & DeletingBit) == 0, TEXT("Resource is being deleted."));
			int32  NumRefs = (OldPacked & NumRefsMask) - 1;
			checkf(NumRefs >= 0, TEXT("Reference count has underflowed."));
			return NumRefs;
		}

		bool MarkForDelete(std::memory_order MemoryOrder)
		{
			uint32 OldPacked = Packed.fetch_or(MarkedForDeleteBit, MemoryOrder);
			check((OldPacked & DeletingBit) == 0);
			return (OldPacked & MarkedForDeleteBit) != 0;
		}

		bool UnmarkForDelete(std::memory_order MemoryOrder)
		{
			uint32 OldPacked = Packed.fetch_xor(MarkedForDeleteBit, MemoryOrder);
			check((OldPacked & DeletingBit) == 0);
			bool  OldMarkedForDelete = (OldPacked & MarkedForDeleteBit) != 0;
			check(OldMarkedForDelete == true);
			return OldMarkedForDelete;
		}

		bool Deleting()
		{
			uint32 LocalPacked = Packed.load(std::memory_order_acquire);
			check((LocalPacked & MarkedForDeleteBit) != 0);
			check((LocalPacked & DeletingBit) == 0);
			uint32 NumRefs = LocalPacked & NumRefsMask;

			if (NumRefs == 0) // caches can bring dead objects back to life
			{
#if DO_CHECK
				Packed.fetch_or(DeletingBit, std::memory_order_acquire);
#endif
				return true;
			}
			else
			{
				UnmarkForDelete(std::memory_order_release);
				return false;
			}
		}

		bool IsValid(std::memory_order MemoryOrder)
		{
			uint32 LocalPacked = Packed.load(MemoryOrder);
			return (LocalPacked & MarkedForDeleteBit) == 0 && (LocalPacked & NumRefsMask) != 0;
		}

		bool IsMarkedForDelete(std::memory_order MemoryOrder)
		{
			return (Packed.load(MemoryOrder) & MarkedForDeleteBit) != 0;
		}

		int32 GetNumRefs(std::memory_order MemoryOrder)
		{
			return Packed.load(MemoryOrder) & NumRefsMask;
		}
	};
	mutable FAtomicFlags AtomicFlags;

	const ERHIResourceType ResourceType;
	uint8 bCommitted : 1;
	uint8 bAllowExtendLifetime : 1;
#if RHI_ENABLE_RESOURCE_INFO
	uint8 bBeingTracked : 1;
	FName OwnerName;
#endif

#if DO_CHECK
	static thread_local FRHIResource const* CurrentlyDeleting;
#endif

	friend FRHICommandListImmediate;
};

enum class EClearBinding
{
	ENoneBound, //no clear color associated with this target.  Target will not do hardware clears on most platforms
	EColorBound, //target has a clear color bound.  Clears will use the bound color, and do hardware clears.
	EDepthStencilBound, //target has a depthstencil value bound.  Clears will use the bound values and do hardware clears.
};

struct FClearValueBinding
{
	struct DSVAlue
	{
		float Depth;
		uint32 Stencil;
	};

	FClearValueBinding()
		: ColorBinding(EClearBinding::EColorBound)
	{
		Value.Color[0] = 0.0f;
		Value.Color[1] = 0.0f;
		Value.Color[2] = 0.0f;
		Value.Color[3] = 0.0f;
	}

	FClearValueBinding(EClearBinding NoBinding)
		: ColorBinding(NoBinding)
	{
		check(ColorBinding == EClearBinding::ENoneBound);

		Value.Color[0] = 0.0f;
		Value.Color[1] = 0.0f;
		Value.Color[2] = 0.0f;
		Value.Color[3] = 0.0f;

		Value.DSValue.Depth = 0.0f;
		Value.DSValue.Stencil = 0;
	}

	explicit FClearValueBinding(const FLinearColor& InClearColor)
		: ColorBinding(EClearBinding::EColorBound)
	{
		Value.Color[0] = InClearColor.R;
		Value.Color[1] = InClearColor.G;
		Value.Color[2] = InClearColor.B;
		Value.Color[3] = InClearColor.A;
	}

	explicit FClearValueBinding(float DepthClearValue, uint32 StencilClearValue = 0)
		: ColorBinding(EClearBinding::EDepthStencilBound)
	{
		Value.DSValue.Depth = DepthClearValue;
		Value.DSValue.Stencil = StencilClearValue;
	}

	FLinearColor GetClearColor() const
	{
		ensure(ColorBinding == EClearBinding::EColorBound);
		return FLinearColor(Value.Color[0], Value.Color[1], Value.Color[2], Value.Color[3]);
	}

	void GetDepthStencil(float& OutDepth, uint32& OutStencil) const
	{
		ensure(ColorBinding == EClearBinding::EDepthStencilBound);
		OutDepth = Value.DSValue.Depth;
		OutStencil = Value.DSValue.Stencil;
	}

	bool operator==(const FClearValueBinding& Other) const
	{
		if (ColorBinding == Other.ColorBinding)
		{
			if (ColorBinding == EClearBinding::EColorBound)
			{
				return
					Value.Color[0] == Other.Value.Color[0] &&
					Value.Color[1] == Other.Value.Color[1] &&
					Value.Color[2] == Other.Value.Color[2] &&
					Value.Color[3] == Other.Value.Color[3];

			}
			if (ColorBinding == EClearBinding::EDepthStencilBound)
			{
				return
					Value.DSValue.Depth == Other.Value.DSValue.Depth &&
					Value.DSValue.Stencil == Other.Value.DSValue.Stencil;
			}
			return true;
		}
		return false;
	}

	friend inline uint32 GetTypeHash(FClearValueBinding const& Binding)
	{
		uint32 Hash = GetTypeHash(Binding.ColorBinding);

		if (Binding.ColorBinding == EClearBinding::EColorBound)
		{
			Hash = HashCombine(Hash, GetTypeHash(Binding.Value.Color[0]));
			Hash = HashCombine(Hash, GetTypeHash(Binding.Value.Color[1]));
			Hash = HashCombine(Hash, GetTypeHash(Binding.Value.Color[2]));
			Hash = HashCombine(Hash, GetTypeHash(Binding.Value.Color[3]));
		}
		else if (Binding.ColorBinding == EClearBinding::EDepthStencilBound)
		{
			Hash = HashCombine(Hash, GetTypeHash(Binding.Value.DSValue.Depth  ));
			Hash = HashCombine(Hash, GetTypeHash(Binding.Value.DSValue.Stencil));
		}

		return Hash;
	}

	EClearBinding ColorBinding;

	union ClearValueType
	{
		float Color[4];
		DSVAlue DSValue;
	} Value;

	// common clear values
	static RHI_API const FClearValueBinding None;
	static RHI_API const FClearValueBinding Black;
	static RHI_API const FClearValueBinding BlackMaxAlpha;
	static RHI_API const FClearValueBinding White;
	static RHI_API const FClearValueBinding Transparent;
	static RHI_API const FClearValueBinding DepthOne;
	static RHI_API const FClearValueBinding DepthZero;
	static RHI_API const FClearValueBinding DepthNear;
	static RHI_API const FClearValueBinding DepthFar;	
	static RHI_API const FClearValueBinding Green;
	static RHI_API const FClearValueBinding DefaultNormal8Bit;
};

class FExclusiveDepthStencil
{
public:
	enum Type
	{
		// don't use those directly, use the combined versions below
		// 4 bits are used for depth and 4 for stencil to make the hex value readable and non overlapping
		DepthNop = 0x00,
		DepthRead = 0x01,
		DepthWrite = 0x02,
		DepthMask = 0x0f,
		StencilNop = 0x00,
		StencilRead = 0x10,
		StencilWrite = 0x20,
		StencilMask = 0xf0,

		// use those:
		DepthNop_StencilNop = DepthNop + StencilNop,
		DepthRead_StencilNop = DepthRead + StencilNop,
		DepthWrite_StencilNop = DepthWrite + StencilNop,
		DepthNop_StencilRead = DepthNop + StencilRead,
		DepthRead_StencilRead = DepthRead + StencilRead,
		DepthWrite_StencilRead = DepthWrite + StencilRead,
		DepthNop_StencilWrite = DepthNop + StencilWrite,
		DepthRead_StencilWrite = DepthRead + StencilWrite,
		DepthWrite_StencilWrite = DepthWrite + StencilWrite,
	};

private:
	Type Value;

public:
	// constructor
	FExclusiveDepthStencil(Type InValue = DepthNop_StencilNop)
		: Value(InValue)
	{
	}

	inline bool IsUsingDepthStencil() const
	{
		return Value != DepthNop_StencilNop;
	}
	inline bool IsUsingDepth() const
	{
		return (ExtractDepth() != DepthNop);
	}
	inline bool IsUsingStencil() const
	{
		return (ExtractStencil() != StencilNop);
	}
	inline bool IsDepthWrite() const
	{
		return ExtractDepth() == DepthWrite;
	}
	inline bool IsDepthRead() const
	{
		return ExtractDepth() == DepthRead;
	}
	inline bool IsStencilWrite() const
	{
		return ExtractStencil() == StencilWrite;
	}
	inline bool IsStencilRead() const
	{
		return ExtractStencil() == StencilRead;
	}

	inline bool IsAnyWrite() const
	{
		return IsDepthWrite() || IsStencilWrite();
	}

	inline void SetDepthWrite()
	{
		Value = (Type)(ExtractStencil() | DepthWrite);
	}
	inline void SetStencilWrite()
	{
		Value = (Type)(ExtractDepth() | StencilWrite);
	}
	inline void SetDepthStencilWrite(bool bDepth, bool bStencil)
	{
		Value = DepthNop_StencilNop;

		if (bDepth)
		{
			SetDepthWrite();
		}
		if (bStencil)
		{
			SetStencilWrite();
		}
	}
	bool operator==(const FExclusiveDepthStencil& rhs) const
	{
		return Value == rhs.Value;
	}

	bool operator != (const FExclusiveDepthStencil& RHS) const
	{
		return Value != RHS.Value;
	}

	inline bool IsValid(FExclusiveDepthStencil& Current) const
	{
		Type Depth = ExtractDepth();

		if (Depth != DepthNop && Depth != Current.ExtractDepth())
		{
			return false;
		}

		Type Stencil = ExtractStencil();

		if (Stencil != StencilNop && Stencil != Current.ExtractStencil())
		{
			return false;
		}

		return true;
	}

	inline void GetAccess(ERHIAccess& DepthAccess, ERHIAccess& StencilAccess) const
	{
		DepthAccess = ERHIAccess::None;

		// SRV access is allowed whilst a depth stencil target is "readable".
		constexpr ERHIAccess DSVReadOnlyMask =
			ERHIAccess::DSVRead;

		// If write access is required, only the depth block can access the resource.
		constexpr ERHIAccess DSVReadWriteMask =
			ERHIAccess::DSVRead |
			ERHIAccess::DSVWrite;

		if (IsUsingDepth())
		{
			DepthAccess = IsDepthWrite() ? DSVReadWriteMask : DSVReadOnlyMask;
		}

		StencilAccess = ERHIAccess::None;

		if (IsUsingStencil())
		{
			StencilAccess = IsStencilWrite() ? DSVReadWriteMask : DSVReadOnlyMask;
		}
	}

	template <typename TFunction>
	inline void EnumerateSubresources(TFunction Function) const
	{
		if (!IsUsingDepthStencil())
		{
			return;
		}

		ERHIAccess DepthAccess = ERHIAccess::None;
		ERHIAccess StencilAccess = ERHIAccess::None;
		GetAccess(DepthAccess, StencilAccess);

		// Same depth / stencil state; single subresource.
		if (DepthAccess == StencilAccess)
		{
			Function(DepthAccess, FRHITransitionInfo::kAllSubresources);
		}
		// Separate subresources for depth / stencil.
		else
		{
			if (DepthAccess != ERHIAccess::None)
			{
				Function(DepthAccess, FRHITransitionInfo::kDepthPlaneSlice);
			}
			if (StencilAccess != ERHIAccess::None)
			{
				Function(StencilAccess, FRHITransitionInfo::kStencilPlaneSlice);
			}
		}
	}

	/**
	* Returns a new FExclusiveDepthStencil to be used to transition a depth stencil resource to readable.
	* If the depth or stencil is already in a readable state, that particular component is returned as Nop,
	* to avoid unnecessary subresource transitions.
	*/
	inline FExclusiveDepthStencil GetReadableTransition() const
	{
		FExclusiveDepthStencil::Type NewDepthState = IsDepthWrite()
			? FExclusiveDepthStencil::DepthRead
			: FExclusiveDepthStencil::DepthNop;

		FExclusiveDepthStencil::Type NewStencilState = IsStencilWrite()
			? FExclusiveDepthStencil::StencilRead
			: FExclusiveDepthStencil::StencilNop;

		return (FExclusiveDepthStencil::Type)(NewDepthState | NewStencilState);
	}

	/**
	* Returns a new FExclusiveDepthStencil to be used to transition a depth stencil resource to readable.
	* If the depth or stencil is already in a readable state, that particular component is returned as Nop,
	* to avoid unnecessary subresource transitions.
	*/
	inline FExclusiveDepthStencil GetWritableTransition() const
	{
		FExclusiveDepthStencil::Type NewDepthState = IsDepthRead()
			? FExclusiveDepthStencil::DepthWrite
			: FExclusiveDepthStencil::DepthNop;

		FExclusiveDepthStencil::Type NewStencilState = IsStencilRead()
			? FExclusiveDepthStencil::StencilWrite
			: FExclusiveDepthStencil::StencilNop;

		return (FExclusiveDepthStencil::Type)(NewDepthState | NewStencilState);
	}

	uint32 GetIndex() const
	{
		// Note: The array to index has views created in that specific order.

		// we don't care about the Nop versions so less views are needed
		// we combine Nop and Write
		switch (Value)
		{
		case DepthWrite_StencilNop:
		case DepthNop_StencilWrite:
		case DepthWrite_StencilWrite:
		case DepthNop_StencilNop:
			return 0; // old DSAT_Writable

		case DepthRead_StencilNop:
		case DepthRead_StencilWrite:
			return 1; // old DSAT_ReadOnlyDepth

		case DepthNop_StencilRead:
		case DepthWrite_StencilRead:
			return 2; // old DSAT_ReadOnlyStencil

		case DepthRead_StencilRead:
			return 3; // old DSAT_ReadOnlyDepthAndStencil
		}
		// should never happen
		check(0);
		return -1;
	}
	static const uint32 MaxIndex = 4;

private:
	inline Type ExtractDepth() const
	{
		return (Type)(Value & DepthMask);
	}
	inline Type ExtractStencil() const
	{
		return (Type)(Value & StencilMask);
	}
	friend uint32 GetTypeHash(const FExclusiveDepthStencil& Ds);
};

//
// State blocks
//

class FRHISamplerState : public FRHIResource 
{
public:
	FRHISamplerState() : FRHIResource(RRT_SamplerState) {}
	virtual bool IsImmutable() const { return false; }
	virtual FRHIDescriptorHandle GetBindlessHandle() const { return FRHIDescriptorHandle(); }
};

class FRHIRasterizerState : public FRHIResource
{
public:
	FRHIRasterizerState() : FRHIResource(RRT_RasterizerState) {}
	virtual bool GetInitializer(struct FRasterizerStateInitializerRHI& Init) { return false; }
};

class FRHIDepthStencilState : public FRHIResource
{
public:
	FRHIDepthStencilState() : FRHIResource(RRT_DepthStencilState) {}
#if ENABLE_RHI_VALIDATION
	FExclusiveDepthStencil ActualDSMode;
#endif
	virtual bool GetInitializer(struct FDepthStencilStateInitializerRHI& Init) { return false; }
};

class FRHIBlendState : public FRHIResource
{
public:
	FRHIBlendState() : FRHIResource(RRT_BlendState) {}
	virtual bool GetInitializer(class FBlendStateInitializerRHI& Init) { return false; }
};

template <typename RHIState, typename RHIStateInitializer>
static bool MatchRHIState(RHIState* LHSState, RHIState* RHSState)
{
	RHIStateInitializer LHSStateInitializerRHI;
	RHIStateInitializer RHSStateInitializerRHI;
	if (LHSState)
	{
		LHSState->GetInitializer(LHSStateInitializerRHI);
	}
	if (RHSState)
	{
		RHSState->GetInitializer(RHSStateInitializerRHI);
	}
	return LHSStateInitializerRHI == RHSStateInitializerRHI;
}

//
// Shader bindings
//

typedef TArray<struct FVertexElement,TFixedAllocator<MaxVertexElementCount> > FVertexDeclarationElementList;

class FRHIVertexDeclaration : public FRHIResource
{
public:
	FRHIVertexDeclaration() : FRHIResource(RRT_VertexDeclaration) {}
	virtual bool GetInitializer(FVertexDeclarationElementList& Init) { return false; }
	virtual uint32 GetPrecachePSOHash() const { return 0; }
};

class FRHIBoundShaderState : public FRHIResource
{
public:
	FRHIBoundShaderState() : FRHIResource(RRT_BoundShaderState) {}
};

//
// Shaders
//

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define RHI_INCLUDE_SHADER_DEBUG_DATA 1
#else
	#define RHI_INCLUDE_SHADER_DEBUG_DATA 0
#endif

#if RHI_INCLUDE_SHADER_DEBUG_DATA
	#define RHI_IF_SHADER_DEBUG_DATA(...)	__VA_ARGS__
#else
	#define RHI_IF_SHADER_DEBUG_DATA(...)
#endif

struct FShaderResourceTable
{
	/** Bits indicating which resource tables contain resources bound to this shader. */
	uint32 ResourceTableBits = 0;

	/** Mapping of bound SRVs to their location in resource tables. */
	TArray<uint32> ShaderResourceViewMap;

	/** Mapping of bound sampler states to their location in resource tables. */
	TArray<uint32> SamplerMap;

	/** Mapping of bound UAVs to their location in resource tables. */
	TArray<uint32> UnorderedAccessViewMap;

	/** Hash of the layouts of resource tables at compile time, used for runtime validation. */
	TArray<uint32> ResourceTableLayoutHashes;

	/** Mapping of bound Textures to their location in resource tables. */
	TArray<uint32> TextureMap;

	/** Mapping of bound Resource Collections to their location in resource tables. */
	TArray<uint32> ResourceCollectionMap;

	friend bool operator == (const FShaderResourceTable& A, const FShaderResourceTable& B)
	{
		bool bEqual = true;
		bEqual &= (A.ResourceTableBits == B.ResourceTableBits);
		bEqual &= (A.ShaderResourceViewMap    .Num() == B.ShaderResourceViewMap    .Num());
		bEqual &= (A.SamplerMap               .Num() == B.SamplerMap               .Num());
		bEqual &= (A.UnorderedAccessViewMap   .Num() == B.UnorderedAccessViewMap   .Num());
		bEqual &= (A.ResourceTableLayoutHashes.Num() == B.ResourceTableLayoutHashes.Num());
		bEqual &= (A.TextureMap               .Num() == B.TextureMap               .Num());
		bEqual &= (A.ResourceCollectionMap    .Num() == B.ResourceCollectionMap    .Num());

		if (!bEqual)
		{
			return false;
		}

		bEqual &= (FMemory::Memcmp(A.ShaderResourceViewMap    .GetData(), B.ShaderResourceViewMap    .GetData(), A.ShaderResourceViewMap    .GetTypeSize() * A.ShaderResourceViewMap    .Num()) == 0);
		bEqual &= (FMemory::Memcmp(A.SamplerMap               .GetData(), B.SamplerMap               .GetData(), A.SamplerMap               .GetTypeSize() * A.SamplerMap               .Num()) == 0);
		bEqual &= (FMemory::Memcmp(A.UnorderedAccessViewMap   .GetData(), B.UnorderedAccessViewMap   .GetData(), A.UnorderedAccessViewMap   .GetTypeSize() * A.UnorderedAccessViewMap   .Num()) == 0);
		bEqual &= (FMemory::Memcmp(A.ResourceTableLayoutHashes.GetData(), B.ResourceTableLayoutHashes.GetData(), A.ResourceTableLayoutHashes.GetTypeSize() * A.ResourceTableLayoutHashes.Num()) == 0);
		bEqual &= (FMemory::Memcmp(A.TextureMap               .GetData(), B.TextureMap               .GetData(), A.TextureMap               .GetTypeSize() * A.TextureMap               .Num()) == 0);
		bEqual &= (FMemory::Memcmp(A.ResourceCollectionMap    .GetData(), B.ResourceCollectionMap    .GetData(), A.ResourceCollectionMap    .GetTypeSize() * A.ResourceCollectionMap    .Num()) == 0);
		return bEqual;
	}

	friend FArchive& operator << (FArchive& Ar, FShaderResourceTable& SRT)
	{
		Ar << SRT.ResourceTableBits;
		Ar << SRT.ShaderResourceViewMap;
		Ar << SRT.SamplerMap;
		Ar << SRT.UnorderedAccessViewMap;
		Ar << SRT.ResourceTableLayoutHashes;
		Ar << SRT.TextureMap;
		Ar << SRT.ResourceCollectionMap;
	
		return Ar;
	}
};

namespace UE
{
	namespace RHICore
	{
		// Workaround for layering issue. FShaderParametersMetadata is in RenderCore,
		// so we can't move the logic for initializing the StaticSlots array to the RHI module.
		void InitStaticUniformBufferSlots(FRHIShaderData* ShaderData);
	}
}

struct FRHICreateShaderDesc
{
	TConstArrayView<uint8> Code;
	// Optional Owner of the memory backing the View
	TSharedPtr<void> CodeOwner;
	// Optional pointer to the shader library this shader is being created from
	class FRHIShaderLibrary* Library;

	FRHICreateShaderDesc(TConstArrayView<uint8> InCode, class FRHIShaderLibrary* InLibrary = nullptr)
	: Code(InCode)
	, Library(InLibrary)
	{
	}

	template<typename DeleterType, typename = std::enable_if_t<!std::is_pointer_v<std::decay_t<DeleterType>>>>
	FRHICreateShaderDesc(TConstArrayView<uint8> InCode, DeleterType&& InDeleter, class FRHIShaderLibrary* InLibrary = nullptr)
	 : Code(InCode)
	// Giving the view data here serve two purposes:
	// - Hints on the intent that this ownership correspond to the data behind the View
	// - Make the Owner valid to differentiate between data that stay valid as long as we keep a reference to it versus temporary data
	 , CodeOwner(MakeShareable((void*)InCode.GetData(), Forward<DeleterType>(InDeleter)))
	 , Library(InLibrary)
	{
	}
};

class FRHIShaderData
{
public:
	const FShaderResourceTable& GetShaderResourceTable() const
	{
		return ShaderResourceTable;
	}

	const TArray<FUniformBufferStaticSlot>& GetStaticSlots() const
	{
		return StaticSlots;
	}

	void SerializeShaderResourceTable(FArchive& Ar)
	{
		Ar << ShaderResourceTable;
	}

protected:
	FShaderResourceTable ShaderResourceTable;
	TArray<FUniformBufferStaticSlot> StaticSlots;

	// Workaround for layering issue. FShaderParametersMetadata is in RenderCore,
	// so we can't move the logic for initializing the StaticSlots array to the RHI module.
	friend void UE::RHICore::InitStaticUniformBufferSlots(FRHIShaderData* ShaderData);
};

class FRHIShader : public FRHIResource, public FRHIShaderData
{
public:
	void SetHash(const FShaderHash& InHash) { Hash = InHash; }
	const FShaderHash& GetHash() const { return Hash; }

#if RHI_INCLUDE_SHADER_DEBUG_DATA

	// for debugging only e.g. MaterialName:ShaderFile.usf or ShaderFile.usf/EntryFunc
	struct
	{
		FString ShaderName;
		TArray<FName> UniformBufferNames;
	} Debug;
	
	bool HasShaderName() const { return !Debug.ShaderName.IsEmpty(); } 

	const TCHAR* GetShaderName() const
	{
		return Debug.ShaderName.Len()
			? *Debug.ShaderName
			: TEXT("<unknown>");
	}

	FString GetUniformBufferName(uint32 Index) const
	{
		return Debug.UniformBufferNames.IsValidIndex(Index)
			? Debug.UniformBufferNames[Index].ToString()
			: TEXT("<unknown>");
	}

	TArray<FShaderCodeValidationStride> DebugStrideValidationData;
	TArray<FShaderCodeValidationType> DebugSRVTypeValidationData;
	TArray<FShaderCodeValidationType> DebugUAVTypeValidationData;
	TArray<FShaderCodeValidationUBSize> DebugUBSizeValidationData;

#else

	bool HasShaderName() const { return false; }
	const TCHAR* GetShaderName() const { return TEXT("<unknown>"); }
	FString GetUniformBufferName(uint32 Index) const { return TEXT("<unknown>"); }

#endif

	FRHIShader() = delete;
	FRHIShader(ERHIResourceType InResourceType, EShaderFrequency InFrequency)
		: FRHIResource(InResourceType)
		, Frequency(InFrequency)
		, bNoDerivativeOps(false)
		, bHasShaderBundleUsage(false)
#if PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS
		, bHasSpecializationConstants(false)
#endif
	{
	}

#if PLATFORM_WINDOWS
	RHI_API virtual ~FRHIShader();	
	RHI_API void SetInUseByPSOCompilation(bool bInUse);
#endif // PLATFORM_WINDOWS

	inline EShaderFrequency GetFrequency() const
	{
		return Frequency;
	}

	inline void SetNoDerivativeOps(bool bValue)
	{
		bNoDerivativeOps = bValue;
	}

	inline bool HasNoDerivativeOps() const
	{
		return bNoDerivativeOps;
	}

	inline void SetShaderBundleUsage(bool bValue)
	{
		bHasShaderBundleUsage = bValue;
	}

	inline bool HasShaderBundleUsage() const
	{
		return bHasShaderBundleUsage;
	}

#if PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS
	inline void SetSpecializationConstants(bool bValue)
	{
		bHasSpecializationConstants = bValue;
	}

	inline bool HasSpecializationConstants() const
	{
		return bHasSpecializationConstants;
	}

	const TMap<uint32, TArray<uint32>>& GetSpecializationExclusions() const
	{
		return SpecializationExclusions;
	}

	void SetSpecializationExclusions(TMap<uint32, TArray<uint32>>&& Exclusions)
	{
		SpecializationExclusions = Exclusions;
	}

	bool IsExcludedFromSpecialization(uint32 BindPoint, uint32 SpecializationHash) const
	{
		if (bHasSpecializationConstants)
		{
			if (const TArray<uint32>* Array = SpecializationExclusions.Find(BindPoint))
			{
				return Array->Find(SpecializationHash) != INDEX_NONE;
			}
		}
		return false;
	}
#else
	constexpr bool HasSpecializationConstants() const { return false; }
#endif // PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS

private:
#if PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS
	TMap<uint32, TArray<uint32>> SpecializationExclusions;
#endif // PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS
	FShaderHash Hash;
	EShaderFrequency Frequency;
	uint8 bNoDerivativeOps : 1;
	uint8 bHasShaderBundleUsage : 1;
#if PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS
	uint8 bHasSpecializationConstants : 1;
#endif // PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS
#if PLATFORM_WINDOWS
	volatile int16 InUseByPSOCompilation = 0;
#endif // PLATFORM_WINDOWS
};

class FRHIGraphicsShader : public FRHIShader
{
public:
	explicit FRHIGraphicsShader(ERHIResourceType InResourceType, EShaderFrequency InFrequency)
		: FRHIShader(InResourceType, InFrequency) {}
};

class FRHIVertexShader : public FRHIGraphicsShader
{
public:
	FRHIVertexShader() : FRHIGraphicsShader(RRT_VertexShader, SF_Vertex) {}
};

class FRHIMeshShader : public FRHIGraphicsShader
{
public:
	FRHIMeshShader() : FRHIGraphicsShader(RRT_MeshShader, SF_Mesh) {}
};

class FRHIAmplificationShader : public FRHIGraphicsShader
{
public:
	FRHIAmplificationShader() : FRHIGraphicsShader(RRT_AmplificationShader, SF_Amplification) {}
};

class FRHIPixelShader : public FRHIGraphicsShader
{
public:
	FRHIPixelShader() : FRHIGraphicsShader(RRT_PixelShader, SF_Pixel) {}
};

class FRHIGeometryShader : public FRHIGraphicsShader
{
public:
	FRHIGeometryShader() : FRHIGraphicsShader(RRT_GeometryShader, SF_Geometry) {}
};

class FRHIRayTracingShader : public FRHIShader
{
public:
	explicit FRHIRayTracingShader(EShaderFrequency InFrequency) : FRHIShader(RRT_RayTracingShader, InFrequency) {}

	uint32 RayTracingPayloadType = 0; // This corresponds to the ERayTracingPayloadType enum associated with the shader
	uint32 RayTracingPayloadSize = 0; // The (maximum) size of the payload associated with this shader
	uint32 LocalBindingDataSize = 0; // Size of the local shader binding data needed for this shader
};

class FRHIRayGenShader : public FRHIRayTracingShader
{
public:
	FRHIRayGenShader() : FRHIRayTracingShader(SF_RayGen) {}
};

class FRHIRayMissShader : public FRHIRayTracingShader
{
public:
	FRHIRayMissShader() : FRHIRayTracingShader(SF_RayMiss) {}
};

class FRHIRayCallableShader : public FRHIRayTracingShader
{
public:
	FRHIRayCallableShader() : FRHIRayTracingShader(SF_RayCallable) {}
};

class FRHIRayHitGroupShader : public FRHIRayTracingShader
{
public:
	FRHIRayHitGroupShader() : FRHIRayTracingShader(SF_RayHitGroup) {}
};

class FRHIComputeShader : public FRHIShader
{
public:
	FRHIComputeShader() : FRHIShader(RRT_ComputeShader, SF_Compute)
	, Stats(nullptr)
	{
	}
	
	inline void SetStats(struct FPipelineStateStats* Ptr) { Stats = Ptr; }
	RHI_API void UpdateStats();

private:
	struct FPipelineStateStats* Stats;
};

class FRHIWorkGraphShader : public FRHIShader
{
public:
	explicit FRHIWorkGraphShader(EShaderFrequency InFrequency)
		: FRHIShader(RRT_WorkGraphShader, InFrequency)
	{
	}
};

class FRHIWorkGraphRootShader : public FRHIWorkGraphShader
{
public:
	FRHIWorkGraphRootShader() : FRHIWorkGraphShader(SF_WorkGraphRoot) {}
};

class FRHIWorkGraphComputeNodeShader : public FRHIWorkGraphShader
{
public:
	FRHIWorkGraphComputeNodeShader() : FRHIWorkGraphShader(SF_WorkGraphComputeNode) {}
};

//
// Pipeline States
//

class FRHIGraphicsPipelineState : public FRHIResource 
{
public:
	FRHIGraphicsPipelineState() : FRHIResource(RRT_GraphicsPipelineState) {}

	inline void SetSortKey(uint64 InSortKey) { SortKey = InSortKey; }
	inline uint64 GetSortKey() const { return SortKey; }

	virtual FRHIGraphicsShader* GetShader(EShaderFrequency Frequency) const = 0;

private:
	uint64 SortKey = 0;

#if ENABLE_RHI_VALIDATION
	friend class FValidationContext;
	friend class FValidationRHI;
	FExclusiveDepthStencil DSMode;
#endif
};

class FRHIComputePipelineState : public FRHIResource
{
public:
	FRHIComputePipelineState(FRHIComputeShader* InComputeShader) :
		FRHIResource(RRT_ComputePipelineState)
		, ComputeShader(InComputeShader)
	{
		check(InComputeShader);
	}

	inline void SetValid(bool InIsValid) { bIsValid = InIsValid; }
	inline bool IsValid() const { return bIsValid; }
	virtual void MarkUsed() { bUsed = true; }
	bool IsUsed() { return bUsed; }

	inline FRHIComputeShader* GetComputeShader() const
	{
		return ComputeShader;
	}

protected:
	TRefCountPtr<FRHIComputeShader> ComputeShader;

private:
	bool bIsValid = true;
	bool bUsed = false;
};

class FRHIWorkGraphPipelineState : public FRHIResource
{
public:
	FRHIWorkGraphPipelineState() : FRHIResource(RRT_WorkGraphPipelineState) {}
};

class FRHIRayTracingPipelineState : public FRHIResource
#if ENABLE_RHI_VALIDATION
	, public RHIValidation::FRayTracingPipelineState
#endif
{
public:
	FRHIRayTracingPipelineState(const FRayTracingPipelineStateInitializer& InInitializer) : FRHIResource(RRT_RayTracingPipelineState) 
#if ENABLE_RHI_VALIDATION
		, RHIValidation::FRayTracingPipelineState(InInitializer)
#endif
	{}
};

//
// Buffers
//

struct FRHIUniformBufferResource
{
	/** Byte offset to each resource in the uniform buffer memory. */
	uint16 MemberOffset;

	/** Type of the member that allow (). */
	EUniformBufferBaseType MemberType;

	/** Compare two uniform buffer layout resources. */
	friend inline bool operator==(const FRHIUniformBufferResource& A, const FRHIUniformBufferResource& B)
	{
		return A.MemberOffset == B.MemberOffset
			&& A.MemberType == B.MemberType;
	}
};

inline constexpr uint16 kUniformBufferInvalidOffset = TNumericLimits<uint16>::Max();

struct FRHIUniformBufferLayoutInitializer;

/** The layout of a uniform buffer in memory. */
struct FRHIUniformBufferLayout : public FRHIResource
{
	FRHIUniformBufferLayout() = delete;

	RHI_API explicit FRHIUniformBufferLayout(const FRHIUniformBufferLayoutInitializer& Initializer);

	inline const FString& GetDebugName() const
	{
		return Name;
	}

	inline uint32 GetHash() const
	{
		checkSlow(Hash != 0);
		return Hash;
	}

	inline bool HasRenderTargets() const
	{
		return RenderTargetsOffset != kUniformBufferInvalidOffset;
	}

	inline bool HasExternalOutputs() const
	{
		return EnumHasAnyFlags(Flags, ERHIUniformBufferFlags::HasNonGraphOutputs);
	}

	inline bool HasStaticSlot() const
	{
		return IsUniformBufferStaticSlotValid(StaticSlot);
	}

	const FString Name;

	/** The list of all resource inlined into the shader parameter structure. */
	const TArray<FRHIUniformBufferResource> Resources;

	/** The list of all RDG resource references inlined into the shader parameter structure. */
	const TArray<FRHIUniformBufferResource> GraphResources;

	/** The list of all RDG texture references inlined into the shader parameter structure. */
	const TArray<FRHIUniformBufferResource> GraphTextures;

	/** The list of all RDG buffer references inlined into the shader parameter structure. */
	const TArray<FRHIUniformBufferResource> GraphBuffers;

	/** The list of all RDG uniform buffer references inlined into the shader parameter structure. */
	const TArray<FRHIUniformBufferResource> GraphUniformBuffers;

	/** The list of all non-RDG uniform buffer references inlined into the shader parameter structure. */
	const TArray<FRHIUniformBufferResource> UniformBuffers;

	const uint32 Hash;

	/** The size of the constant buffer in bytes. */
	const uint32 ConstantBufferSize;

	/** The render target binding slots offset, if it exists. */
	const uint16 RenderTargetsOffset;

	/** The static slot (if applicable). */
	const FUniformBufferStaticSlot StaticSlot;

	/** The binding flags describing how this resource can be bound to the RHI. */
	const EUniformBufferBindingFlags BindingFlags;

	/** Flags to signal different Uniform Buffer states. */
	const ERHIUniformBufferFlags Flags;

	/** Compare two uniform buffer layouts. */
	friend inline bool operator==(const FRHIUniformBufferLayout& A, const FRHIUniformBufferLayout& B)
	{
		return A.ConstantBufferSize == B.ConstantBufferSize
			&& A.StaticSlot == B.StaticSlot
			&& A.BindingFlags == B.BindingFlags
			&& A.Resources == B.Resources;
	}
};

class FRHIUniformBuffer : public FRHIResource
#if ENABLE_RHI_VALIDATION
	, public RHIValidation::FUniformBufferResource
#endif
{
public:
	FRHIUniformBuffer() = delete;

	/** Initialization constructor. */
	FRHIUniformBuffer(const FRHIUniformBufferLayout* InLayout)
	: FRHIResource(RRT_UniformBuffer)
	, Layout(InLayout)
	, LayoutConstantBufferSize(InLayout->ConstantBufferSize)
	{}

	/** @return The number of bytes in the uniform buffer. */
	uint32 GetSize() const
	{
		check(LayoutConstantBufferSize == Layout->ConstantBufferSize);
		return LayoutConstantBufferSize;
	}
	const FRHIUniformBufferLayout& GetLayout() const { return *Layout; }
	const FRHIUniformBufferLayout* GetLayoutPtr() const { return Layout; }

	const TArray<TRefCountPtr<FRHIResource>>& GetResourceTable() const { return ResourceTable; }

	virtual FRHIDescriptorHandle GetBindlessHandle() const
	{
		return FRHIDescriptorHandle();
	}

protected:
	TArray<TRefCountPtr<FRHIResource>> ResourceTable;

	RHI_API virtual ~FRHIUniformBuffer();

private:
	/** Layout of the uniform buffer. */
	TRefCountPtr<const FRHIUniformBufferLayout> Layout;

	uint32 LayoutConstantBufferSize;
};

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
namespace UE::RHI::UniformBufferContentMap
{
	RHI_API void Update(const FRHIUniformBuffer* InUniformBuffer, const void* Contents);
	RHI_API void Remove(const FRHIUniformBuffer* InUniformBuffer);
	RHI_API const void* Get(const FRHIUniformBuffer* InUniformBuffer);
}
#endif

class FRHIViewableResource : public FRHIResource
{
public:
	ERHIAccess GetTrackedAccess_Unsafe() const
	{
		return TrackedAccess.Access;
	}

	FName GetName() const
	{
		return Name;
	}

#if ENABLE_RHI_VALIDATION
	virtual RHIValidation::FResource* GetValidationTrackerResource() = 0;
#endif

protected:
	FRHIViewableResource(ERHIResourceType InResourceType, ERHIAccess InAccess, const TCHAR* InName, FName InOwnerName)
		: FRHIResource(InResourceType)
		, TrackedAccess(InAccess)
	{
#if RHI_ENABLE_RESOURCE_INFO
		if (InName)
		{
			Name = FName(InName);
		}
		SetOwnerName(InOwnerName);
#endif
	}

	virtual void SetTrackedAccessFromContext(FRHITrackedAccess InTrackedAccess)
	{
		TrackedAccess = InTrackedAccess;
	}

	void TakeOwnership(FRHIViewableResource& Other)
	{
		TrackedAccess = Other.TrackedAccess;
	}

	void ReleaseOwnership()
	{
		TrackedAccess = ERHIAccess::Unknown;
	}

	FName Name;

private:
	FRHITrackedAccess TrackedAccess;

	friend class FRHICommandListBase;
	friend class FRHIComputeCommandList;
	friend class IRHIComputeContext;
};

struct FRHIBufferDesc
{
	/** Size of the buffer memory, in bytes */
	uint32 Size = 0;

	/** Distance in bytes between elements of the buffer */
	uint32 Stride = 0;

	/** Flags describing the usage of the buffer */
	EBufferUsageFlags Usage = EBufferUsageFlags::None;

	/* A mask representing which GPUs to create the resource on, in a multi-GPU system. */
	FRHIGPUMask GPUMask = FRHIGPUMask::All();

	FRHIBufferDesc() = default;
	FRHIBufferDesc(uint32 InSize, uint32 InStride, EBufferUsageFlags InUsage)
		: Size  (InSize)
		, Stride(InStride)
		, Usage (InUsage)
	{
	}
	FRHIBufferDesc(uint32 InSize, uint32 InStride, EBufferUsageFlags InUsage, FRHIGPUMask InGPUMask)
		: Size   (InSize)
		, Stride (InStride)
		, Usage  (InUsage)
		, GPUMask(InGPUMask)
	{
	}

	static FRHIBufferDesc Null()
	{
		return FRHIBufferDesc(0, 0, EBufferUsageFlags::NullResource);
	}

	bool IsNull() const
	{
		if (EnumHasAnyFlags(Usage, EBufferUsageFlags::NullResource))
		{
			// The null resource descriptor should have its other fields zeroed, and no additional flags.
			check(Size == 0 && Stride == 0 && Usage == EBufferUsageFlags::NullResource);
			return true;
		}

		return false;
	}

	friend uint32 GetTypeHash(const FRHIBufferDesc& Desc)
	{
		uint32 Hash = GetTypeHash(Desc.Size);
		Hash = HashCombine(Hash, GetTypeHash(Desc.Stride));
		Hash = HashCombine(Hash, GetTypeHash(Desc.Usage));
		Hash = HashCombine(Hash, GetTypeHash(Desc.GPUMask.GetNative()));
		return Hash;
	}

	bool operator==(const FRHIBufferDesc& Other) const
	{
		return Size == Other.Size
			&& Stride == Other.Stride
			&& Usage == Other.Usage
			&& GPUMask == Other.GPUMask;
	}

	bool operator!=(const FRHIBufferDesc& Other) const
	{
		return !(*this == Other);
	}

	FRHIBufferDesc& operator=(const FRHIBufferDesc& Other)
	{
		Size    = Other.Size;
		Stride  = Other.Stride;
		Usage   = Other.Usage;
		GPUMask = Other.GPUMask;

		return *this;
	}
};

RHI_API ERHIAccess RHIGetDefaultResourceState(EBufferUsageFlags InUsage, bool bInHasInitialData);

enum class ERHIBufferInitAction
{
	// Default for the RHI, data can be "undefined"
	Default,

	// Zero all buffer data
	Zeroed,

	// Upload data from a provided FResourceArrayUploadInterface. This data will be discarded after it's used.
	ResourceArray,

	// Caller will use FRHIBufferInitializer to set the initial buffer contents.
	Initializer,
};

struct FRHIBufferCreateDesc : public FRHIBufferDesc
{
	static FRHIBufferCreateDesc Create(const TCHAR* InDebugName, EBufferUsageFlags InUsage)
	{
		return FRHIBufferCreateDesc(InDebugName, InUsage);
	}

	static FRHIBufferCreateDesc Create(const TCHAR* InDebugName, uint32 InSize, uint32 InStride, EBufferUsageFlags InUsage)
	{
		return FRHIBufferCreateDesc(InDebugName, InSize, InStride, InUsage);
	}
	
	static FRHIBufferCreateDesc Create(const TCHAR* InDebugName, const FRHIBufferDesc& InDesc)
	{
		return FRHIBufferCreateDesc(InDebugName, InDesc);
	}

	static FRHIBufferCreateDesc CreateNull(const TCHAR* InDebugName)
	{
		return Create(InDebugName, 0, 0, EBufferUsageFlags::NullResource);
	}

	static FRHIBufferCreateDesc CreateVertex(const TCHAR* InDebugName)
	{
		return Create(InDebugName, EBufferUsageFlags::VertexBuffer);
	}

	static FRHIBufferCreateDesc CreateVertex(const TCHAR* InDebugName, uint32 InSize)
	{
		return Create(InDebugName, InSize, 0, EBufferUsageFlags::VertexBuffer);
	}

	template<typename TVertexType>
	static FRHIBufferCreateDesc CreateVertex(const TCHAR* InDebugName, uint32 InCount)
	{
		return Create(InDebugName, InCount * sizeof(TVertexType), 0, EBufferUsageFlags::VertexBuffer);
	}

	static FRHIBufferCreateDesc CreateIndex(const TCHAR* InDebugName)
	{
		return Create(InDebugName, EBufferUsageFlags::IndexBuffer);
	}

	static FRHIBufferCreateDesc CreateIndex(const TCHAR* InDebugName, uint32 InSize, uint32 InStride)
	{
		return Create(InDebugName, InSize, InStride, EBufferUsageFlags::IndexBuffer);
	}

	template<typename TIndexType>
	static FRHIBufferCreateDesc CreateIndex(const TCHAR* InDebugName, uint32 InCount)
	{
		return Create(InDebugName, InCount * sizeof(TIndexType), sizeof(TIndexType), EBufferUsageFlags::IndexBuffer);
	}

	static FRHIBufferCreateDesc CreateStructured(const TCHAR* InDebugName)
	{
		return Create(InDebugName, EBufferUsageFlags::StructuredBuffer);
	}

	static FRHIBufferCreateDesc CreateStructured(const TCHAR* InDebugName, uint32 InSize, uint32 InStride)
	{
		return Create(InDebugName, InSize, InStride, EBufferUsageFlags::StructuredBuffer);
	}

	template<typename TStructureType>
	static FRHIBufferCreateDesc CreateStructured(const TCHAR* InDebugName, uint32 InCount)
	{
		return Create(InDebugName, InCount * sizeof(TStructureType), sizeof(TStructureType), EBufferUsageFlags::StructuredBuffer);
	}

	static FRHIBufferCreateDesc CreateByteAddress(const TCHAR* InDebugName)
	{
		return Create(InDebugName, EBufferUsageFlags::ByteAddressBuffer);
	}

	static FRHIBufferCreateDesc CreateByteAddress(const TCHAR* InDebugName, uint32 InSize, uint32 InStride)
	{
		return Create(InDebugName, InSize, InStride, EBufferUsageFlags::ByteAddressBuffer);
	}

	static FRHIBufferCreateDesc CreateUniform(const TCHAR* InDebugName)
	{
		return Create(InDebugName, EBufferUsageFlags::UniformBuffer);
	}

	FRHIBufferCreateDesc() = default;

	FRHIBufferCreateDesc(const TCHAR* InDebugName, EBufferUsageFlags InUsage)
		: DebugName(InDebugName)
	{
		Usage = InUsage;
	}

	FRHIBufferCreateDesc(const TCHAR* InDebugName, uint32 InSize, uint32 InStride, EBufferUsageFlags InUsage)
		: FRHIBufferDesc(InSize, InStride, InUsage)
		, DebugName(InDebugName)
	{
	}

	FRHIBufferCreateDesc(const TCHAR* InDebugName, const FRHIBufferDesc& InOtherDesc)
		: FRHIBufferDesc(InOtherDesc)
		, DebugName(InDebugName)
	{
	}

	FRHIBufferCreateDesc& SetSize(uint32 InSize)              { Size   = InSize;     return *this; }
	FRHIBufferCreateDesc& SetStride(uint32 InStride)          { Stride = InStride;   return *this; }
	FRHIBufferCreateDesc& SetUsage(EBufferUsageFlags InUsage) { Usage  = InUsage;    return *this; }
	FRHIBufferCreateDesc& AddUsage(EBufferUsageFlags InUsage) { Usage |= InUsage;    return *this; }
	FRHIBufferCreateDesc& SetGPUMask(FRHIGPUMask InGPUMask)   { GPUMask = InGPUMask; return *this; }

	FRHIBufferCreateDesc& SetDebugName(const TCHAR* InDebugName)           { DebugName = InDebugName;       return *this; }
	FRHIBufferCreateDesc& SetInitialState(ERHIAccess InInitialState)       { InitialState = InInitialState; return *this; }
	FRHIBufferCreateDesc& DetermineInitialState()                          { if (InitialState == ERHIAccess::Unknown) InitialState = RHIGetDefaultResourceState(Usage, false); return *this; }
	FRHIBufferCreateDesc& SetInitAction(ERHIBufferInitAction InInitAction) { InitAction = InInitAction;     return *this; }
	FRHIBufferCreateDesc& SetClassName(FName InClassName)                  { ClassName = InClassName;       return *this; }
	FRHIBufferCreateDesc& SetOwnerName(FName InOwnerName)                  { OwnerName = InOwnerName;       return *this; }

	FRHIBufferCreateDesc& SetInitActionNone()
	{
		return SetInitAction(ERHIBufferInitAction::Default);
	}
	FRHIBufferCreateDesc& SetInitActionZeroData()
	{
		return SetInitAction(ERHIBufferInitAction::Zeroed);
	}
	FRHIBufferCreateDesc& SetInitActionInitializer()
	{
		return SetInitAction(ERHIBufferInitAction::Initializer);
	}
	FRHIBufferCreateDesc& SetInitActionResourceArray(FResourceArrayUploadInterface* InInitialData)
	{
		InitialData = InInitialData;
		return SetInitAction(ERHIBufferInitAction::ResourceArray);
	}

	FName GetTraceClassName() const
	{
		const static FLazyName FRHIBufferName(TEXT("FRHIBuffer"));
		return (ClassName == NAME_None) ? FRHIBufferName : ClassName;
	}

	/* A friendly name for the resource. */
	const TCHAR* DebugName = nullptr;

	/* Provider of initial data for the buffer. InitialData->Discard() will be called after the data is read. */
	FResourceArrayUploadInterface* InitialData = nullptr;

	/* The RHI access state that the resource will be created in. */
	ERHIAccess InitialState = ERHIAccess::Unknown;

	/* Tells how to initialize (or not) the buffer's data. */
	ERHIBufferInitAction InitAction = ERHIBufferInitAction::Default;

	/* The owner class of FRHIBuffer used for Insight asset metadata tracing */
	FName ClassName = NAME_None;

	/* The owner name used for Insight asset metadata tracing */
	FName OwnerName = NAME_None;
};

class FRHIBuffer : public FRHIViewableResource
#if ENABLE_RHI_VALIDATION
	, public RHIValidation::FBufferResource
#endif
{
protected:
	FRHIBuffer() = delete;

	/** Initialization constructor. Should only be called by platform RHI implementations. */
	RHI_API FRHIBuffer(const FRHIBufferCreateDesc& CreateDesc);

public:
	const FRHIBufferDesc& GetDesc() const
	{
		return Desc;
	}

	/** @return The number of bytes in the buffer. */
	uint32 GetSize() const
	{
		return Desc.Size;
	}

	/** @return The stride in bytes of the buffer. */
	uint32 GetStride() const
	{
		return Desc.Stride;
	}

	/** @return The usage flags used to create the buffer. */
	EBufferUsageFlags GetUsage() const
	{
		return Desc.Usage;
	}

	void SetName(FName InName)
	{
		Name = InName;
	}

#if ENABLE_RHI_VALIDATION
	virtual RHIValidation::FResource* GetValidationTrackerResource() final override
	{
		return this;
	}
#endif

protected:
	void TakeOwnership(FRHIBuffer& Other)
	{
		FRHIViewableResource::TakeOwnership(Other);
		Desc = Other.Desc;
	}

	void ReleaseOwnership()
	{
		FRHIViewableResource::ReleaseOwnership();
		Desc = FRHIBufferDesc::Null();
	}

private:
	FRHIBufferDesc Desc;
};

/** Represents a simple indirection to a vertex buffer to bind to a source stream. The underlying vertex buffer can be updated dynamically using an RHI command list. */
class FRHIStreamSourceSlot : public FRHIResource
{
	friend FRHICommandListBase;
	friend FRHICommandList;
public:
	static TRefCountPtr<FRHIStreamSourceSlot> Create(FRHIBuffer* InBuffer)
	{
		return new FRHIStreamSourceSlot(InBuffer);
	}

private:
	FRHIStreamSourceSlot(FRHIBuffer* InBuffer)
		: FRHIResource(RRT_StreamSourceSlot)
		, Buffer(InBuffer)
	{}

	TRefCountPtr<FRHIBuffer> Buffer;
};

//
// Textures
//

/** Descriptor used to create a texture resource */
struct FRHITextureDesc
{
	FRHITextureDesc() = default;

	FRHITextureDesc(const FRHITextureDesc& Other)
	{
		*this = Other;
	}

	FRHITextureDesc(ETextureDimension InDimension)
		: Dimension(InDimension)
	{}

	FRHITextureDesc(
		  ETextureDimension   InDimension
		, ETextureCreateFlags InFlags
		, EPixelFormat        InFormat
		, FClearValueBinding  InClearValue
		, FIntPoint           InExtent
		, uint16              InDepth
		, uint16              InArraySize
		, uint8               InNumMips
		, uint8               InNumSamples
		, uint32              InExtData
		)
		: Flags     (InFlags     )
		, ClearValue(InClearValue)
		, ExtData   (InExtData   )
		, Extent    (InExtent    )
		, Depth     (InDepth     )
		, ArraySize (InArraySize )
		, NumMips   (InNumMips   )
		, NumSamples(InNumSamples)
		, Dimension (InDimension )
		, Format    (InFormat    )
	{}

	friend uint32 GetTypeHash(const FRHITextureDesc& Desc)
	{
		uint32 Hash = GetTypeHash(Desc.Dimension);
		Hash = HashCombine(Hash, GetTypeHash(Desc.Flags		));
		Hash = HashCombine(Hash, GetTypeHash(Desc.Format	));
		Hash = HashCombine(Hash, GetTypeHash(Desc.UAVFormat	));
		Hash = HashCombine(Hash, GetTypeHash(Desc.Extent	));
		Hash = HashCombine(Hash, GetTypeHash(Desc.Depth		));
		Hash = HashCombine(Hash, GetTypeHash(Desc.ArraySize	));
		Hash = HashCombine(Hash, GetTypeHash(Desc.NumMips	));
		Hash = HashCombine(Hash, GetTypeHash(Desc.NumSamples));
		Hash = HashCombine(Hash, GetTypeHash(Desc.FastVRAMPercentage));
		Hash = HashCombine(Hash, GetTypeHash(Desc.ClearValue));
		Hash = HashCombine(Hash, GetTypeHash(Desc.ExtData   ));
		Hash = HashCombine(Hash, GetTypeHash(Desc.GPUMask.GetNative()));
		Hash = HashCombine(Hash, GetTypeHash(Desc.AliasableFormats));
		// Add BlockBytes to the hash computation to handle cases where PixelFormat changes at runtime (e.g., PF_SceneDepth).
		// This ensures the hash correctly reflects changes in block size, preventing potential mismatches or incorrect caching when the pixel format varies dynamically.
		Hash = HashCombine(Hash, GetTypeHash(GPixelFormats[Desc.Format].BlockBytes));
		return Hash;
	}

	bool operator == (const FRHITextureDesc& Other) const
	{
		return Dimension  == Other.Dimension
			&& Flags      == Other.Flags
			&& Format     == Other.Format
			&& UAVFormat  == Other.UAVFormat
			&& Extent     == Other.Extent
			&& Depth      == Other.Depth
			&& ArraySize  == Other.ArraySize
			&& NumMips    == Other.NumMips
			&& NumSamples == Other.NumSamples
			&& FastVRAMPercentage == Other.FastVRAMPercentage
			&& ClearValue == Other.ClearValue
			&& ExtData    == Other.ExtData
			&& GPUMask    == Other.GPUMask
			&& AliasableFormats	== Other.AliasableFormats;
	}

	bool operator != (const FRHITextureDesc& Other) const
	{
		return !(*this == Other);
	}

	FRHITextureDesc& operator=(const FRHITextureDesc& Other)
	{
		Dimension			= Other.Dimension;
		Flags				= Other.Flags;
		Format				= Other.Format;
		UAVFormat			= Other.UAVFormat;
		Extent				= Other.Extent;
		Depth				= Other.Depth;
		ArraySize			= Other.ArraySize;
		NumMips				= Other.NumMips;
		NumSamples			= Other.NumSamples;
		ClearValue			= Other.ClearValue;
		ExtData				= Other.ExtData;
		FastVRAMPercentage	= Other.FastVRAMPercentage;
		GPUMask				= Other.GPUMask;
		AliasableFormats	= Other.AliasableFormats;

		return *this;
	}

	bool IsTexture2D() const
	{
		return Dimension == ETextureDimension::Texture2D || Dimension == ETextureDimension::Texture2DArray;
	}

	bool IsTexture3D() const
	{
		return Dimension == ETextureDimension::Texture3D;
	}

	bool IsTextureCube() const
	{
		return Dimension == ETextureDimension::TextureCube || Dimension == ETextureDimension::TextureCubeArray;
	}

	bool IsTextureArray() const
	{
		return Dimension == ETextureDimension::Texture2DArray || Dimension == ETextureDimension::TextureCubeArray;
	}

	bool IsMipChain() const
	{
		return NumMips > 1;
	}

	bool IsMultisample() const
	{
		return NumSamples > 1;
	}

	FIntVector GetSize() const
	{
		return FIntVector(Extent.X, Extent.Y, Depth);
	}

	void Reset()
	{
		// Usually we don't want to propagate MSAA samples.
		NumSamples = 1;

		// Remove UAV flag for textures that don't need it (some formats are incompatible).
		Flags |= TexCreate_RenderTargetable;
		Flags &= ~(TexCreate_UAV | TexCreate_ResolveTargetable | TexCreate_DepthStencilResolveTarget | TexCreate_Memoryless);

		AliasableFormats.Empty();
	}

	/** Returns whether this descriptor conforms to requirements. */
	bool IsValid() const
	{
		return FRHITextureDesc::Validate(*this, /* Name = */ TEXT(""), /* bFatal = */ false);
	}

	/** Texture flags passed on to RHI texture. */
	ETextureCreateFlags Flags = TexCreate_None;

	/** Clear value to use when fast-clearing the texture. */
	FClearValueBinding ClearValue;

	/* A mask representing which GPUs to create the resource on, in a multi-GPU system. */
	FRHIGPUMask GPUMask = FRHIGPUMask::All();

	/** Platform-specific additional data. Used for offline processed textures on some platforms. */
	uint32 ExtData = 0;

	/** Extent of the texture in x and y. */
	FIntPoint Extent = FIntPoint(1, 1);

	/** Depth of the texture if the dimension is 3D. */
	uint16 Depth = 1;

	/** The number of array elements in the texture. (Keep at 1 if dimension is 3D). */
	uint16 ArraySize = 1;

	/** Number of mips in the texture mip-map chain. */
	uint8 NumMips = 1;

	/** Number of samples in the texture. >1 for MSAA. */
	uint8 NumSamples = 1;

	/** Texture dimension to use when creating the RHI texture. */
	ETextureDimension Dimension = ETextureDimension::Texture2D;

	/** Pixel format used to create RHI texture. */
	EPixelFormat Format = PF_Unknown;

	/** Texture format used when creating the UAV. PF_Unknown means to use the default one (same as Format). */
	EPixelFormat UAVFormat = PF_Unknown;

	/** Resource memory percentage which should be allocated onto fast VRAM (hint-only). (encoding into 8bits, 0..255 -> 0%..100%) */
	uint8 FastVRAMPercentage = 0xFF;

	/** Array of pixel formats that this texture can alias to */
	TArray<EPixelFormat, TInlineAllocator<1>> AliasableFormats;

	/** Check the validity. */
	static bool CheckValidity(const FRHITextureDesc& Desc, const TCHAR* Name)
	{
		return FRHITextureDesc::Validate(Desc, Name, /* bFatal = */ true);
	}

	/**
	 * Returns an estimated total memory size the described texture will occupy in GPU memory.
	 * This is an estimate because it only considers the dimensions / format etc of the texture, 
	 * not any specifics about platform texture layout.
	 * 
	 * To get a true measure of a texture resource for the current running platform RHI, use RHICalcTexturePlatformSize().
	 * 
	 * @param FirstMipIndex - the index of the most detailed mip to consider in the memory size calculation. Must be < NumMips and <= LastMipIndex.
	 * @param LastMipIndex  - the index of the least detailed mip to consider in the memory size calculation. Must be < NumMips and >= FirstMipIndex.
	 */
	RHI_API uint64 CalcMemorySizeEstimate(uint32 FirstMipIndex, uint32 LastMipIndex) const;

	uint64 CalcMemorySizeEstimate(uint32 FirstMipIndex = 0) const
	{
		return CalcMemorySizeEstimate(FirstMipIndex, NumMips - 1);
	}

	uint16 GetSubresourceCount() const
	{
		uint16 NumFaces = ((Dimension == ETextureDimension::TextureCube) || (Dimension == ETextureDimension::TextureCubeArray)) ? 6 : 1;
		uint16 NumPlanes = (IsStencilFormat(Format) || Format == PF_D24) ? 2 : 1;

		return (ArraySize * (NumMips * NumFaces) * NumPlanes);
	}

private:
	RHI_API static bool Validate(const FRHITextureDesc& Desc, const TCHAR* Name, bool bFatal);
};

// @todo deprecate
using FRHITextureCreateInfo = FRHITextureDesc;

extern RHI_API ERHIAccess RHIGetDefaultResourceState(ETextureCreateFlags InUsage, bool bInHasInitialData);

enum class ERHITextureInitAction
{
	// Default for the RHI, data can be "undefined" after creation.
	Default,

	// Upload data from a provided FResourceBulkDataInterface. This data will be discarded after it's used.
	BulkData,

	// Caller will use FRHITextureInitializer to set the initial texture contents.
	Initializer,
};

struct FRHITextureCreateDesc : public FRHITextureDesc
{
	static FRHITextureCreateDesc Create(const TCHAR* InDebugName, ETextureDimension InDimension)
	{
		return FRHITextureCreateDesc(InDebugName, InDimension);
	}

	static FRHITextureCreateDesc Create2D(const TCHAR* InDebugName)
	{
		return FRHITextureCreateDesc(InDebugName, ETextureDimension::Texture2D);
	}

	static FRHITextureCreateDesc Create2DArray(const TCHAR* InDebugName)
	{
		return FRHITextureCreateDesc(InDebugName, ETextureDimension::Texture2DArray);
	}

	static FRHITextureCreateDesc Create3D(const TCHAR* InDebugName)
	{
		return FRHITextureCreateDesc(InDebugName, ETextureDimension::Texture3D);
	}

	static FRHITextureCreateDesc CreateCube(const TCHAR* InDebugName)
	{
		return FRHITextureCreateDesc(InDebugName, ETextureDimension::TextureCube);
	}

	static FRHITextureCreateDesc CreateCubeArray(const TCHAR* InDebugName)
	{
		return FRHITextureCreateDesc(InDebugName, ETextureDimension::TextureCubeArray);
	}

	static FRHITextureCreateDesc Create2D(const TCHAR* DebugName, FIntPoint Size, EPixelFormat Format)
	{
		return Create2D(DebugName)
			.SetExtent(Size)
			.SetFormat(Format);
	}

	static FRHITextureCreateDesc Create2D(const TCHAR* DebugName, int32 SizeX, int32 SizeY, EPixelFormat Format)
	{
		return Create2D(DebugName)
			.SetExtent(SizeX, SizeY)
			.SetFormat(Format);
	}

	static FRHITextureCreateDesc Create2DArray(const TCHAR* DebugName, FIntPoint Size, uint16 ArraySize, EPixelFormat Format)
	{
		return Create2DArray(DebugName)
			.SetExtent(Size)
			.SetFormat(Format)
			.SetArraySize((uint16)ArraySize);
	}

	static FRHITextureCreateDesc Create2DArray(const TCHAR* DebugName, int32 SizeX, int32 SizeY, int32 ArraySize, EPixelFormat Format)
	{
		return Create2DArray(DebugName)
			.SetExtent(SizeX, SizeY)
			.SetFormat(Format)
			.SetArraySize((uint16)ArraySize);
	}

	static FRHITextureCreateDesc Create3D(const TCHAR* DebugName, FIntVector Size, EPixelFormat Format)
	{
		return Create3D(DebugName)
			.SetExtent(Size.X, Size.Y)
			.SetDepth((uint16)Size.Z)
			.SetFormat(Format);
	}

	static FRHITextureCreateDesc Create3D(const TCHAR* DebugName, int32 SizeX, int32 SizeY, int32 SizeZ, EPixelFormat Format)
	{
		return Create3D(DebugName)
			.SetExtent(SizeX, SizeY)
			.SetDepth((uint16)SizeZ)
			.SetFormat(Format);
	}

	static FRHITextureCreateDesc CreateCube(const TCHAR* DebugName, uint32 Size, EPixelFormat Format)
	{
		return CreateCube(DebugName)
			.SetExtent(Size)
			.SetFormat(Format);
	}

	static FRHITextureCreateDesc CreateCubeArray(const TCHAR* DebugName, uint32 Size, uint16 ArraySize, EPixelFormat Format)
	{
		return CreateCubeArray(DebugName)
			.SetExtent(Size)
			.SetFormat(Format)
			.SetArraySize((uint16)ArraySize);
	}

	FRHITextureCreateDesc() = default;

	// Constructor with minimal argument set. Name and dimension are always required.
	FRHITextureCreateDesc(const TCHAR* InDebugName, ETextureDimension InDimension)
		: FRHITextureDesc(InDimension)
		, DebugName(InDebugName)
	{
	}

	// Constructor for when you already have an FRHITextureDesc
	FRHITextureCreateDesc(
		  FRHITextureDesc const&      InDesc
		, ERHIAccess                  InInitialState
		, TCHAR const*                InDebugName
		, FResourceBulkDataInterface* InBulkData     = nullptr
		)
		: FRHITextureDesc(InDesc)
		, DebugName(InDebugName)
		, InitialState(InInitialState)
	{
		if (InBulkData)
		{
			SetInitActionBulkData(InBulkData);
		}
	}

	void CheckValidity() const
	{
		FRHITextureDesc::CheckValidity(*this, DebugName);

		ensureMsgf(InitialState != ERHIAccess::Unknown, TEXT("Resource %s cannot be created in an unknown state."), DebugName);
	}

	FRHITextureCreateDesc& SetFlags(ETextureCreateFlags InFlags)               { Flags = InFlags;                          return *this; }
	FRHITextureCreateDesc& AddFlags(ETextureCreateFlags InFlags)               { Flags |= InFlags;                         return *this; }
	FRHITextureCreateDesc& SetClearValue(FClearValueBinding InClearValue)      { ClearValue = InClearValue;                return *this; }
	FRHITextureCreateDesc& SetExtData(uint32 InExtData)                        { ExtData = InExtData;                      return *this; }
	FRHITextureCreateDesc& SetExtent(const FIntPoint& InExtent)                { Extent = InExtent;                        return *this; }
	FRHITextureCreateDesc& SetExtent(int32 InExtentX, int32 InExtentY)         { Extent = FIntPoint(InExtentX, InExtentY); return *this; }
	FRHITextureCreateDesc& SetExtent(uint32 InExtent)                          { Extent = FIntPoint(InExtent);             return *this; }
	FRHITextureCreateDesc& SetDepth(uint16 InDepth)                            { Depth = InDepth;                          return *this; }
	FRHITextureCreateDesc& SetArraySize(uint16 InArraySize)                    { ArraySize = InArraySize;                  return *this; }
	FRHITextureCreateDesc& SetNumMips(uint8 InNumMips)                         { NumMips = InNumMips;                      return *this; }
	FRHITextureCreateDesc& SetNumSamples(uint8 InNumSamples)                   { NumSamples = InNumSamples;                return *this; }
	FRHITextureCreateDesc& SetDimension(ETextureDimension InDimension)         { Dimension = InDimension;                  return *this; }
	FRHITextureCreateDesc& SetFormat(EPixelFormat InFormat)                    { Format = InFormat;                        return *this; }
	FRHITextureCreateDesc& SetUAVFormat(EPixelFormat InUAVFormat)              { UAVFormat = InUAVFormat;                  return *this; }
	FRHITextureCreateDesc& SetDebugName(const TCHAR* InDebugName)              { DebugName = InDebugName;                  return *this; }
	FRHITextureCreateDesc& SetInitialState(ERHIAccess InInitialState)          { InitialState = InInitialState;            return *this; }
	FRHITextureCreateDesc& SetInitAction(ERHITextureInitAction InInitAction)   { InitAction = InInitAction;                return *this; }
	FRHITextureCreateDesc& SetGPUMask(FRHIGPUMask InGPUMask)                   { GPUMask = InGPUMask;                      return *this; }
	FRHITextureCreateDesc& SetClassName(FName InClassName)			           { ClassName = InClassName;				   return *this; }
	FRHITextureCreateDesc& SetOwnerName(FName InOwnerName)                     { OwnerName = InOwnerName;                  return *this; }
	FRHITextureCreateDesc& AddAliasbleFormat(EPixelFormat InFormat)            { AliasableFormats.Add(InFormat);           return *this; }

	FRHITextureCreateDesc& SetFastVRAMPercentage(float InFastVRAMPercentage)
	{
		FastVRAMPercentage = uint8(FMath::Clamp(InFastVRAMPercentage, 0.f, 1.0f) * 0xFF);
		return *this;
	}

	FRHITextureCreateDesc& DetermineInititialState()
	{
		if (InitialState == ERHIAccess::Unknown)
		{
			InitialState = RHIGetDefaultResourceState(Flags, BulkData != nullptr);
		}
		return *this;
	}

	FRHITextureCreateDesc& SetInitActionNone()
	{
		return SetInitAction(ERHITextureInitAction::Default);
	}
	FRHITextureCreateDesc& SetInitActionInitializer()
	{
		return SetInitAction(ERHITextureInitAction::Initializer);
	}
	FRHITextureCreateDesc& SetInitActionBulkData(FResourceBulkDataInterface* InBulkData)
	{
		BulkData = InBulkData;
		return SetInitAction(ERHITextureInitAction::BulkData);
	}

	FName GetTraceClassName() const
	{
		const static FLazyName FRHITextureName(TEXT("FRHITexture"));
		return (ClassName == NAME_None) ? FRHITextureName : ClassName;
	}

	/* A friendly name for the resource. */
	const TCHAR* DebugName = nullptr;

	/* The RHI access state that the resource will be created in. */
	ERHIAccess InitialState = ERHIAccess::Unknown;

	/* Tells how to initialize (or not) the texture's data. */
	ERHITextureInitAction InitAction = ERHITextureInitAction::Default;

	/* Optional initial data to fill the resource with. */
	FResourceBulkDataInterface* BulkData = nullptr;

	/* The owner class of FRHITexture used for Insight asset metadata tracing */
	FName ClassName = NAME_None;

	/* The owner name used for Insight asset metadata tracing */
	FName OwnerName = NAME_None;
};

class FRHITexture : public FRHIViewableResource
#if ENABLE_RHI_VALIDATION
	, public RHIValidation::FTextureResource
#endif
{
protected:
	FRHITexture() = delete;

	/** Initialization constructor. Should only be called by platform RHI implementations. */
	RHI_API FRHITexture(const FRHITextureCreateDesc& CreateDesc);

	/** Constructor for texture references */
	RHI_API FRHITexture(ERHIResourceType InResourceType);

public:
	/**
	 * Get the texture description used to create the texture
	 * Still virtual because FRHITextureReference can override this function - remove virtual when FRHITextureReference is deprecated
	 *
	 * @return TextureDesc used to create the texture
	 */
	virtual const FRHITextureDesc& GetDesc() const { return TextureDesc; }
	
	///
	/// Virtual functions implemented per RHI
	/// 
	
	virtual class FRHITextureReference* GetTextureReference() { return NULL; }
	virtual FRHIDescriptorHandle GetDefaultBindlessHandle() const { return FRHIDescriptorHandle(); }

	/**
	 * Returns access to the platform-specific native resource pointer.  This is designed to be used to provide plugins with access
	 * to the underlying resource and should be used very carefully or not at all.
	 *
	 * @return	The pointer to the native resource or NULL if it not initialized or not supported for this resource type for some reason
	 */
	virtual void* GetNativeResource() const
	{
		// Override this in derived classes to expose access to the native texture resource
		return nullptr;
	}

	/**
	 * Returns access to the platform-specific native shader resource view pointer.  This is designed to be used to provide plugins with access
	 * to the underlying resource and should be used very carefully or not at all.
	 *
	 * @return	The pointer to the native resource or NULL if it not initialized or not supported for this resource type for some reason
	 */
	virtual void* GetNativeShaderResourceView() const
	{
		// Override this in derived classes to expose access to the native texture resource
		return nullptr;
	}

	/**
	 * Returns access to the platform-specific RHI texture baseclass.  This is designed to provide the RHI with fast access to its base classes in the face of multiple inheritance.
	 * @return	The pointer to the platform-specific RHI texture baseclass or NULL if it not initialized or not supported for this RHI
	 */
	virtual void* GetTextureBaseRHI()
	{
		// Override this in derived classes to expose access to the native texture resource
		return nullptr;
	}

	virtual void GetWriteMaskProperties(void*& OutData, uint32& OutSize)
	{
		OutData = nullptr;
		OutSize = 0;
	}

	///
	/// Helper getter functions - non virtual
	/// 

	/**
	 * Returns the x, y & z dimensions if the texture
	 * The Z component will always be 1 for 2D/cube resources and will contain depth for volume textures & array size for array textures
	 */
	FIntVector GetSizeXYZ() const
	{
		const FRHITextureDesc& Desc = GetDesc();
		switch (Desc.Dimension)
		{
		case ETextureDimension::Texture2D:		  return FIntVector(Desc.Extent.X, Desc.Extent.Y, 1);
		case ETextureDimension::Texture2DArray:	  return FIntVector(Desc.Extent.X, Desc.Extent.Y, Desc.ArraySize);
		case ETextureDimension::Texture3D:		  return FIntVector(Desc.Extent.X, Desc.Extent.Y, Desc.Depth);
		case ETextureDimension::TextureCube:	  return FIntVector(Desc.Extent.X, Desc.Extent.Y, 1);
		case ETextureDimension::TextureCubeArray: return FIntVector(Desc.Extent.X, Desc.Extent.Y, Desc.ArraySize);
		}
		return FIntVector(0, 0, 0);
	}

	/**
	 * Returns the dimensions (i.e. the actual number of texels in each dimension) of the specified mip. ArraySize is ignored.
	 * The Z component will always be 1 for 2D/cube resources and will contain depth for volume textures.
	 * This differs from GetSizeXYZ() which returns ArraySize in Z for 2D arrays.
	 */
	FIntVector GetMipDimensions(uint8 MipIndex) const
	{
		const FRHITextureDesc& Desc = GetDesc();
		return FIntVector(
			FMath::Max<int32>(Desc.Extent.X >> MipIndex, 1),
			FMath::Max<int32>(Desc.Extent.Y >> MipIndex, 1),
			FMath::Max<int32>(Desc.Depth    >> MipIndex, 1)
		);
	}

	/** @return Whether the texture is multi sampled. */
	bool IsMultisampled() const { return GetDesc().NumSamples > 1; }

	/** @return Whether the texture has a clear color defined */
	bool HasClearValue() const
	{
		return GetDesc().ClearValue.ColorBinding != EClearBinding::ENoneBound;
	}

	/** @return the clear color value if set */
	FLinearColor GetClearColor() const
	{
		return GetDesc().ClearValue.GetClearColor();
	}

	/** @return the depth & stencil clear value if set */
	void GetDepthStencilClearValue(float& OutDepth, uint32& OutStencil) const
	{
		return GetDesc().ClearValue.GetDepthStencil(OutDepth, OutStencil);
	}

	/** @return the depth clear value if set */
	float GetDepthClearValue() const
	{
		float Depth;
		uint32 Stencil;
		GetDesc().ClearValue.GetDepthStencil(Depth, Stencil);
		return Depth;
	}

	/** @return the stencil clear value if set */
	uint32 GetStencilClearValue() const
	{
		float Depth;
		uint32 Stencil;
		GetDesc().ClearValue.GetDepthStencil(Depth, Stencil);
		return Stencil;
	}

	///
	/// RenderTime & Name functions - non virtual
	/// 

	/** sets the last time this texture was cached in a resource table. */
	void SetLastRenderTime(double InLastRenderTime)
	{
		LastRenderTime.store(InLastRenderTime, std::memory_order_relaxed);
	}

	double GetLastRenderTime() const
	{
		return LastRenderTime.load(std::memory_order_relaxed);
	}

	RHI_API void SetName(FName InName);

	///
	/// Deprecated functions
	/// 

	//UE_DEPRECATED(5.1, "FRHITexture2D is deprecated, please use FRHITexture directly")
	inline FRHITexture* GetTexture2D() { return TextureDesc.Dimension == ETextureDimension::Texture2D ? this : nullptr; }
	//UE_DEPRECATED(5.1, "FRHITexture2DArray is deprecated, please use FRHITexture directly")
	inline FRHITexture* GetTexture2DArray() { return TextureDesc.Dimension == ETextureDimension::Texture2DArray ? this : nullptr; }
	//UE_DEPRECATED(5.1, "FRHITexture3D is deprecated, please use FRHITexture directly")
	inline FRHITexture* GetTexture3D() { return TextureDesc.Dimension == ETextureDimension::Texture3D ? this : nullptr; }
	//UE_DEPRECATED(5.1, "FRHITextureCube is deprecated, please use FRHITexture directly")
	inline FRHITexture* GetTextureCube() { return TextureDesc.IsTextureCube() ? this : nullptr; }

	//UE_DEPRECATED(5.1, "GetSizeX() is deprecated, please use GetDesc().Extent.X instead")
	uint32 GetSizeX() const { return GetDesc().Extent.X; }

	//UE_DEPRECATED(5.1, "GetSizeY() is deprecated, please use GetDesc().Extent.Y instead")
	uint32 GetSizeY() const { return GetDesc().Extent.Y; }

	//UE_DEPRECATED(5.1, "GetSizeXY() is deprecated, please use GetDesc().Extent.X or GetDesc().Extent.Y instead")
	FIntPoint GetSizeXY() const { return FIntPoint(GetDesc().Extent.X, GetDesc().Extent.Y); }

	//UE_DEPRECATED(5.1, "GetSizeZ() is deprecated, please use GetDesc().ArraySize instead for TextureArrays and GetDesc().Depth for 3D textures")
	uint32 GetSizeZ() const { return GetSizeXYZ().Z; }

	//UE_DEPRECATED(5.1, "GetNumMips() is deprecated, please use GetDesc().NumMips instead")
	uint32 GetNumMips() const { return GetDesc().NumMips; }

	//UE_DEPRECATED(5.1, "GetFormat() is deprecated, please use GetDesc().Format instead")
	EPixelFormat GetFormat() const { return GetDesc().Format; }

	//UE_DEPRECATED(5.1, "GetFlags() is deprecated, please use GetDesc().Flags instead")
	ETextureCreateFlags GetFlags() const { return GetDesc().Flags; }

	//UE_DEPRECATED(5.1, "GetNumSamples() is deprecated, please use GetDesc().NumSamples instead")
	uint32 GetNumSamples() const { return GetDesc().NumSamples; }

	//UE_DEPRECATED(5.1, "GetClearBinding() is deprecated, please use GetDesc().ClearValue instead")
	const FClearValueBinding GetClearBinding() const { return GetDesc().ClearValue; }

	//UE_DEPRECATED(5.1, "GetSize() is deprecated, please use GetDesc().Extent.X instead")
	uint32 GetSize() const { check(GetDesc().IsTextureCube()); return GetDesc().Extent.X; }

#if ENABLE_RHI_VALIDATION
	virtual RHIValidation::FResource* GetValidationTrackerResource() override
	{
		// Use the method inherited from RHIValidation::FTextureResource, as that's already a virtual overridden
		// by subclasses such as FRHITextureReference to return the correct storage for the tracker information.
		return GetTrackerResource();
	}
#endif

	static constexpr double EmptyLastRenderTime = -FLT_MAX;

private:
	friend class FRHITextureReference;

	FRHITextureDesc TextureDesc;

	/** The last time the resource was rendered. */
	std::atomic<double> LastRenderTime = EmptyLastRenderTime;
};

//
// Misc
//

class FRHIGPUFence : public FRHIResource
{
public:
	FRHIGPUFence(FName InName)
		: FRHIResource(RRT_GPUFence)
		, FenceName(InName)
	{}

	virtual void Clear() = 0;

	/**
	 * Poll the fence to see if the GPU has signaled it.
	 * @returns True if and only if the GPU fence has been inserted and the GPU has signaled the fence.
	 * 
	 * CAUTION: Do not call poll in a loop to block until completion. Some platform RHIs will not signal
	 *          fences unless the RHI thread continues to make progress and submit GPU work.
	 *
	 *          To block for completion, call Wait(), which can only be done from the render thread.
	 *
	 *          Use Poll() only to check a fence. If it returns false, continue to do useful work and recheck
	 *          later, ideally on the following frame.
	 */
	virtual bool Poll() const = 0;

	/**
	 * Poll on a subset of the GPUs that this fence supports.
	 * 
	 * CAUTION: Do not call poll in a loop to block until completion. Some platform RHIs will not signal
	 *          fences unless the RHI thread continues to make progress and submit GPU work.
	 *
	 *          To block for completion, call Wait(), which can only be done from the render thread.
	 *
	 *          Use Poll() only to check a fence. If it returns false, continue to do useful work and recheck
	 *          later, ideally on the following frame.
	 */
	virtual bool Poll(FRHIGPUMask GPUMask) const
	{
		checkf(GPUMask == FRHIGPUMask::GPU0(), TEXT("The current platform RHI does not implement MGPU support for RHI GPU fences."));
		return Poll();
	}

	/**
	 * Blocks the caller until the fence has passed on the GPU. Can only be called from the render thread.
	 *
	 * CAUTION: Use of this function may cause a hitch if the fence is not yet signalled and we have to wait for
	 *          the GPU to catch up. Rendering code should be written to avoid immediately waiting on GPU fences.
	 */
	virtual void Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const = 0;

	FName GetFName() const { return FenceName; }

	FThreadSafeCounter NumPendingWriteCommands;

protected:
	const FName FenceName;
};

class FRHIRenderQuery : public FRHIResource
{
public:
	FRHIRenderQuery() : FRHIResource(RRT_RenderQuery) {}
};

class FRHIRenderQueryPool;

class FRHIPooledRenderQuery
{
	TRefCountPtr<FRHIRenderQuery> Query;
	FRHIRenderQueryPool* QueryPool = nullptr;

public:
	FRHIPooledRenderQuery() = default;
	FRHIPooledRenderQuery(FRHIRenderQueryPool* InQueryPool, TRefCountPtr<FRHIRenderQuery>&& InQuery);
	~FRHIPooledRenderQuery();

	FRHIPooledRenderQuery(const FRHIPooledRenderQuery&) = delete;
	FRHIPooledRenderQuery& operator=(const FRHIPooledRenderQuery&) = delete;
	FRHIPooledRenderQuery(FRHIPooledRenderQuery&&) = default;
	FRHIPooledRenderQuery& operator=(FRHIPooledRenderQuery&&) = default;

	bool IsValid() const
	{
		return Query.IsValid();
	}

	FRHIRenderQuery* GetQuery() const
	{
		return Query;
	}

	void ReleaseQuery();
};

class FRHIRenderQueryPool : public FRHIResource
{
public:
	FRHIRenderQueryPool() : FRHIResource(RRT_RenderQueryPool) {}
	virtual ~FRHIRenderQueryPool() = default;
	virtual FRHIPooledRenderQuery AllocateQuery() = 0;

private:
	friend class FRHIPooledRenderQuery;
	virtual void ReleaseQuery(TRefCountPtr<FRHIRenderQuery>&& Query) = 0;
};

inline FRHIPooledRenderQuery::FRHIPooledRenderQuery(FRHIRenderQueryPool* InQueryPool, TRefCountPtr<FRHIRenderQuery>&& InQuery) 
	: Query(MoveTemp(InQuery))
	, QueryPool(InQueryPool)
{
	check(IsInParallelRenderingThread());
}

inline void FRHIPooledRenderQuery::ReleaseQuery()
{
	if (QueryPool && Query.IsValid())
	{
		QueryPool->ReleaseQuery(MoveTemp(Query));
		QueryPool = nullptr;
	}
	check(!Query.IsValid());
}

inline FRHIPooledRenderQuery::~FRHIPooledRenderQuery()
{
	check(IsInParallelRenderingThread());
	ReleaseQuery();
}

struct FRHIPresentArgs
{
	FRHIPresentArgs(uint64 InFrameCounter, bool bInPresent, bool bInLockToVsync):
		FrameCounter(InFrameCounter),
		bPresent((uint8)bInPresent),
		bLockToVsync((uint8)bInLockToVsync)
	{ }


	/** Frame index that is going to Presented. */
	uint64 FrameCounter;

	/** true if the Engine should call Present the frame to the display */
	uint8 bPresent:1;

	/** true if the Engine should sync the frame presentation to the display vsync*/
	uint8 bLockToVsync:1;
};

class FRHIViewport : public FRHIResource 
{
public:
	FRHIViewport() : FRHIResource(RRT_Viewport) {}

	/**
	 * Returns access to the platform-specific native resource pointer.  This is designed to be used to provide plugins with access
	 * to the underlying resource and should be used very carefully or not at all.
	 *
	 * @return	The pointer to the native resource or NULL if it not initialized or not supported for this resource type for some reason
	 */
	virtual void* GetNativeSwapChain() const { return nullptr; }
	/**
	 * Returns access to the platform-specific native resource pointer to a backbuffer texture.  This is designed to be used to provide plugins with access
	 * to the underlying resource and should be used very carefully or not at all.
	 *
	 * @return	The pointer to the native resource or NULL if it not initialized or not supported for this resource type for some reason
	 */
	virtual void* GetNativeBackBufferTexture() const { return nullptr; }
	/**
	 * Returns access to the platform-specific native resource pointer to a backbuffer rendertarget. This is designed to be used to provide plugins with access
	 * to the underlying resource and should be used very carefully or not at all.
	 *
	 * @return	The pointer to the native resource or NULL if it not initialized or not supported for this resource type for some reason
	 */
	virtual void* GetNativeBackBufferRT() const { return nullptr; }

	/**
	 * Returns access to the platform-specific native window. This is designed to be used to provide plugins with access
	 * to the underlying resource and should be used very carefully or not at all. 
	 *
	 * @return	The pointer to the native resource or NULL if it not initialized or not supported for this resource type for some reason.
	 * AddParam could represent any additional platform-specific data (could be null).
	 */
	virtual void* GetNativeWindow(void** AddParam = nullptr) const { return nullptr; }

	/**
	 * Sets custom Present handler on the viewport
	 */
	virtual void SetCustomPresent(class FRHICustomPresent*) {}
	
	/**
	 * Creates a platform-level native slate rendering layer for handling rendering order with platform-native views.
	 * This means that everything on top of the specified layer will be rendered to a different platform-level view than 
	 * everything on the bottom of the specified layer.
	 * Right now this is only implemented in MetalRHI for macOS NSView interop with Slate.
	 */
	virtual void CreateNativeLayer(int32 NewNativeLayer, void* NativeViewHandle, const TArray<TInterval<int32>>& LayerRanges) {}
	
	/**
	 * Deletes a platform-level native layer created by CreateNativeLayer.
	 * Right now this is only implemented in MetalRHI for macOS NSView interop with Slate.
	 */
	virtual void DeleteNativeLayer(int32 OldNativeLayer) {}
	
	/**
	 * Sets the default native layer for items to render into. If the WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	 * compile flag is false this function will have no effect. 
	 */
	virtual void SetDefaultNativeLayer(int32 NativeLayer) {}

	/**
	* Some platforms can specify a Crop rectangle for the Viewport
	*/
	virtual void SetWindowCropSize( const FIntRect& NewCropWindow ) {}

	/**
	 * Returns currently set custom present handler.
	 */
	virtual class FRHICustomPresent* GetCustomPresent() const { return nullptr; }

	virtual FRHITexture* GetOptionalSDRBackBuffer(FRHITexture* BackBuffer) const { return nullptr; }

	/**
	 * Ticks the viewport on the Game thread
	 */
	 UE_DEPRECATED(5.8, "FRHIViewport::Tick is deprecated and unused.")
	virtual void Tick(float DeltaTime) {}

	virtual void WaitForFrameEventCompletion() { }

	virtual void IssueFrameEvent() { }

	void SetPresentLatencyCallbacks(TFunction<void()>&& InBeginCallback, TFunction<void()>&& InEndCallback)
	{
		PresentBeginLatencyCallback = MoveTemp(InBeginCallback);
		PresentEndLatencyCallback = MoveTemp(InEndCallback);
	}

protected:
	TFunction<void()> PresentBeginLatencyCallback;
	TFunction<void()> PresentEndLatencyCallback;
};

/** Used to specify a texture metadata plane when creating a view. */
enum class ERHITexturePlane : uint8
{
	// The primary plane is used with default compression behavior.
	Primary = 0,

	// The primary plane is used without decompressing it.
	PrimaryCompressed = 1,

	// The depth plane is used with default compression behavior.
	Depth = 2,

	// The stencil plane is used with default compression behavior.
	Stencil = 3,

	// The HTile plane is used.
	HTile = 4,

	// the FMask plane is used.
	FMask = 5,

	// the CMask plane is used.
	CMask = 6,

	// This enum is packed into various structures. Avoid adding new 
	// members without verifying structure sizes aren't increased.
	Num,
	NumBits = 3,

	// @todo deprecate
	None = Primary,
	CompressedSurface = PrimaryCompressed,
};
static_assert((1u << uint32(ERHITexturePlane::NumBits)) >= uint32(ERHITexturePlane::Num), "Not enough bits in the ERHITexturePlane enum");

//UE_DEPRECATED(5.3, "Use ERHITexturePlane.")
using ERHITextureMetaDataAccess = ERHITexturePlane;

//
// Views
//

template <typename TType>
struct TRHIRange
{
	TType First = 0;
	TType Num = 0;

	TRHIRange() = default;
	TRHIRange(uint32 InFirst, uint32 InNum)
		: First(InFirst)
		, Num  (InNum)
	{
		check( InFirst < TNumericLimits<TType>::Max()
			&& InNum   < TNumericLimits<TType>::Max()
			&& (InFirst + InNum) < TNumericLimits<TType>::Max());
	}

	TType ExclusiveLast() const { return First + Num; }
	TType InclusiveLast() const { return First + Num - 1; }

	bool IsInRange(uint32 Value) const
	{
		check(Value < TNumericLimits<TType>::Max());
		return Value >= First && Value < ExclusiveLast();
	}
};

using FRHIRange8  = TRHIRange<uint8>;
using FRHIRange16 = TRHIRange<uint16>;

//
// The unified RHI view descriptor. These are stored in the base FRHIView type, and packed to minimize memory usage.
// Platform RHI implementations use the GetViewInfo() functions to convert an FRHIViewDesc into the required info to make a view / descriptor for the GPU.
//
struct FRHIViewDesc
{
	enum class EViewType : uint8
	{
		BufferSRV,
		BufferUAV,
		TextureSRV,
		TextureUAV
	};

	enum class EBufferType : uint8
	{
		Unknown               = 0,

		Typed                 = 1,
		Structured            = 2,
		AccelerationStructure = 3,
		Raw                   = 4
	};

	enum class EDimension : uint8
	{
		Unknown          = 0,

		Texture2D        = 1,
		Texture2DArray   = 2,
		TextureCube      = 3,
		TextureCubeArray = 4,
		Texture3D        = 5,

		NumBits          = 3
	};

	// Properties that apply to all views.
	struct FCommon
	{
		EViewType    ViewType;
		EPixelFormat Format;
	};

	// Properties shared by buffer views. Some fields are SRV or UAV specific.
	struct FBuffer : public FCommon
	{
		EBufferType BufferType;
		uint8       bAtomicCounter : 1; // UAV only
		uint8       bAppendBuffer  : 1; // UAV only
		uint8       /* padding */  : 6;
		uint32      OffsetInBytes;
		union
		{
			struct
			{
				uint32 NumElements;
				uint32 Stride;
			};
			FRHIRayTracingScene* RayTracingScene; // only if BufferType == AccelerationStructure
		};

		struct FViewInfo;
	protected:
		FViewInfo GetViewInfo(FRHIBuffer* TargetBuffer) const;
	};

	// Properties shared by texture views. Some fields are SRV or UAV specific.
	struct FTexture : public FCommon
	{
		ERHITexturePlane Plane        : uint32(ERHITexturePlane::NumBits);
		uint8            bDisableSRGB : 1; // SRV only
		EDimension       Dimension    : uint32(EDimension::NumBits);
		FRHIRange8       MipRange;    // UAVs only support 1 mip
		FRHIRange16      ArrayRange;

		struct FViewInfo;
	protected:
		FViewInfo GetViewInfo(FRHITexture* TargetTexture) const;
	};

	struct FBufferSRV : public FBuffer
	{
		struct FInitializer;
		struct FViewInfo;
		RHI_API FViewInfo GetViewInfo(FRHIBuffer* TargetBuffer) const;
	};

	struct FBufferUAV : public FBuffer
	{
		struct FInitializer;
		struct FViewInfo;
		RHI_API FViewInfo GetViewInfo(FRHIBuffer* TargetBuffer) const;
	};

	struct FTextureSRV : public FTexture
	{
		struct FInitializer;
		struct FViewInfo;
		RHI_API FViewInfo GetViewInfo(FRHITexture* TargetTexture) const;
	};

	struct FTextureUAV : public FTexture
	{
		struct FInitializer;
		struct FViewInfo;
		RHI_API FViewInfo GetViewInfo(FRHITexture* TargetTexture) const;
	};

	union
	{
		FCommon Common;
		union
		{
			FBufferSRV SRV;
			FBufferUAV UAV;
		} Buffer;
		union
		{
			FTextureSRV SRV;
			FTextureUAV UAV;
		} Texture;
	};

	static inline FBufferSRV::FInitializer CreateBufferSRV();
	static inline FBufferUAV::FInitializer CreateBufferUAV();

	static inline FTextureSRV::FInitializer CreateTextureSRV();
	static inline FTextureUAV::FInitializer CreateTextureUAV();

	bool IsSRV() const { return Common.ViewType == EViewType::BufferSRV || Common.ViewType == EViewType::TextureSRV; }
	bool IsUAV() const { return !IsSRV(); }

	bool IsBuffer () const { return Common.ViewType == EViewType::BufferSRV || Common.ViewType == EViewType::BufferUAV; }
	bool IsTexture() const { return !IsBuffer(); }

	bool operator == (FRHIViewDesc const& RHS) const
	{
		return FMemory::Memcmp(this, &RHS, sizeof(*this)) == 0;
	}

	bool operator != (FRHIViewDesc const& RHS) const
	{
		return !(*this == RHS);
	}

	FRHIViewDesc()
		: FRHIViewDesc(EViewType::BufferSRV)
	{
		FMemory::Memzero(*this);
	}

	static const TCHAR* GetBufferTypeString(EBufferType BufferType);
	static const TCHAR* GetTextureDimensionString(EDimension Dimension);

protected:
	FRHIViewDesc(EViewType ViewType)
	{
		FMemory::Memzero(*this);
		Common.ViewType = ViewType;
	}
};

// These static asserts are to ensure the descriptor is minimal in size and can be copied around by-value.
// If they fail, consider re-packing the struct.
static_assert(sizeof(FRHIViewDesc) == 16, "Packing of FRHIViewDesc is unexpected.");
static_assert(TIsTrivial<FRHIViewDesc>::Value, "FRHIViewDesc must be a trivial type.");

struct FRHIViewDesc::FBufferSRV::FInitializer : private FRHIViewDesc
{
	friend FRHIViewDesc;
	friend FRHICommandListBase;
	friend struct FShaderResourceViewInitializer;
	friend struct FRawBufferShaderResourceViewInitializer;

protected:
	FInitializer()
		: FRHIViewDesc(EViewType::BufferSRV)
	{}

public:
	FInitializer& SetType(EBufferType Type)
	{
		check(Type != EBufferType::Unknown);
		Buffer.SRV.BufferType = Type;
		return *this;
	}

	//
	// Provided for back-compat with existing code. Consider using SetType() instead for more direct control over the view.
	// For example, it is possible to create a typed view of a BUF_ByteAddress buffer, but not using this function which always choses raw access.
	//
	FInitializer& SetTypeFromBuffer(FRHIBuffer* TargetBuffer)
	{
		check(TargetBuffer);
		checkf(!TargetBuffer->GetDesc().IsNull(), TEXT("Null buffer resources are placeholders for the streaming system. They do not contain a valid descriptor for this function to use. Call SetType() instead."));

		Buffer.SRV.BufferType =
			EnumHasAnyFlags(TargetBuffer->GetUsage(), BUF_ByteAddressBuffer    ) ? EBufferType::Raw                   :
			EnumHasAnyFlags(TargetBuffer->GetUsage(), BUF_StructuredBuffer     ) ? EBufferType::Structured            :
			EnumHasAnyFlags(TargetBuffer->GetUsage(), BUF_AccelerationStructure) ? EBufferType::AccelerationStructure :
			EBufferType::Typed;
		return *this;
	}

	FInitializer& SetFormat(EPixelFormat InFormat)
	{
		Buffer.SRV.Format = InFormat;
		return *this;
	}

	FInitializer& SetOffsetInBytes(uint32 InOffsetBytes)
	{
		Buffer.SRV.OffsetInBytes = InOffsetBytes;
		return *this;
	}

	FInitializer& SetStride(uint32 InStride)
	{
		check(Buffer.SRV.BufferType != EBufferType::Unknown && Buffer.SRV.BufferType != EBufferType::AccelerationStructure);
		Buffer.SRV.Stride = InStride;
		return *this;
	}

	FInitializer& SetNumElements(uint32 InNumElements)
	{
		check(Buffer.SRV.BufferType != EBufferType::Unknown && Buffer.SRV.BufferType != EBufferType::AccelerationStructure);
		Buffer.SRV.NumElements = InNumElements;
		return *this;
	}

	FInitializer& SetRayTracingScene(FRHIRayTracingScene* InRayTracingScene)
	{
		check(Buffer.SRV.BufferType == EBufferType::AccelerationStructure);
		Buffer.SRV.RayTracingScene = InRayTracingScene;
		return *this;
	}
};

struct FRHIViewDesc::FBufferUAV::FInitializer : private FRHIViewDesc
{
	friend FRHIViewDesc;
	friend FRHICommandListBase;

protected:
	FInitializer()
		: FRHIViewDesc(EViewType::BufferUAV)
	{}

public:
	FInitializer& SetType(EBufferType Type)
	{
		check(Type != EBufferType::Unknown);
		Buffer.UAV.BufferType = Type;
		return *this;
	}

	//
	// Provided for back-compat with existing code. Consider using SetType() instead for more direct control over the view.
	// For example, it is possible to create a typed view of a BUF_ByteAddress buffer, but not using this function which always choses raw access.
	//
	FInitializer& SetTypeFromBuffer(FRHIBuffer* TargetBuffer)
	{
		check(TargetBuffer);
		checkf(!TargetBuffer->GetDesc().IsNull(), TEXT("Null buffer resources are placeholders for the streaming system. They do not contain a valid descriptor for this function to use. Call SetType() instead."));

		Buffer.UAV.BufferType =
			EnumHasAnyFlags(TargetBuffer->GetUsage(), BUF_ByteAddressBuffer    ) ? EBufferType::Raw                   :
			EnumHasAnyFlags(TargetBuffer->GetUsage(), BUF_StructuredBuffer     ) ? EBufferType::Structured            :
			EnumHasAnyFlags(TargetBuffer->GetUsage(), BUF_AccelerationStructure) ? EBufferType::AccelerationStructure :
			EBufferType::Typed;
		return *this;
	}

	FInitializer& SetFormat(EPixelFormat InFormat)
	{
		Buffer.UAV.Format = InFormat;
		return *this;
	}

	FInitializer& SetOffsetInBytes(uint32 InOffsetBytes)
	{
		Buffer.UAV.OffsetInBytes = InOffsetBytes;
		return *this;
	}

	FInitializer& SetStride(uint32 InStride)
	{
		Buffer.UAV.Stride = InStride;
		return *this;
	}

	FInitializer& SetNumElements(uint32 InNumElements)
	{
		Buffer.UAV.NumElements = InNumElements;
		return *this;
	}

	FInitializer& SetAtomicCounter(bool InAtomicCounter)
	{
		Buffer.UAV.bAtomicCounter = InAtomicCounter;
		return *this;
	}

	FInitializer& SetAppendBuffer(bool InAppendBuffer)
	{
		Buffer.UAV.bAppendBuffer = InAppendBuffer;
		return *this;
	}
};

struct FRHIViewDesc::FTextureSRV::FInitializer : private FRHIViewDesc
{
	friend FRHIViewDesc;
	friend FRHICommandListBase;

protected:
	FInitializer()
		: FRHIViewDesc(EViewType::TextureSRV)
	{}

public:
	//
	// Specifies the type of view to create. Must match the shader parameter this view will be bound to.
	// 
	// The dimension is allowed to differ from the underlying resource's dimensions, e.g. to create a view
	// compatible with a Texture2D<> shader parameter, but where the underlying resource is a texture 2D array.
	//
	// Some combinations are not valid, e.g. 3D textures can only have 3D views.
	//
	FInitializer& SetDimension(ETextureDimension InDimension)
	{
		switch (InDimension)
		{
		default: checkNoEntry(); break;
		case ETextureDimension::Texture2D       : Texture.SRV.Dimension = EDimension::Texture2D       ; break;
		case ETextureDimension::Texture2DArray  : Texture.SRV.Dimension = EDimension::Texture2DArray  ; break;
		case ETextureDimension::Texture3D       : Texture.SRV.Dimension = EDimension::Texture3D       ; break;
		case ETextureDimension::TextureCube     : Texture.SRV.Dimension = EDimension::TextureCube     ; break;
		case ETextureDimension::TextureCubeArray: Texture.SRV.Dimension = EDimension::TextureCubeArray; break;
		}
		return *this;
	}

	//
	// Provided for back-compat with existing code. Consider using SetDimension() instead for more direct control over the view.
	// For example, it is possible to create a 2D view of a 2DArray texture, but not using this function which always choses 2DArray dimension.
	//
	FInitializer& SetDimensionFromTexture(FRHITexture* TargetTexture)
	{
		check(TargetTexture);
		SetDimension(TargetTexture->GetDesc().Dimension);
		return *this;
	}

	FInitializer& SetFormat(EPixelFormat InFormat)
	{
		Texture.SRV.Format = InFormat;
		return *this;
	}

	FInitializer& SetPlane(ERHITexturePlane InPlane)
	{
		Texture.SRV.Plane = InPlane;
		return *this;
	}

	FInitializer& SetMipRange(uint8 InFirstMip, uint8 InNumMips)
	{
		Texture.SRV.MipRange.First = InFirstMip;
		Texture.SRV.MipRange.Num = InNumMips;
		return *this;
	}

	//
	// The meaning of array "elements" is given by the dimension of the underlying resource.
	// I.e. a view of a TextureCubeArray resource indexes the array in whole cubes.
	// 
	//     [0] = the first cube (2D slices 0 to 5)
	//     [1] = the second cube (2D slices 6 to 11)
	// 
	// If the view dimension is smaller than the resource dimension, the array range will be further limited.
	// E.g. creating a Texture2D dimension view of a TextureCubeArray resource
	//
	FInitializer& SetArrayRange(uint16 InFirstElement, uint16 InNumElements)
	{
		Texture.SRV.ArrayRange.First = InFirstElement;
		Texture.SRV.ArrayRange.Num = InNumElements;
		return *this;
	}

	FInitializer& SetDisableSRGB(bool InDisableSRGB)
	{
		Texture.SRV.bDisableSRGB = InDisableSRGB;
		return *this;
	}
};

struct FRHIViewDesc::FTextureUAV::FInitializer : private FRHIViewDesc
{
	friend FRHIViewDesc;
	friend FRHICommandListBase;

protected:
	FInitializer()
		: FRHIViewDesc(EViewType::TextureUAV)
	{
		// Texture UAVs only support 1 mip
		Texture.UAV.MipRange.Num = 1;
	}

public:
	//
	// Specifies the type of view to create. Must match the shader parameter this view will be bound to.
	// 
	// The dimension is allowed to differ from the underlying resource's dimensions, e.g. to create a view
	// compatible with an RWTexture2D<> shader parameter, but where the underlying resource is a texture 2D array.
	//
	// Some combinations are not valid, e.g. 3D textures can only have 3D views.
	//
	FInitializer& SetDimension(ETextureDimension InDimension)
	{
		switch (InDimension)
		{
		default: checkNoEntry(); break;
		case ETextureDimension::Texture2D       : Texture.UAV.Dimension = EDimension::Texture2D       ; break;
		case ETextureDimension::Texture2DArray  : Texture.UAV.Dimension = EDimension::Texture2DArray  ; break;
		case ETextureDimension::Texture3D       : Texture.UAV.Dimension = EDimension::Texture3D       ; break;
		case ETextureDimension::TextureCube     : Texture.UAV.Dimension = EDimension::TextureCube     ; break;
		case ETextureDimension::TextureCubeArray: Texture.UAV.Dimension = EDimension::TextureCubeArray; break;
		}
		return *this;
	}

	//
	// Provided for back-compat with existing code. Consider using SetDimension() instead for more direct control over the view.
	// For example, it is possible to create a 2D view of a 2DArray texture, but not using this function which always choses 2DArray dimension.
	//
	FInitializer& SetDimensionFromTexture(FRHITexture* TargetTexture)
	{
		check(TargetTexture);
		SetDimension(TargetTexture->GetDesc().Dimension);
		return *this;
	}

	FInitializer& SetFormat(EPixelFormat InFormat)
	{
		Texture.UAV.Format = InFormat;
		return *this;
	}

	FInitializer& SetPlane(ERHITexturePlane InPlane)
	{
		Texture.UAV.Plane = InPlane;
		return *this;
	}

	FInitializer& SetMipLevel(uint8 InMipLevel)
	{
		Texture.UAV.MipRange.First = InMipLevel;
		return *this;
	}

	//
	// The meaning of array "elements" is given by the dimension of the underlying resource.
	// I.e. a view of a TextureCubeArray resource indexes the array in whole cubes.
	// 
	//     [0] = the first cube (2D slices 0 to 5)
	//     [1] = the second cube (2D slices 6 to 11)
	// 
	// If the view dimension is smaller than the resource dimension, the array range will be further limited.
	// E.g. creating a Texture2D dimension view of a TextureCubeArray resource
	//
	FInitializer& SetArrayRange(uint16 InFirstElement, uint16 InNumElements)
	{
		Texture.UAV.ArrayRange.First = InFirstElement;
		Texture.UAV.ArrayRange.Num = InNumElements;
		return *this;
	}
};

inline FRHIViewDesc::FBufferSRV::FInitializer FRHIViewDesc::CreateBufferSRV()
{
	return FRHIViewDesc::FBufferSRV::FInitializer();
}

inline FRHIViewDesc::FBufferUAV::FInitializer FRHIViewDesc::CreateBufferUAV()
{
	return FRHIViewDesc::FBufferUAV::FInitializer();
}

inline FRHIViewDesc::FTextureSRV::FInitializer FRHIViewDesc::CreateTextureSRV()
{
	return FRHIViewDesc::FTextureSRV::FInitializer();
}

inline FRHIViewDesc::FTextureUAV::FInitializer FRHIViewDesc::CreateTextureUAV()
{
	return FRHIViewDesc::FTextureUAV::FInitializer();
}

//
// Used by platform RHIs to create views of buffers. The data in this structure is computed in GetViewInfo(),
// and is specific to a particular buffer resource. It is not intended to be stored in a view instance.
//
struct FRHIViewDesc::FBuffer::FViewInfo
{
	// The offset in bytes from the beginning of the viewed buffer resource.
	uint32 OffsetInBytes;

	// The size in bytes of a single element in the view.
	uint32 StrideInBytes;

	// The number of elements visible in the view.
	uint32 NumElements;

	// The total number of bytes the data visible in the view covers (i.e. stride * numelements).
	uint32 SizeInBytes;

	// Whether this is a typed / structured / raw view etc.
	EBufferType BufferType;

	// The format of the data exposed by this view. PF_Unknown for all buffer types except typed buffer views.
	EPixelFormat Format;

	// When true, the view is referring to a BUF_NullResource, so a null descriptor should be created.
	bool bNullView;
};

// Buffer SRV specific info
struct FRHIViewDesc::FBufferSRV::FViewInfo : public FRHIViewDesc::FBuffer::FViewInfo
{};

// Buffer UAV specific info
struct FRHIViewDesc::FBufferUAV::FViewInfo : public FRHIViewDesc::FBuffer::FViewInfo
{
	bool bAtomicCounter = false;
	bool bAppendBuffer = false;
};

//
// Used by platform RHIs to create views of textures. The data in this structure is computed in GetViewInfo(),
// and is specific to a particular texture resource. It is not intended to be stored in a view instance.
//
struct FRHIViewDesc::FTexture::FViewInfo
{
	//
	// The range of array "elements" the view covers.
	// 
	// The meaning of "elements" is given by the view dimension.
	// I.e. a view with "Dimension == CubeArray" indexes the array in whole cubes.
	// 
	//		- [0]: the first cube (2D slices 0 to 5)
	//		- [1]: the second cube (2D slices 6 to 11)
	// 
	// 3D textures always have ArrayRange.Num == 1 because there are no "3D texture arrays".
	//
	FRHIRange16 ArrayRange;

	// Which plane of a texture to access (i.e. color, depth, stencil etc). 
	ERHITexturePlane Plane;

	// The typed format to use when reading / writing data in the viewed texture.
	EPixelFormat Format;

	// Specifies how to treat the texture resource when creating the view.
	// E.g. it is possible to create a 2DArray view of a 2D or Cube texture.
	EDimension Dimension : uint32(EDimension::NumBits);

	// True when the view covers every mip of the resource.
	uint8 bAllMips : 1;

	// True when the view covers every array slice of the resource.
	// This includes depth slices for 3D textures, and faces of texture cubes.
	uint8 bAllSlices : 1;
};

// Texture SRV specific info
struct FRHIViewDesc::FTextureSRV::FViewInfo : public FRHIViewDesc::FTexture::FViewInfo
{
	// The range of texture mips the view covers.
	FRHIRange8 MipRange;

	// Indicates if this view should use an sRGB variant of the typed format.
	uint8 bSRGB : 1;
};


// Texture UAV specific info
struct FRHIViewDesc::FTextureUAV::FViewInfo : public FRHIViewDesc::FTexture::FViewInfo
{
	// The single mip level covered by this view.
	uint8 MipLevel;
};

class FRHIView : public FRHIResource
{
public:
	FRHIView(ERHIResourceType InResourceType, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc)
		: FRHIResource(InResourceType)
		, Resource(InResource)
		, ViewDesc(InViewDesc)
	{
		checkf(InResource, TEXT("Cannot create a view of a nullptr resource."));
	}

	virtual FRHIDescriptorHandle GetBindlessHandle() const
	{
		return FRHIDescriptorHandle();
	}

	FRHIViewableResource* GetResource() const
	{
		return Resource;
	}

	FRHIBuffer* GetBuffer() const
	{
		check(ViewDesc.IsBuffer());
		return static_cast<FRHIBuffer*>(Resource.GetReference());
	}

	FRHITexture* GetTexture() const
	{
		check(ViewDesc.IsTexture());
		return static_cast<FRHITexture*>(Resource.GetReference());
	}

	bool IsBuffer () const { return ViewDesc.IsBuffer (); }
	bool IsTexture() const { return ViewDesc.IsTexture(); }

#if ENABLE_RHI_VALIDATION
	RHIValidation::FViewIdentity GetViewIdentity() const
	{
		return RHIValidation::FViewIdentity(Resource, ViewDesc);
	}
#endif

	FRHIViewDesc const& GetDesc() const
	{
		return ViewDesc;
	}

private:
	TRefCountPtr<FRHIViewableResource> Resource;

protected:
	FRHIViewDesc const ViewDesc;
};

class FRHIUnorderedAccessView : public FRHIView
{
public:
	explicit FRHIUnorderedAccessView(FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc)
		: FRHIView(RRT_UnorderedAccessView, InResource, InViewDesc)
	{
		check(ViewDesc.IsUAV());
	}
};

class FRHIShaderResourceView : public FRHIView
{
public:
	explicit FRHIShaderResourceView(FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc)
		: FRHIView(RRT_ShaderResourceView, InResource, InViewDesc)
	{
		check(ViewDesc.IsSRV());
	}
};

inline ERHIDescriptorType RHIDescriptorTypeFromViewType(FRHIViewDesc::EViewType ViewType)
{
	switch (ViewType)
	{
	default:
	case FRHIViewDesc::EViewType::BufferSRV:
		return ERHIDescriptorType::BufferSRV;

	case FRHIViewDesc::EViewType::BufferUAV:
		return ERHIDescriptorType::BufferUAV;

	case FRHIViewDesc::EViewType::TextureSRV:
		return ERHIDescriptorType::TextureSRV;

	case FRHIViewDesc::EViewType::TextureUAV:
		return ERHIDescriptorType::TextureUAV;
	}
}

inline ERHIDescriptorType RHIDescriptorTypeFromViewDesc(const FRHIViewDesc& InViewDesc)
{
	return RHIDescriptorTypeFromViewType(InViewDesc.Common.ViewType);
}

//
// Ray tracing resources
//

struct FRayTracingGeometryOfflineDataHeader
{
	uint32 Reserved[6] = { 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF };

	bool IsValid() const
	{
		return *this != FRayTracingGeometryOfflineDataHeader();
	}

	bool operator==(const FRayTracingGeometryOfflineDataHeader& Other) const
	{
		return Reserved[0] == Other.Reserved[0]
			&& Reserved[1] == Other.Reserved[1]
			&& Reserved[2] == Other.Reserved[2]
			&& Reserved[3] == Other.Reserved[3]
			&& Reserved[4] == Other.Reserved[4]
			&& Reserved[5] == Other.Reserved[5];
	}

	bool operator!=(const FRayTracingGeometryOfflineDataHeader& Other) const
	{
		return !(*this == Other);
	}

	friend FArchive& operator<<(FArchive& Ar, FRayTracingGeometryOfflineDataHeader& Header)
	{
		Ar << Header.Reserved[0];
		Ar << Header.Reserved[1];
		Ar << Header.Reserved[2];
		Ar << Header.Reserved[3];
		Ar << Header.Reserved[4];
		Ar << Header.Reserved[5];
		return Ar;
	}
};

enum ERayTracingGeometryType
{
	// Indexed or non-indexed triangle list with fixed function ray intersection.
	// Vertex buffer must contain vertex positions as VET_Float3.
	// Vertex stride must be at least 12 bytes, but may be larger to support custom per-vertex data.
	// Index buffer may be provided for indexed triangle lists. Implicit triangle list is assumed otherwise.
	RTGT_Triangles,

	// Custom primitive type that requires an intersection shader.
	// Vertex buffer for procedural geometry must contain one AABB per primitive as {float3 MinXYZ, float3 MaxXYZ}.
	// Vertex stride must be at least 24 bytes, but may be larger to support custom per-primitive data.
	// Index buffers can't be used with procedural geometry.
	RTGT_Procedural,

	// Linear Swept Sphere (LSS) is a ray tracing primitive that represents the volume traced by a sphere moving
	// linearly between two endpoints, optionally with varying radii. It efficiently models strand-like geometry
	// such as hair or fur without tessellating into triangles, offering smoother appearance, lower memory use,
	// and hardware-accelerated ray intersections.
	RTGT_LinearSweptSpheres,
};
DECLARE_INTRINSIC_TYPE_LAYOUT(ERayTracingGeometryType);

enum class ERayTracingGeometryInitializerType : uint8
{
	// Fully initializes the RayTracingGeometry object: creates underlying buffer and initializes shader parameters.
	Rendering,

	// Does not create underlying buffer or shader parameters. Used by the streaming system as an object that is streamed into. 
	StreamingDestination,

	// Creates buffers but does not create shader parameters. Used for intermediate objects in the streaming system.
	StreamingSource,
};
DECLARE_INTRINSIC_TYPE_LAYOUT(ERayTracingGeometryInitializerType);

struct FRayTracingGeometrySegment
{
public:
	FBufferRHIRef VertexBuffer = nullptr;
	EVertexElementType VertexBufferElementType = VET_Float3;

	// Offset in bytes from the base address of the vertex buffer.
	uint32 VertexBufferOffset = 0;

	// Number of bytes between elements of the vertex buffer (sizeof VET_Float3 by default).
	// Must be equal or greater than the size of the position vector.
	uint32 VertexBufferStride = 12;

	// Number of vertices (positions) in VertexBuffer.
	// If an index buffer is present, this must be at least the maximum index value in the index buffer + 1.
	uint32 MaxVertices = 0;

	// Primitive range for this segment.
	uint32 FirstPrimitive = 0;
	uint32 NumPrimitives = 0;

	// Indicates whether any-hit shader could be invoked when hitting this geometry segment.
	// Setting this to `false` turns off any-hit shaders, making the section "opaque" and improving ray tracing performance.
	bool bForceOpaque = false;

	// Any-hit shader may be invoked multiple times for the same primitive during ray traversal.
	// Setting this to `false` guarantees that only a single instance of any-hit shader will run per primitive, at some performance cost.
	bool bAllowDuplicateAnyHitShaderInvocation = true;

	// Indicates whether this section is enabled and should be taken into account during acceleration structure creation
	bool bEnabled = true;
};

struct FRayTracingGeometryInitializer
{
public:
	FBufferRHIRef IndexBuffer = nullptr;

	// Offset in bytes from the base address of the index buffer.
	uint32 IndexBufferOffset = 0;

	// Stride in bytes of the index buffer (2 for 16-bit indices, 4 for 32-bit).
	uint32 IndexBufferStride = 0;

	ERayTracingGeometryType GeometryType = RTGT_Triangles;

	// Total number of primitives in all segments of the geometry. Only used for validation.
	uint32 TotalPrimitiveCount = 0;

	bool bFastBuild = false;
	bool bAllowUpdate = false;
	bool bAllowCompaction = true;
	bool bTemplate = false;
	ERayTracingGeometryInitializerType Type = ERayTracingGeometryInitializerType::Rendering;

	// Partitions of geometry to allow different shader and resource bindings.
	// All ray tracing geometries must have at least one segment.
	TArray<FRayTracingGeometrySegment> Segments;

	// Offline built geometry data. If null, the geometry will be built by the RHI at runtime.
	FResourceArrayUploadInterface* OfflineData = nullptr;
	FRayTracingGeometryOfflineDataHeader OfflineDataHeader;

	// Pointer to an existing ray tracing geometry which the new geometry is built from.
	FRHIRayTracingGeometry* SourceGeometry = nullptr;

	// Use FDebugName for auto-generated debug names with numbered suffixes, it is a variation of FMemoryImageName with optional number postfix.
	FDebugName DebugName;
	// Store the path name of the owner object for resource tracking. FMemoryImageName allows a conversion to/from FName.
	FName OwnerName;

	uint32 GetIndexBufferStride() const
	{
		if (IndexBufferStride != 0)
		{
			return IndexBufferStride;
		}

		if (IndexBuffer)
		{
			ensureMsgf(false, 
				TEXT("FRayTracingGeometryInitializer::IndexBufferStride not set; falling back to IndexBuffer->GetStride(). ")
				TEXT("Set IndexBufferStride when building the initializer to avoid a data race with index buffer streaming."));
			return IndexBuffer->GetStride();
		}

		return 0;
	}
};

#if DO_CHECK
inline bool operator==(const FRayTracingGeometryInitializer& LHS, const FRayTracingGeometryInitializer& RHS)
{
	// Can't compare LHS == RHS directly due to some members not having equality operators

	if (LHS.IndexBuffer != RHS.IndexBuffer
		|| LHS.IndexBufferOffset != RHS.IndexBufferOffset
		|| LHS.IndexBufferStride != RHS.IndexBufferStride
		|| LHS.GeometryType != RHS.GeometryType
		|| LHS.TotalPrimitiveCount != RHS.TotalPrimitiveCount)
	{
		return false;
	}

	// Can't compare Segments directly due to some members not having equality operators
	if (LHS.Segments.Num() != RHS.Segments.Num())
	{
		return false;
	}

	for (int32 SegmentIndex = 0; SegmentIndex < LHS.Segments.Num(); ++SegmentIndex)
	{
		//if (LHS.Segments[SegmentIndex] != RHS.Segments[SegmentIndex])
		//{
		//	return false;
		//}

		if (LHS.Segments[SegmentIndex].VertexBuffer != RHS.Segments[SegmentIndex].VertexBuffer
			|| LHS.Segments[SegmentIndex].VertexBufferElementType != RHS.Segments[SegmentIndex].VertexBufferElementType
			|| LHS.Segments[SegmentIndex].VertexBufferOffset != RHS.Segments[SegmentIndex].VertexBufferOffset
			|| LHS.Segments[SegmentIndex].VertexBufferStride != RHS.Segments[SegmentIndex].VertexBufferStride
			|| LHS.Segments[SegmentIndex].MaxVertices != RHS.Segments[SegmentIndex].MaxVertices
			|| LHS.Segments[SegmentIndex].FirstPrimitive != RHS.Segments[SegmentIndex].FirstPrimitive
			|| LHS.Segments[SegmentIndex].NumPrimitives != RHS.Segments[SegmentIndex].NumPrimitives
			|| LHS.Segments[SegmentIndex].bForceOpaque != RHS.Segments[SegmentIndex].bForceOpaque
			|| LHS.Segments[SegmentIndex].bAllowDuplicateAnyHitShaderInvocation != RHS.Segments[SegmentIndex].bAllowDuplicateAnyHitShaderInvocation
			|| LHS.Segments[SegmentIndex].bEnabled != RHS.Segments[SegmentIndex].bEnabled)
		{
			return false;
		}
	}

	if (LHS.OfflineData != RHS.OfflineData
		|| LHS.SourceGeometry != RHS.SourceGeometry
		|| LHS.bFastBuild != RHS.bFastBuild
		|| LHS.bAllowUpdate != RHS.bAllowUpdate
		|| LHS.bAllowCompaction != RHS.bAllowCompaction
		|| LHS.Type != RHS.Type)
	{
		return false;
	}

	// Can't compare DebugName directly due to FDebugName not having equality operator
	if (LHS.OwnerName != RHS.OwnerName)
	{
		return false;
	}

	return true;
}
#endif

enum ERayTracingSceneLifetime
{
	// Scene may only be used during the frame when it was created.
	RTSL_SingleFrame,

	// Scene may be constructed once and used in any number of later frames (not currently implemented).
	// RTSL_MultiFrame,
};

enum class ERayTracingAccelerationStructureFlags
{
	None = 0,
	AllowUpdate = 1 << 0,
	AllowCompaction = 1 << 1,
	FastTrace = 1 << 2,
	FastBuild = 1 << 3,
	MinimizeMemory = 1 << 4,
};
ENUM_CLASS_FLAGS(ERayTracingAccelerationStructureFlags);

enum class ERayTracingShaderBindingTableLifetime : uint8
{
	Transient,						//< SBT will be reallocated each frame
	Persistent,						//< SBT will be persistently stored and only new or changed bindings will be set
};
ENUM_CLASS_FLAGS(ERayTracingShaderBindingTableLifetime);

enum class ERayTracingShaderBindingMode : uint8
{
	Disabled	= 0,				//< No binding data at all
	Inline		= 1 << 0,			//< Binding data for inline raytracing
	RTPSO		= 1 << 1,			//< Binding data for raytracing using RTPSOs
};
ENUM_CLASS_FLAGS(ERayTracingShaderBindingMode);

enum class ERayTracingHitGroupIndexingMode : uint8
{
	Allow,
	Disallow,
};
ENUM_CLASS_FLAGS(ERayTracingHitGroupIndexingMode);

struct FRayTracingShaderBindingTableInitializer
{
	// Defines lifetime of the shader binding table
	ERayTracingShaderBindingTableLifetime Lifetime = ERayTracingShaderBindingTableLifetime::Transient;

	// Defines which types of binding data needs to be stored in the SBT (Inline and/or RTPSO)
	ERayTracingShaderBindingMode ShaderBindingMode = ERayTracingShaderBindingMode::Disabled;
	
	// Allow indexing of the hit group shaders for RTPSO bindings - if disabled then the SBT won't store any hit group data
	ERayTracingHitGroupIndexingMode HitGroupIndexingMode = ERayTracingHitGroupIndexingMode::Allow;

	// Local binding data size used for each entry in the SBT (needs to be at least as big as the local binding data size of all shaders used in the SBT) 
	uint32 LocalBindingDataSize = 0;

	// This value controls how many elements will be allocated in the shader binding table per geometry segment.
	// Changing this value allows different hit shaders to be used for different effects.
	// For example, setting this to 2 allows one hit shader for regular material evaluation and a different one for shadows.
	// Desired hit shader can be selected by providing appropriate RayContributionToHitGroupIndex to TraceRay() function.
	// Use ShaderSlot argument in SetRayTracingHitGroup() to assign shaders and resources for specific part of the shder binding table record.
	uint32 NumShaderSlotsPerGeometrySegment = 1;

	// Maximum number of geometry segments which can be stored in the hit group binding data
	uint32 NumGeometrySegments = 0;

	// At least one miss shader must be present in a ray tracing scene.
	// Default miss shader is always in slot 0. Default shader must not use local resources.
	// Custom miss shaders can be bound to other slots using SetRayTracingMissShader().
	uint32 NumMissShaderSlots = 1;

	// Defines how many different callable shaders with unique resource bindings can be bound to this scene.
	// Shaders and resources are assigned to slots in the scene using SetRayTracingCallableShader().
	uint32 NumCallableShaderSlots = 0;
};

struct FRayTracingSceneInitializer
{
	// Maximum number of instances in this scene. Actual number of instances is specified in FRayTracingSceneBuildParams.
	uint32 MaxNumInstances = 0;

	UE_DEPRECATED(5.6, "Use FRayTracingShaderBindingTableInitializer instead.")
	uint32 NumTotalSegments = 0;

	// Defines whether data in this scene should persist between frames.
	// Currently only single-frame lifetime is supported.
	ERayTracingSceneLifetime Lifetime = RTSL_SingleFrame;
	
	// Controls the flags of the ray tracing scene build.
	ERayTracingAccelerationStructureFlags BuildFlags = ERayTracingAccelerationStructureFlags::FastTrace;

	FName DebugName;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRayTracingSceneInitializer() = default;
	FRayTracingSceneInitializer(FRayTracingSceneInitializer&&) = default;
	FRayTracingSceneInitializer& operator=(FRayTracingSceneInitializer&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

struct FRayTracingAccelerationStructureSize
{
	uint64 ResultSize = 0;
	uint64 BuildScratchSize = 0;
	uint64 UpdateScratchSize = 0;
};

struct FRayTracingAccelerationStructureOfflineMetadata
{
	uint32 Size = 0;
	uint32 ScratchSize = 0;
	uint32 SerializedSize = 0;
	uint32 SerializedOffset = 0;
};

class FRHIRayTracingAccelerationStructure
	: public FRHIResource
#if ENABLE_RHI_VALIDATION
	, public RHIValidation::FAccelerationStructureResource
#endif
{
public:
	FRayTracingAccelerationStructureSize GetSizeInfo() const
	{
		return SizeInfo;
	};

protected:
	FRHIRayTracingAccelerationStructure(bool bIsRayTracingScene)
		: FRHIResource(bIsRayTracingScene ? RRT_RayTracingScene : RRT_RayTracingGeometry)
	{}

	FRayTracingAccelerationStructureSize SizeInfo = {};
};

using FRayTracingAccelerationStructureAddress UE_DEPRECATED(5.7, "FRayTracingAccelerationStructureAddress is deprecated and should no longer be used.") = uint64;

/** Bottom level ray tracing acceleration structure (contains triangles). */
class FRHIRayTracingGeometry : public FRHIRayTracingAccelerationStructure
{
public:
	FRHIRayTracingGeometry()
		: FRHIRayTracingAccelerationStructure(false)
	{}

	FRHIRayTracingGeometry(const FRayTracingGeometryInitializer& InInitializer)
		: FRHIRayTracingAccelerationStructure(false)
		, Initializer(InInitializer)
	{}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use RHICmdList.WriteResourceGPUAddresses(...) instead.")
	virtual FRayTracingAccelerationStructureAddress GetAccelerationStructureAddress(uint64 GPUIndex) const = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual bool IsCompressed() const { return false; }

	const FRayTracingGeometryInitializer& GetInitializer() const
	{
		return Initializer;
	}

	uint32 GetNumSegments() const 
	{ 
		return Initializer.Segments.Num(); 
	}

protected:

	void ValidateIndexBufferStride() const
	{
#if DO_CHECK
		if (Initializer.IndexBuffer && Initializer.IndexBufferStride != 0)
		{
			ensureMsgf(Initializer.IndexBufferStride == Initializer.IndexBuffer->GetStride(),
				TEXT("FRayTracingGeometryInitializer::IndexBufferStride (%u) does not match the live index buffer ")
				TEXT("stride (%u) at build time - the captured stride is stale or was set incorrectly by the producer."),
				Initializer.IndexBufferStride, Initializer.IndexBuffer->GetStride());
		}
#endif
	}

	FRayTracingGeometryInitializer Initializer = {};
};

/** Top level ray tracing acceleration structure (contains instances of meshes). */
class FRHIRayTracingScene
	: public FRHIRayTracingAccelerationStructure
{
public:
	FRHIRayTracingScene()
		: FRHIRayTracingAccelerationStructure(true)
	{}

	virtual const FRayTracingSceneInitializer& GetInitializer() const = 0;

protected:
	RHI_API void SetReferencedResources(TConstArrayView<FRHIRayTracingGeometry*> InReferencedGeometries, TConstArrayView<FRHIBuffer*> InReferencedBuffers);

	// Unique list of geometries referenced by all instances in this scene.
	// Any referenced geometry is kept alive while the scene is alive.
	TArray<TRefCountPtr<FRHIRayTracingGeometry>> ReferencedGeometries;

	// Unique list of buffers referenced by all instances in this scene.
	// Any referenced buffer is kept alive while the scene is alive.
	TArray<TRefCountPtr<FRHIBuffer>> ReferencedBuffers;
};

struct FRayTracingClusterOperationSize
{
	uint64 ResultMaxSizeInBytes = 0;
	uint64 ScratchSizeInBytes = 0;
};

enum class ERayTracingClusterOperationType : uint8
{
	MOVE,						// Moves CLAS, CLAS Templates, or Clustered BLAS
	CLAS_BUILD,					// Builds CLAS from clusters of triangles
	CLAS_BUILD_TEMPLATES,		// Builds CLAS templates from clusters of triangles
	CLAS_INSTANTIATE_TEMPLATES,	// Instantiates CLAS templates
	BLAS_BUILD					// Builds Clustered BLAS from CLAS
};

enum class ERayTracingClusterOperationMoveType : uint8
{
	BOTTOM_LEVEL,				// Moved objects are Clustered BLAS
	CLUSTER_LEVEL,				// Moved objects are CLAS
	TEMPLATE					// Moved objects are CLAS Templates
};

enum class ERayTracingClusterOperationMode : uint8
{
	IMPLICIT_DESTINATIONS,		// Provide total buffer space, driver places results within, returns VAs and actual sizes
	EXPLICIT_DESTINATIONS,		// Provide individual target VAs, driver places them there, returns actual sizes
	GET_SIZES					// Get minimum size per element
};

enum class ERayTracingClusterOperationFlags : uint8
{
	NONE = 0x0,
	FAST_TRACE = 0x1,
	FAST_BUILD = 0x2,
	NO_OVERLAP = 0x4,
	ALLOW_OMM = 0x8
};
ENUM_CLASS_FLAGS(ERayTracingClusterOperationFlags);

struct FRayTracingClusterOperationMoveInitializer
{
	ERayTracingClusterOperationMoveType Type;
	uint32 MaxBytes = 0;
};

struct FRayTracingClusterOperationCLASInitializer
{
	// Format of the CLAS vertices
	EVertexElementType VertexFormat = VET_Float3;

	// Index of the last geometry in a single CLAS
	uint32 MaxGeometryIndex = 0;

	// Maximum number of unique geometries in a single CLAS
	uint32 MaxUniqueGeometryCount = 1;

	// Maximum number of triangles in a single CLAS
	uint32 MaxTriangleCount = 0;

	// Maximum number of vertices in a single CLAS
	uint32 MaxVertexCount = 0;

	// Maximum number of triangles across all CLAS (of the current cluster operation)
	uint32 MaxTotalTriangleCount = 0;

	// Maximum number of vertices across all CLAS (of the current cluster operation)
	uint32 MaxTotalVertexCount = 0;

	// Minimum number of bits to be truncated in vertex positions across all CLAS (in the current cluster operation)
	uint32 MinPositionTruncateBitCount = 0;
};

struct FRayTracingClusterOperationBLASInitializer
{
	// Maximum number of CLAS referenced by a single BLAS
	uint32 MaxCLASPerBLASCount = 0;

	// Maximum number of CLAS references across all BLAS (in the current cluster operation)
	uint32 MaxTotalCLASCount = 0;
};

struct FRayTracingClusterOperationInitializer
{
	// Maximum number of acceleration structures (or templates) to build/instantiate/move
	uint32 MaxResultCount = 0;

	ERayTracingClusterOperationType Type;
	ERayTracingClusterOperationMode Mode;
	ERayTracingClusterOperationFlags Flags;

	union
	{
		FRayTracingClusterOperationMoveInitializer Move;
		FRayTracingClusterOperationCLASInitializer CLAS;
		FRayTracingClusterOperationBLASInitializer BLAS;
	} Operation;
};

class FRHIShaderBindingTable
	: public FRHIResource
#if ENABLE_RHI_VALIDATION
	, public RHIValidation::FShaderBindingTable
#endif
{
public:
	FRHIShaderBindingTable(const FRayTracingShaderBindingTableInitializer& InInitializer)
		: FRHIResource(RRT_RayTracingShaderBindingTable)	
#if ENABLE_RHI_VALIDATION
		, RHIValidation::FShaderBindingTable(InInitializer)
#endif
		, Initializer(InInitializer)
	{}

	const FRayTracingShaderBindingTableInitializer& GetInitializer() const
	{
		return Initializer;
	}

	// Returns the size and stride of the structured buffer for RHI-specific inline parameters associated with this SBT.
	// Returns 0 if current RHI does not require this buffer.
	virtual FRHISizeAndStride GetInlineBindingDataSizeAndStride() const
	{		
		return FRHISizeAndStride{0,0};
	}

protected:
	FRayTracingShaderBindingTableInitializer Initializer = {};
};

enum class ERHIShaderBundleMode : uint8
{
	// Compute shaders
	CS,

	// Mesh and pixel shaders
	MSPS,

	// Vertex and pixel shaders
	VSPS,

	MAX
};

struct FShaderBundleCreateInfo
{
	uint32 NumRecords	= 0u;
	uint32 ArgOffset	= 0u;
	uint32 ArgStride	= 0u;

	ERHIShaderBundleMode Mode = ERHIShaderBundleMode::CS;
};

class FRHIShaderBundle : public FRHIResource
{
public:
	const uint32 NumRecords;
	const uint32 ArgOffset;
	const uint32 ArgStride;
	const ERHIShaderBundleMode Mode;

public:
	FRHIShaderBundle() = delete;
	FRHIShaderBundle(const FShaderBundleCreateInfo& CreateInfo)
	: FRHIResource(RRT_ShaderBundle)
	, NumRecords(CreateInfo.NumRecords)
	, ArgOffset(CreateInfo.ArgOffset)
	, ArgStride(CreateInfo.ArgStride)
	, Mode(CreateInfo.Mode)
	{
#if DO_CHECK
		if (Mode == ERHIShaderBundleMode::CS)
		{
			// Load3
			check(ArgStride >= 12u);
		}
		else if (Mode == ERHIShaderBundleMode::MSPS)
		{
			// Load
			check(ArgStride >= 4u);
		}
		else if (Mode == ERHIShaderBundleMode::VSPS)
		{
			// Load4
			check(ArgStride >= 16u);
		}
		else
		{
			checkNoEntry();
		}
#endif
	}

	const TCHAR* GetModeName() const
	{
		switch (Mode)
		{
		case ERHIShaderBundleMode::CS:
			return TEXT("CS");
		case ERHIShaderBundleMode::MSPS:
			return TEXT("MSPS");
		case ERHIShaderBundleMode::VSPS:
			return TEXT("VSPS");
			break;
		case ERHIShaderBundleMode::MAX:
		default:
			checkNoEntry();
			return TEXT("<none>");
		}
	}
};

/* Generic staging buffer class used by FRHIGPUMemoryReadback
* RHI specific staging buffers derive from this
*/
class FRHIStagingBuffer : public FRHIResource
{
public:
	FRHIStagingBuffer()
		: FRHIResource(RRT_StagingBuffer)
		, bIsLocked(false)
	{}

	virtual ~FRHIStagingBuffer() = default;

	virtual void *Lock(uint32 Offset, uint32 NumBytes) = 0;
	virtual void Unlock() = 0;

	// For debugging, may not be implemented on all RHIs
	virtual uint64 GetGPUSizeBytes() const { return 0; }

protected:
	bool bIsLocked;
};

class FGenericRHIStagingBuffer : public FRHIStagingBuffer
{
public:
	FGenericRHIStagingBuffer()
		: FRHIStagingBuffer()
	{}

	~FGenericRHIStagingBuffer() = default;

	RHI_API virtual void* Lock(uint32 Offset, uint32 NumBytes) final override;
	RHI_API virtual void Unlock() final override;
	virtual uint64 GetGPUSizeBytes() const final override { return ShadowBuffer.IsValid() ? ShadowBuffer->GetSize() : 0; }

	FBufferRHIRef ShadowBuffer;
	uint32 Offset;
};

class FRHIRenderTargetView
{
public:
	FRHITexture* Texture = nullptr;
	uint32 MipIndex = 0;

	/** Array slice or texture cube face.  Only valid if texture resource was created with TexCreate_TargetArraySlicesIndependently! */
	uint32 ArraySliceIndex = ~0u;

	ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ENoAction;
	ERenderTargetStoreAction StoreAction = ERenderTargetStoreAction::ENoAction;

	FRHIRenderTargetView() = default;
	FRHIRenderTargetView(FRHIRenderTargetView&&) = default;
	FRHIRenderTargetView(const FRHIRenderTargetView&) = default;
	FRHIRenderTargetView& operator=(FRHIRenderTargetView&&) = default;
	FRHIRenderTargetView& operator=(const FRHIRenderTargetView&) = default;

	//common case
	explicit FRHIRenderTargetView(FRHITexture* InTexture, ERenderTargetLoadAction InLoadAction) :
		Texture(InTexture),
		MipIndex(0),
		ArraySliceIndex(-1),
		LoadAction(InLoadAction),
		StoreAction(ERenderTargetStoreAction::EStore)
	{}

	//common case
	explicit FRHIRenderTargetView(FRHITexture* InTexture, ERenderTargetLoadAction InLoadAction, uint32 InMipIndex, uint32 InArraySliceIndex) :
		Texture(InTexture),
		MipIndex(InMipIndex),
		ArraySliceIndex(InArraySliceIndex),
		LoadAction(InLoadAction),
		StoreAction(ERenderTargetStoreAction::EStore)
	{}
	
	explicit FRHIRenderTargetView(FRHITexture* InTexture, uint32 InMipIndex, uint32 InArraySliceIndex, ERenderTargetLoadAction InLoadAction, ERenderTargetStoreAction InStoreAction) :
		Texture(InTexture),
		MipIndex(InMipIndex),
		ArraySliceIndex(InArraySliceIndex),
		LoadAction(InLoadAction),
		StoreAction(InStoreAction)
	{}

	bool operator==(const FRHIRenderTargetView& Other) const
	{
		return 
			Texture == Other.Texture &&
			MipIndex == Other.MipIndex &&
			ArraySliceIndex == Other.ArraySliceIndex &&
			LoadAction == Other.LoadAction &&
			StoreAction == Other.StoreAction;
	}
};

class FRHIDepthRenderTargetView
{
public:
	FRHITexture* Texture;

	ERenderTargetLoadAction		DepthLoadAction;
	ERenderTargetStoreAction	DepthStoreAction;
	ERenderTargetLoadAction		StencilLoadAction;

private:
	ERenderTargetStoreAction	StencilStoreAction;
	FExclusiveDepthStencil		DepthStencilAccess;
public:

	// accessor to prevent write access to StencilStoreAction
	ERenderTargetStoreAction GetStencilStoreAction() const { return StencilStoreAction; }
	// accessor to prevent write access to DepthStencilAccess
	FExclusiveDepthStencil GetDepthStencilAccess() const { return DepthStencilAccess; }

	explicit FRHIDepthRenderTargetView() :
		Texture(nullptr),
		DepthLoadAction(ERenderTargetLoadAction::ENoAction),
		DepthStoreAction(ERenderTargetStoreAction::ENoAction),
		StencilLoadAction(ERenderTargetLoadAction::ENoAction),
		StencilStoreAction(ERenderTargetStoreAction::ENoAction),
		DepthStencilAccess(FExclusiveDepthStencil::DepthNop_StencilNop)
	{
		Validate();
	}

	//common case
	explicit FRHIDepthRenderTargetView(FRHITexture* InTexture, ERenderTargetLoadAction InLoadAction, ERenderTargetStoreAction InStoreAction) :
		Texture(InTexture),
		DepthLoadAction(InLoadAction),
		DepthStoreAction(InStoreAction),
		StencilLoadAction(InLoadAction),
		StencilStoreAction(InStoreAction),
		DepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilWrite)
	{
		Validate();
	}

	explicit FRHIDepthRenderTargetView(FRHITexture* InTexture, ERenderTargetLoadAction InLoadAction, ERenderTargetStoreAction InStoreAction, FExclusiveDepthStencil InDepthStencilAccess) :
		Texture(InTexture),
		DepthLoadAction(InLoadAction),
		DepthStoreAction(InStoreAction),
		StencilLoadAction(InLoadAction),
		StencilStoreAction(InStoreAction),
		DepthStencilAccess(InDepthStencilAccess)
	{
		Validate();
	}

	explicit FRHIDepthRenderTargetView(FRHITexture* InTexture, ERenderTargetLoadAction InDepthLoadAction, ERenderTargetStoreAction InDepthStoreAction, ERenderTargetLoadAction InStencilLoadAction, ERenderTargetStoreAction InStencilStoreAction) :
		Texture(InTexture),
		DepthLoadAction(InDepthLoadAction),
		DepthStoreAction(InDepthStoreAction),
		StencilLoadAction(InStencilLoadAction),
		StencilStoreAction(InStencilStoreAction),
		DepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilWrite)
	{
		Validate();
	}

	explicit FRHIDepthRenderTargetView(FRHITexture* InTexture, ERenderTargetLoadAction InDepthLoadAction, ERenderTargetStoreAction InDepthStoreAction, ERenderTargetLoadAction InStencilLoadAction, ERenderTargetStoreAction InStencilStoreAction, FExclusiveDepthStencil InDepthStencilAccess) :
		Texture(InTexture),
		DepthLoadAction(InDepthLoadAction),
		DepthStoreAction(InDepthStoreAction),
		StencilLoadAction(InStencilLoadAction),
		StencilStoreAction(InStencilStoreAction),
		DepthStencilAccess(InDepthStencilAccess)
	{
		Validate();
	}

	void Validate() const
	{
		// VK and Metal MAY leave the attachment in an undefined state if the StoreAction is DontCare. So we can't assume read-only implies it should be DontCare unless we know for sure it will never be used again.
		// ensureMsgf(DepthStencilAccess.IsDepthWrite() || DepthStoreAction == ERenderTargetStoreAction::ENoAction, TEXT("Depth is read-only, but we are performing a store.  This is a waste on mobile.  If depth can't change, we don't need to store it out again"));
		/*ensureMsgf(DepthStencilAccess.IsStencilWrite() || StencilStoreAction == ERenderTargetStoreAction::ENoAction, TEXT("Stencil is read-only, but we are performing a store.  This is a waste on mobile.  If stencil can't change, we don't need to store it out again"));*/
	}

	bool operator==(const FRHIDepthRenderTargetView& Other) const
	{
		return
			Texture == Other.Texture &&
			DepthLoadAction == Other.DepthLoadAction &&
			DepthStoreAction == Other.DepthStoreAction &&
			StencilLoadAction == Other.StencilLoadAction &&
			StencilStoreAction == Other.StencilStoreAction &&
			DepthStencilAccess == Other.DepthStencilAccess;
	}
};

class FRHISetRenderTargetsInfo
{
public:
	// Color Render Targets Info
	FRHIRenderTargetView ColorRenderTarget[MaxSimultaneousRenderTargets];	
	int32 NumColorRenderTargets;
	bool bClearColor;

	// Color Render Targets Info
	FRHIRenderTargetView ColorResolveRenderTarget[MaxSimultaneousRenderTargets];	
	bool bHasResolveAttachments;

	// Depth/Stencil Render Target Info
	FRHIDepthRenderTargetView DepthStencilRenderTarget;	
	// Used when depth resolve is enabled.
	FRHIDepthRenderTargetView DepthStencilResolveRenderTarget;
	bool bClearDepth;
	bool bClearStencil;

	FRHITexture* ShadingRateTexture;
	EVRSRateCombiner ShadingRateTextureCombiner;

	uint8 MultiViewCount;

	FRHISetRenderTargetsInfo() :
		NumColorRenderTargets(0),
		bClearColor(false),
		bHasResolveAttachments(false),
		bClearDepth(false),
		ShadingRateTexture(nullptr),
		MultiViewCount(0)
	{}

	FRHISetRenderTargetsInfo(int32 InNumColorRenderTargets, const FRHIRenderTargetView* InColorRenderTargets, const FRHIDepthRenderTargetView& InDepthStencilRenderTarget) :
		NumColorRenderTargets(InNumColorRenderTargets),
		bClearColor(InNumColorRenderTargets > 0 && InColorRenderTargets[0].LoadAction == ERenderTargetLoadAction::EClear),
		bHasResolveAttachments(false),
		DepthStencilRenderTarget(InDepthStencilRenderTarget),		
		bClearDepth(InDepthStencilRenderTarget.Texture && InDepthStencilRenderTarget.DepthLoadAction == ERenderTargetLoadAction::EClear),
		ShadingRateTexture(nullptr),
		ShadingRateTextureCombiner(VRSRB_Passthrough)
	{
		check(InNumColorRenderTargets <= 0 || InColorRenderTargets);
		for (int32 Index = 0; Index < InNumColorRenderTargets; ++Index)
		{
			ColorRenderTarget[Index] = InColorRenderTargets[Index];			
		}
	}
	// @todo metal mrt: This can go away after all the cleanup is done
	void SetClearDepthStencil(bool bInClearDepth, bool bInClearStencil = false)
	{
		if (bInClearDepth)
		{
			DepthStencilRenderTarget.DepthLoadAction = ERenderTargetLoadAction::EClear;
		}
		if (bInClearStencil)
		{
			DepthStencilRenderTarget.StencilLoadAction = ERenderTargetLoadAction::EClear;
		}
		bClearDepth = bInClearDepth;		
		bClearStencil = bInClearStencil;		
	}

	uint32 CalculateHash() const
	{
		// Need a separate struct so we can memzero/remove dependencies on reference counts
		struct FHashableStruct
		{
			// *2 for color and resolves
			// depth goes in the third-to-last slot
			// depth resolve goes in the second-to-last slot
			// shading rate goes in the last slot
			FRHITexture* Texture[MaxSimultaneousRenderTargets*2 + 3];
			uint32 MipIndex[MaxSimultaneousRenderTargets];
			uint32 ArraySliceIndex[MaxSimultaneousRenderTargets];
			ERenderTargetLoadAction LoadAction[MaxSimultaneousRenderTargets];
			ERenderTargetStoreAction StoreAction[MaxSimultaneousRenderTargets];

			ERenderTargetLoadAction		DepthLoadAction;
			ERenderTargetStoreAction	DepthStoreAction;
			ERenderTargetLoadAction		StencilLoadAction;
			ERenderTargetStoreAction	StencilStoreAction;
			FExclusiveDepthStencil		DepthStencilAccess;

			bool bClearDepth;
			bool bClearStencil;
			bool bClearColor;
			bool bHasResolveAttachments;
			uint8 MultiViewCount;

			void Set(const FRHISetRenderTargetsInfo& RTInfo)
			{
				FMemory::Memzero(*this);
				for (int32 Index = 0; Index < RTInfo.NumColorRenderTargets; ++Index)
				{
					Texture[Index] = RTInfo.ColorRenderTarget[Index].Texture;
					Texture[MaxSimultaneousRenderTargets+Index] = RTInfo.ColorResolveRenderTarget[Index].Texture;
					MipIndex[Index] = RTInfo.ColorRenderTarget[Index].MipIndex;
					ArraySliceIndex[Index] = RTInfo.ColorRenderTarget[Index].ArraySliceIndex;
					LoadAction[Index] = RTInfo.ColorRenderTarget[Index].LoadAction;
					StoreAction[Index] = RTInfo.ColorRenderTarget[Index].StoreAction;
				}

				Texture[MaxSimultaneousRenderTargets*2] = RTInfo.DepthStencilRenderTarget.Texture;
				Texture[MaxSimultaneousRenderTargets*2 + 1] = RTInfo.DepthStencilResolveRenderTarget.Texture;
				Texture[MaxSimultaneousRenderTargets*2 + 2] = RTInfo.ShadingRateTexture;
				DepthLoadAction = RTInfo.DepthStencilRenderTarget.DepthLoadAction;
				DepthStoreAction = RTInfo.DepthStencilRenderTarget.DepthStoreAction;
				StencilLoadAction = RTInfo.DepthStencilRenderTarget.StencilLoadAction;
				StencilStoreAction = RTInfo.DepthStencilRenderTarget.GetStencilStoreAction();
				DepthStencilAccess = RTInfo.DepthStencilRenderTarget.GetDepthStencilAccess();

				bClearDepth = RTInfo.bClearDepth;
				bClearStencil = RTInfo.bClearStencil;
				bClearColor = RTInfo.bClearColor;
				bHasResolveAttachments = RTInfo.bHasResolveAttachments;
				MultiViewCount = RTInfo.MultiViewCount;
			}
		};

		FHashableStruct RTHash;
		FMemory::Memzero(RTHash);
		RTHash.Set(*this);
		return FCrc::MemCrc32(&RTHash, sizeof(RTHash));
	}
};

class FRHICustomPresent : public FRHIResource
{
public:
	FRHICustomPresent() : FRHIResource(RRT_CustomPresent) {}
	
	virtual ~FRHICustomPresent() = default; // should release any references to D3D resources.
	
	// Called when viewport is resized.
	virtual void OnBackBufferResize() = 0;

	// Called from render thread to see if a native present will be requested for this frame.
	// @return	true if native Present will be requested for this frame; false otherwise.  Must
	// match value subsequently returned by Present for this frame.
	virtual bool NeedsNativePresent() = 0;
	// In come cases we want to use custom present but still let the native environment handle 
	// advancement of the backbuffer indices.
	// @return true if backbuffer index should advance independently from CustomPresent.
	virtual bool NeedsAdvanceBackbuffer() { return false; };

	// Called from RHI thread when the engine begins drawing to the viewport.
	UE_DEPRECATED(5.7, "FRHICustomPresent::BeginDrawing is deprecated and never called.")
	virtual void BeginDrawing() {};

	// Called from RHI thread to perform custom present.
	// @param InOutSyncInterval - in out param, indicates if vsync is on (>0) or off (==0).
	// @param RHICmdContext - the current rhi command context
	// @return	true if native Present should be also be performed; false otherwise. If it returns
	// true, then InOutSyncInterval could be modified to switch between VSync/NoVSync for the normal 
	// Present.  Must match value previously returned by NeedsNativePresent for this frame.
	virtual bool Present(FRHIViewport* Viewport, IRHICommandContext& RHICmdContext, int32& InOutSyncInterval) { check(false); return true; };

	// Called from Submission thread to perform custom present.
	// @param InOutSyncInterval - in out param, indicates if vsync is on (>0) or off (==0).
	// @return	true if native Present should be also be performed; false otherwise. If it returns
	// true, then InOutSyncInterval could be modified to switch between VSync/NoVSync for the normal 
	// Present.  Must match value previously returned by NeedsNativePresent for this frame.
	virtual bool PresentOnSubmissionThread(int32& InOutSyncInterval) { check(false); return true; };

	// Called after native Present has been called
	virtual void PostPresent() {};

	// Called when rendering thread is acquired
	virtual void OnAcquireThreadOwnership() {}
	// Called when rendering thread is released
	virtual void OnReleaseThreadOwnership() {}
};


// Templates to convert an FRHI*Shader to its enum
template<typename TRHIShader> struct TRHIShaderToEnum {};
template<> struct TRHIShaderToEnum<FRHIVertexShader>           { enum { ShaderFrequency = SF_Vertex        }; };
template<> struct TRHIShaderToEnum<FRHIMeshShader>             { enum { ShaderFrequency = SF_Mesh          }; };
template<> struct TRHIShaderToEnum<FRHIAmplificationShader>    { enum { ShaderFrequency = SF_Amplification }; };
template<> struct TRHIShaderToEnum<FRHIPixelShader>            { enum { ShaderFrequency = SF_Pixel         }; };
template<> struct TRHIShaderToEnum<FRHIGeometryShader>         { enum { ShaderFrequency = SF_Geometry      }; };
template<> struct TRHIShaderToEnum<FRHIComputeShader>          { enum { ShaderFrequency = SF_Compute       }; };
template<> struct TRHIShaderToEnum<FRHIVertexShader*>          { enum { ShaderFrequency = SF_Vertex        }; };
template<> struct TRHIShaderToEnum<FRHIMeshShader*>            { enum { ShaderFrequency = SF_Mesh          }; };
template<> struct TRHIShaderToEnum<FRHIAmplificationShader*>   { enum { ShaderFrequency = SF_Amplification }; };
template<> struct TRHIShaderToEnum<FRHIPixelShader*>           { enum { ShaderFrequency = SF_Pixel         }; };
template<> struct TRHIShaderToEnum<FRHIGeometryShader*>        { enum { ShaderFrequency = SF_Geometry      }; };
template<> struct TRHIShaderToEnum<FRHIComputeShader*>         { enum { ShaderFrequency = SF_Compute       }; };
template<> struct TRHIShaderToEnum<FVertexShaderRHIRef>        { enum { ShaderFrequency = SF_Vertex        }; };
template<> struct TRHIShaderToEnum<FMeshShaderRHIRef>          { enum { ShaderFrequency = SF_Mesh          }; };
template<> struct TRHIShaderToEnum<FAmplificationShaderRHIRef> { enum { ShaderFrequency = SF_Amplification }; };
template<> struct TRHIShaderToEnum<FPixelShaderRHIRef>         { enum { ShaderFrequency = SF_Pixel         }; };
template<> struct TRHIShaderToEnum<FGeometryShaderRHIRef>      { enum { ShaderFrequency = SF_Geometry      }; };
template<> struct TRHIShaderToEnum<FComputeShaderRHIRef>       { enum { ShaderFrequency = SF_Compute       }; };

template<typename TRHIShaderType>
inline const TCHAR* GetShaderFrequencyString(bool bIncludePrefix = true)
{
	return GetShaderFrequencyString(static_cast<EShaderFrequency>(TRHIShaderToEnum<TRHIShaderType>::ShaderFrequency), bIncludePrefix);
}

struct FBoundShaderStateInput
{
	inline FBoundShaderStateInput() {}

	inline FBoundShaderStateInput
	(
		FRHIVertexDeclaration* InVertexDeclarationRHI
		, FRHIVertexShader* InVertexShaderRHI
		, FRHIPixelShader* InPixelShaderRHI
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		, FRHIGeometryShader* InGeometryShaderRHI
#endif
	)
		: VertexDeclarationRHI(InVertexDeclarationRHI)
		, VertexShaderRHI(InVertexShaderRHI)
		, PixelShaderRHI(InPixelShaderRHI)
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		, GeometryShaderRHI(InGeometryShaderRHI)
#endif
	{
	}

#if PLATFORM_SUPPORTS_MESH_SHADERS
	inline FBoundShaderStateInput(
		FRHIMeshShader* InMeshShaderRHI,
		FRHIAmplificationShader* InAmplificationShader,
		FRHIPixelShader* InPixelShaderRHI)
		: PixelShaderRHI(InPixelShaderRHI)
		, MeshShaderRHI(InMeshShaderRHI)
		, AmplificationShaderRHI(InAmplificationShader)
	{
	}
#endif

	void AddRefResources()
	{
		if (GetMeshShader())
		{
			check(VertexDeclarationRHI == nullptr);
			check(VertexShaderRHI == nullptr);
			check(GetWorkGraphShader() == nullptr);
			GetMeshShader()->AddRef();

			if (GetAmplificationShader())
			{
				GetAmplificationShader()->AddRef();
			}
		}
		else if (GetWorkGraphShader())
		{
			check(VertexDeclarationRHI == nullptr);
			check(VertexShaderRHI == nullptr);
			check(GetMeshShader() == nullptr);
			GetWorkGraphShader()->AddRef();
		}
		else
		{
			check(VertexDeclarationRHI);
			VertexDeclarationRHI->AddRef();

			check(VertexShaderRHI);
			VertexShaderRHI->AddRef();
		}

		if (PixelShaderRHI)
		{
			PixelShaderRHI->AddRef();
		}

		if (GetGeometryShader())
		{
			GetGeometryShader()->AddRef();
		}
	}

	void ReleaseResources()
	{
		if (GetMeshShader())
		{
			check(VertexDeclarationRHI == nullptr);
			check(VertexShaderRHI == nullptr);
			check(GetWorkGraphShader() == nullptr);
			GetMeshShader()->Release();

			if (GetAmplificationShader())
			{
				GetAmplificationShader()->Release();
			}
		}
		else if (GetWorkGraphShader())
		{
			check(VertexDeclarationRHI == nullptr);
			check(VertexShaderRHI == nullptr);
			check(GetMeshShader() == nullptr);
			GetWorkGraphShader()->Release();
		}
		else
		{
			check(VertexDeclarationRHI);
			VertexDeclarationRHI->Release();

			check(VertexShaderRHI);
			VertexShaderRHI->Release();
		}

		if (PixelShaderRHI)
		{
			PixelShaderRHI->Release();
		}

		if (GetGeometryShader())
		{
			GetGeometryShader()->Release();
		}
	}

	FRHIVertexShader* GetVertexShader() const { return VertexShaderRHI; }
	FRHIPixelShader* GetPixelShader() const { return PixelShaderRHI; }

#if PLATFORM_SUPPORTS_MESH_SHADERS
	FRHIMeshShader* GetMeshShader() const { return MeshShaderRHI; }
	void SetMeshShader(FRHIMeshShader* InMeshShader) { MeshShaderRHI = InMeshShader; }
	FRHIAmplificationShader* GetAmplificationShader() const { return AmplificationShaderRHI; }
	void SetAmplificationShader(FRHIAmplificationShader* InAmplificationShader) { AmplificationShaderRHI = InAmplificationShader; }
	FSpecializationContainerType MeshShaderSpecializationConstants;
	FSpecializationContainerType AmplificationShaderSpecializationConstants;
#else
	constexpr FRHIMeshShader* GetMeshShader() const { return nullptr; }
	void SetMeshShader(FRHIMeshShader*) {}
	constexpr FRHIAmplificationShader* GetAmplificationShader() const { return nullptr; }
	void SetAmplificationShader(FRHIAmplificationShader*) {}
#endif

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	FRHIGeometryShader* GetGeometryShader() const { return GeometryShaderRHI; }
	void SetGeometryShader(FRHIGeometryShader* InGeometryShader) { GeometryShaderRHI = InGeometryShader; }
	FSpecializationContainerType GeometryShaderSpecializationConstants;
#else
	constexpr FRHIGeometryShader* GetGeometryShader() const { return nullptr; }
	void SetGeometryShader(FRHIGeometryShader*) {}
#endif

#if PLATFORM_SUPPORTS_WORKGRAPH_SHADERS
	FRHIWorkGraphShader* GetWorkGraphShader() const { return WorkGraphMeshShaderRHI; }
	void SetWorkGraphShader(FRHIWorkGraphShader* InWorkGraphMeshShader) { WorkGraphMeshShaderRHI = InWorkGraphMeshShader; }
#else
	FRHIWorkGraphShader* GetWorkGraphShader() const { return nullptr; }
	void SetWorkGraphShader(FRHIWorkGraphShader* InWorkGraphMeshShader) {}
#endif

	FRHIVertexDeclaration* VertexDeclarationRHI = nullptr;
	FRHIVertexShader* VertexShaderRHI = nullptr;
	FRHIPixelShader* PixelShaderRHI = nullptr;
	FSpecializationContainerType VertexShaderSpecializationConstants;
	FSpecializationContainerType PixelShaderSpecializationConstants;
private:
#if PLATFORM_SUPPORTS_MESH_SHADERS
	FRHIMeshShader* MeshShaderRHI = nullptr;
	FRHIAmplificationShader* AmplificationShaderRHI = nullptr;
#endif
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	FRHIGeometryShader* GeometryShaderRHI = nullptr;
#endif
#if PLATFORM_SUPPORTS_WORKGRAPH_SHADERS
	FRHIWorkGraphShader* WorkGraphMeshShaderRHI = nullptr;
#endif
};

// Hints for some RHIs that support subpasses
enum class ESubpassHint : uint8
{
	// Regular rendering
	None,

	// Render pass has depth reading subpass
	DepthReadSubpass,

	// Mobile defferred shading subpass
	DeferredShadingSubpass,

	// Mobile MSAA custom resolve subpass. Includes DepthReadSubpass.
	CustomResolveSubpass,
};

enum class EConservativeRasterization : uint8
{
	Disabled,
	Overestimated,
};

struct FGraphicsPipelineRenderTargetsInfo
{
	FGraphicsPipelineRenderTargetsInfo()
		: RenderTargetFormats(InPlace, UE_PIXELFORMAT_TO_UINT8(PF_Unknown))
		, RenderTargetFlags(InPlace, TexCreate_None)
		, DepthStencilAccess(FExclusiveDepthStencil::DepthNop_StencilNop)
	{
	}

	uint32															RenderTargetsEnabled = 0;
	TStaticArray<uint8, MaxSimultaneousRenderTargets>				RenderTargetFormats;
	TStaticArray<ETextureCreateFlags, MaxSimultaneousRenderTargets>	RenderTargetFlags;
	EPixelFormat													DepthStencilTargetFormat = PF_Unknown;
	ETextureCreateFlags												DepthStencilTargetFlag = ETextureCreateFlags::None;
	ERenderTargetLoadAction											DepthTargetLoadAction = ERenderTargetLoadAction::ENoAction;
	ERenderTargetStoreAction										DepthTargetStoreAction = ERenderTargetStoreAction::ENoAction;
	ERenderTargetLoadAction											StencilTargetLoadAction = ERenderTargetLoadAction::ENoAction;
	ERenderTargetStoreAction										StencilTargetStoreAction = ERenderTargetStoreAction::ENoAction;
	FExclusiveDepthStencil											DepthStencilAccess;
	uint16															NumSamples = 0;
	uint8															MultiViewCount = 0;
	bool															bHasFragmentDensityAttachment = false;
};


class FGraphicsPipelineStateInitializer
{
public:
	// Can't use TEnumByte<EPixelFormat> as it changes the struct to be non trivially constructible, breaking memset
	using TRenderTargetFormats		= TStaticArray<uint8/*EPixelFormat*/, MaxSimultaneousRenderTargets>;
	using TRenderTargetFlags		= TStaticArray<ETextureCreateFlags, MaxSimultaneousRenderTargets>;

	FGraphicsPipelineStateInitializer()
		: BlendState(nullptr)
		, RasterizerState(nullptr)
		, DepthStencilState(nullptr)
		, RenderTargetsEnabled(0)
		, RenderTargetFormats(InPlace, UE_PIXELFORMAT_TO_UINT8(PF_Unknown))
		, RenderTargetFlags(InPlace, TexCreate_None)
		, DepthStencilTargetFormat(PF_Unknown)
		, DepthStencilTargetFlag(TexCreate_None)
		, DepthTargetLoadAction(ERenderTargetLoadAction::ENoAction)
		, DepthTargetStoreAction(ERenderTargetStoreAction::ENoAction)
		, StencilTargetLoadAction(ERenderTargetLoadAction::ENoAction)
		, StencilTargetStoreAction(ERenderTargetStoreAction::ENoAction)
		, NumSamples(0)
		, SubpassHint(ESubpassHint::None)
		, SubpassIndex(0)
		, ConservativeRasterization(EConservativeRasterization::Disabled)
		, bDepthBounds(false)
		, MultiViewCount(0)
		, bHasFragmentDensityAttachment(false)
		, bAllowVariableRateShading(true)
		, ShadingRate(EVRSShadingRate::VRSSR_1x1)
		, Flags(0)
		, StatePrecachePSOHash(0)
	{
#if PLATFORM_WINDOWS
		static_assert(sizeof(TRenderTargetFormats::ElementType) == sizeof(uint8/*EPixelFormat*/), "Change TRenderTargetFormats's uint8 to EPixelFormat's size!");
#endif
		static_assert(PF_MAX < MAX_uint8, "TRenderTargetFormats assumes EPixelFormat can fit in a uint8!");
	}

	FGraphicsPipelineStateInitializer(
		FBoundShaderStateInput		InBoundShaderState,
		FRHIBlendState*				InBlendState,
		FRHIRasterizerState*		InRasterizerState,
		FRHIDepthStencilState*		InDepthStencilState,
		EPrimitiveType				InPrimitiveType,
		uint32						InRenderTargetsEnabled,
		const TRenderTargetFormats&	InRenderTargetFormats,
		const TRenderTargetFlags&	InRenderTargetFlags,
		EPixelFormat				InDepthStencilTargetFormat,
		ETextureCreateFlags			InDepthStencilTargetFlag,
		ERenderTargetLoadAction		InDepthTargetLoadAction,
		ERenderTargetStoreAction	InDepthTargetStoreAction,
		ERenderTargetLoadAction		InStencilTargetLoadAction,
		ERenderTargetStoreAction	InStencilTargetStoreAction,
		FExclusiveDepthStencil		InDepthStencilAccess,
		uint16						InNumSamples,
		ESubpassHint				InSubpassHint,
		uint8						InSubpassIndex,
		EConservativeRasterization	InConservativeRasterization,
		uint16						InFlags,
		bool						bInDepthBounds,
		uint8						InMultiViewCount,
		bool						bInHasFragmentDensityAttachment,
		bool						bInAllowVariableRateShading,
		EVRSShadingRate				InShadingRate)
		: BoundShaderState(InBoundShaderState)
		, BlendState(InBlendState)
		, RasterizerState(InRasterizerState)
		, DepthStencilState(InDepthStencilState)
		, PrimitiveType(InPrimitiveType)
		, RenderTargetsEnabled(InRenderTargetsEnabled)
		, RenderTargetFormats(InRenderTargetFormats)
		, RenderTargetFlags(InRenderTargetFlags)
		, DepthStencilTargetFormat(InDepthStencilTargetFormat)
		, DepthStencilTargetFlag(InDepthStencilTargetFlag)
		, DepthTargetLoadAction(InDepthTargetLoadAction)
		, DepthTargetStoreAction(InDepthTargetStoreAction)
		, StencilTargetLoadAction(InStencilTargetLoadAction)
		, StencilTargetStoreAction(InStencilTargetStoreAction)
		, DepthStencilAccess(InDepthStencilAccess)
		, NumSamples(InNumSamples)
		, SubpassHint(InSubpassHint)
		, SubpassIndex(InSubpassIndex)
		, ConservativeRasterization(EConservativeRasterization::Disabled)
		, bDepthBounds(bInDepthBounds)
		, MultiViewCount(InMultiViewCount)
		, bHasFragmentDensityAttachment(bInHasFragmentDensityAttachment)
		, bAllowVariableRateShading(bInAllowVariableRateShading)
		, ShadingRate(InShadingRate)
		, Flags(InFlags)
		, StatePrecachePSOHash(0)
	{
	}

	bool operator==(const FGraphicsPipelineStateInitializer& rhs) const
	{
		if (BoundShaderState.VertexDeclarationRHI != rhs.BoundShaderState.VertexDeclarationRHI ||
			BoundShaderState.VertexShaderRHI != rhs.BoundShaderState.VertexShaderRHI ||
			BoundShaderState.PixelShaderRHI != rhs.BoundShaderState.PixelShaderRHI ||
			BoundShaderState.GetMeshShader() != rhs.BoundShaderState.GetMeshShader() ||
			BoundShaderState.GetAmplificationShader() != rhs.BoundShaderState.GetAmplificationShader() ||
			BoundShaderState.GetWorkGraphShader() != rhs.BoundShaderState.GetWorkGraphShader() ||
			BoundShaderState.GetGeometryShader() != rhs.BoundShaderState.GetGeometryShader() ||
			BlendState != rhs.BlendState ||
			RasterizerState != rhs.RasterizerState ||
			DepthStencilState != rhs.DepthStencilState ||
			PrimitiveType != rhs.PrimitiveType ||
			bDepthBounds != rhs.bDepthBounds ||
			MultiViewCount != rhs.MultiViewCount ||
			ShadingRate != rhs.ShadingRate ||
			bAllowVariableRateShading != rhs.bAllowVariableRateShading ||
			bHasFragmentDensityAttachment != rhs.bHasFragmentDensityAttachment ||
			RenderTargetsEnabled != rhs.RenderTargetsEnabled ||
			RenderTargetFormats != rhs.RenderTargetFormats || 
			!RelevantRenderTargetFlagsEqual(RenderTargetFlags, rhs.RenderTargetFlags) || 
			DepthStencilTargetFormat != rhs.DepthStencilTargetFormat || 
			!RelevantDepthStencilFlagsEqual(DepthStencilTargetFlag, rhs.DepthStencilTargetFlag) ||
			DepthTargetLoadAction != rhs.DepthTargetLoadAction ||
			DepthTargetStoreAction != rhs.DepthTargetStoreAction ||
			StencilTargetLoadAction != rhs.StencilTargetLoadAction ||
			StencilTargetStoreAction != rhs.StencilTargetStoreAction || 
			DepthStencilAccess != rhs.DepthStencilAccess ||
			NumSamples != rhs.NumSamples ||
			SubpassHint != rhs.SubpassHint ||
			SubpassIndex != rhs.SubpassIndex ||
			ConservativeRasterization != rhs.ConservativeRasterization)
		{
			return false;
		}

		return true;
	}

	// We care about flags that influence RT formats (which is the only thing the underlying API cares about).
	// In most RHIs, the format is only influenced by TexCreate_SRGB. D3D12 additionally uses TexCreate_Shared in its format selection logic.
	static constexpr ETextureCreateFlags RelevantRenderTargetFlagMask = ETextureCreateFlags::SRGB | ETextureCreateFlags::Shared;

	// We care about flags that influence DS formats (which is the only thing the underlying API cares about).
	// D3D12 shares the format choice function with the RT, so preserving all the flags used there out of abundance of caution.
	static constexpr ETextureCreateFlags RelevantDepthStencilFlagMask = ETextureCreateFlags::SRGB | ETextureCreateFlags::Shared | ETextureCreateFlags::DepthStencilTargetable;

	static bool RelevantRenderTargetFlagsEqual(const TRenderTargetFlags& A, const TRenderTargetFlags& B)
	{
		for (int32 Index = 0; Index < A.Num(); ++Index)
		{
			ETextureCreateFlags FlagsA = A[Index] & RelevantRenderTargetFlagMask;
			ETextureCreateFlags FlagsB = B[Index] & RelevantRenderTargetFlagMask;
			if (FlagsA != FlagsB)
			{
				return false;
			}
		}
		return true;
	}

	static bool RelevantDepthStencilFlagsEqual(const ETextureCreateFlags A, const ETextureCreateFlags B)
	{
		ETextureCreateFlags FlagsA = (A & RelevantDepthStencilFlagMask);
		ETextureCreateFlags FlagsB = (B & RelevantDepthStencilFlagMask);
		return (FlagsA == FlagsB);
	}

	uint32 ComputeNumValidRenderTargets() const
	{
		// Get the count of valid render targets (ignore those at the end of the array with PF_Unknown)
		if (RenderTargetsEnabled > 0)
		{
			int32 LastValidTarget = -1;
			for (int32 i = (int32)RenderTargetsEnabled - 1; i >= 0; i--)
			{
				if (RenderTargetFormats[i] != PF_Unknown)
				{
					LastValidTarget = i;
					break;
				}
			}
			return uint32(LastValidTarget + 1);
		}
		return RenderTargetsEnabled;
	}

	FBoundShaderStateInput			BoundShaderState;
	FRHIBlendState*					BlendState;
	FRHIRasterizerState*			RasterizerState;
	FRHIDepthStencilState*			DepthStencilState;

	EPrimitiveType					PrimitiveType;
	uint32							RenderTargetsEnabled;
	TRenderTargetFormats			RenderTargetFormats;
	TRenderTargetFlags				RenderTargetFlags;
	EPixelFormat					DepthStencilTargetFormat;
	ETextureCreateFlags				DepthStencilTargetFlag;
	ERenderTargetLoadAction			DepthTargetLoadAction;
	ERenderTargetStoreAction		DepthTargetStoreAction;
	ERenderTargetLoadAction			StencilTargetLoadAction;
	ERenderTargetStoreAction		StencilTargetStoreAction;
	FExclusiveDepthStencil			DepthStencilAccess;
	uint16							NumSamples;
	ESubpassHint					SubpassHint;
	uint8							SubpassIndex;
	EConservativeRasterization		ConservativeRasterization;
	bool							bDepthBounds;
	uint8							MultiViewCount;
	bool							bHasFragmentDensityAttachment;
	bool							bAllowVariableRateShading;
	EVRSShadingRate					ShadingRate;
	
	// Note: these flags do NOT affect compilation of this PSO.
	// The resulting object is invariant with respect to whatever is set here, they are
	// behavior hints.
	// They do not participate in equality comparisons or hashing.
	union
	{
		struct
		{
			uint16					Reserved			: 11;
			uint16					bPSOPrecache		: 1;
			uint16					bFromPSOFileCache	: 1;
			uint16					PrecacheCompileType : 3;
		};
		uint16						Flags;
	};

	enum class EPSOPrecacheCompileType : uint8
	{
		NotSet = 0,
		MinPri = 1,
		NormalPri = 2,
		MaxPri = 3,

		NumTypes = 4,
	};
	static_assert((int)EPSOPrecacheCompileType::MaxPri < (1<<3) ); // ensure MaxPri fits within PrecacheCompileType
	void SetPSOPrecacheCompileType(EPSOPrecacheCompileType PrecacheCompileTypeIN) 
	{ 
		check(PrecacheCompileTypeIN <= EPSOPrecacheCompileType::MaxPri && PrecacheCompileTypeIN >= EPSOPrecacheCompileType::MinPri);
		PrecacheCompileType = (uint16)PrecacheCompileTypeIN;
	}
	EPSOPrecacheCompileType GetPSOPrecacheCompileType() const {	return (EPSOPrecacheCompileType)PrecacheCompileType; }

	// Cached hash off all state data provided at creation time (Only contains hash of data which influences the PSO precaching for the current platform)
	// Created from hashing the state data instead of the pointers which are used during fast runtime cache checking and compares
	uint64							StatePrecachePSOHash;
};

class FComputePipelineStateInitializer
{
public:

	FComputePipelineStateInitializer()
		: ComputeShader(nullptr)
		, Flags(0)
	{
	}

	FComputePipelineStateInitializer(FRHIComputeShader* InComputeShader)
		: ComputeShader(InComputeShader)
		, Flags(0)
	{
		checkf(!InComputeShader || !InComputeShader->HasSpecializationConstants(), TEXT("Shader requires specialization constants to be passed in using FComputePipelineStateInitializer."));
	}

	FComputePipelineStateInitializer(
		FRHIComputeShader* InComputeShader,
		const FSpecializationContainerType& InSpecializationConstants)
		: ComputeShader(InComputeShader)
		, SpecializationConstants(InSpecializationConstants)
		, Flags(0)
	{
	}

	FComputePipelineStateInitializer(
		FRHIComputeShader* InComputeShader,
		const FSpecializationContainerType& InSpecializationConstants,
		uint8 InFlags)
		: ComputeShader(InComputeShader)
		, SpecializationConstants(InSpecializationConstants)
		, Flags(InFlags)
	{
	}

	bool operator==(const FComputePipelineStateInitializer& Other) const
	{
		return (ComputeShader == Other.ComputeShader) && 
			(SpecializationConstants == Other.SpecializationConstants);
	}

	FRHIComputeShader* ComputeShader;
	FSpecializationContainerType SpecializationConstants;

	// Note: these flags do NOT affect compilation of this PSO.
	// The resulting object is invariant with respect to whatever is set here, they are
	// behavior hints.
	// They do not participate in equality comparisons or hashing.
	union
	{
		struct
		{
			uint8	Reserved : 6;
			uint8	bPSOPrecache : 1;
			uint8	bFromPSOFileCache : 1;
		};
		uint8		Flags;
	};
};

/** Helper for fast compute of hash for a shader table. */
template<typename TShaderType>
inline uint64 ComputeShaderTableHash(const TArrayView<TShaderType*>& ShaderTable, uint64 InitialHash = 5699878132332235837ull)
{
	uint64 CombinedHash = InitialHash;
	for (FRHIShader* ShaderRHI : ShaderTable)
	{
		uint64 ShaderHash = 0;
		if (ShaderRHI)
		{
			ShaderHash = ShaderRHI->GetHash().Hash;
		}

		// 64 bit hash combination as per boost::hash_combine_impl
		CombinedHash ^= ShaderHash + 0x9e3779b9 + (CombinedHash << 6) + (CombinedHash >> 2);
	}

	return CombinedHash;
}

class FWorkGraphPipelineStateSignature
{
public:
	FWorkGraphPipelineStateSignature() = default;

	bool operator==(const FWorkGraphPipelineStateSignature& Rhs) const
	{
		return NameHash == Rhs.NameHash && 
			NameTableHash == Rhs.NameTableHash && 
			ShaderTableHash == Rhs.ShaderTableHash &&
			GraphicsPSOTableHash == Rhs.GraphicsPSOTableHash;
	}

	friend uint32 GetTypeHash(const FWorkGraphPipelineStateSignature& Initializer)
	{
		return GetTypeHash(Initializer.NameHash) ^ 
			GetTypeHash(Initializer.NameTableHash) ^ 
			GetTypeHash(Initializer.ShaderTableHash) ^
			GetTypeHash(Initializer.GraphicsPSOTableHash);
	}

	uint64 GetNameHash() const { return NameHash; }
	uint64 GetNameTableHash() const { return NameTableHash; }
	uint64 GetShaderTableHash() const { return ShaderTableHash; }
	uint64 GetGraphicsPSOTableHash() const { return GraphicsPSOTableHash; }

protected:
	uint64 NameHash = 0;
	uint64 NameTableHash = 0;
	uint64 ShaderTableHash = 0;
	uint64 GraphicsPSOTableHash = 0;
};

class FWorkGraphPipelineStateInitializer : public FWorkGraphPipelineStateSignature
{
public:
	FWorkGraphPipelineStateInitializer() = default;

	void SetProgramName(TCHAR const* InProgramName)
	{
		ProgramName = InProgramName;
		NameHash = GetTypeHash(ProgramName);
	}

	/** A single mapping of shader export name to work graph node name. An array of these defines the full mapping. */
	struct FNameMap
	{
		FString ExportName;
		FString NodeName;
		uint32 ExportNameHash;

		FNameMap(FString const& InExportName, FString const& InNodeName)
			: ExportName(InExportName)
			, NodeName(InNodeName)
			, ExportNameHash(GetTypeHash(InExportName))
		{}

		friend uint32 GetTypeHash(FNameMap const& NameMap)
		{
			return HashCombineFast(GetTypeHash(NameMap.ExportName), GetTypeHash(NameMap.NodeName));
		}
	};

	void SetNameTable(const TArrayView<FNameMap> InNameMaps, uint64 Hash = 0)
	{
		NameMaps = InNameMaps;
		NameTableHash = Hash ? Hash : GetArrayHash(NameMaps.GetData(), NameMaps.Num());
	}

	void SetShaderTable(const TArrayView<FRHIWorkGraphShader*>& InShaders, int32 InRootShaderIndex = -1, uint64 Hash = 0)
	{
		ShaderTable = InShaders;
		ShaderTableHash = Hash ? Hash : ComputeShaderTableHash(InShaders);
		// RootShaderIndex doesn't need adding to a hash because if used correctly it is an artifact of the shader table.
		// (Only one shader in the array can have a global root signature).
		RootShaderIndex = InRootShaderIndex;
	}

	void SetGraphicsPSOTable(const TArrayView<FGraphicsPipelineStateInitializer const*>& InGraphicsPSOs, uint64 Hash = 0)
	{
		GraphicsPSOTable = InGraphicsPSOs;
		GraphicsPSOTableHash = Hash ? Hash : ComputeGraphicsPSOTableHash(InGraphicsPSOs);
	}

	FString const& GetProgramName() const { return ProgramName; }
	TArray<FNameMap> const& GetNameTable() const { return NameMaps; }
	int32 GetRootShaderIndex() const { return RootShaderIndex; }
	TArrayView<FRHIWorkGraphShader*> const& GetShaderTable() const { return ShaderTable; }
	TArrayView<FGraphicsPipelineStateInitializer const*> const& GetGraphicsPSOTable() const { return GraphicsPSOTable; }

private:
	RHI_API uint64 ComputeGraphicsPSOTableHash(const TArrayView<FGraphicsPipelineStateInitializer const*>& InGraphicsPSOTable, uint64 InitialHash = 5699878132332235837ull);

	FString ProgramName;
	TArray<FNameMap> NameMaps;
	int32 RootShaderIndex = -1;
	TArrayView<FRHIWorkGraphShader*> ShaderTable;
	TArrayView<FGraphicsPipelineStateInitializer const*> GraphicsPSOTable;
};

class FRayTracingPipelineStateSignature
{
public:

	uint32 MaxAttributeSizeInBytes = 8; // sizeof FRayTracingIntersectionAttributes declared in RayTracingCommon.ush
	uint32 MaxPayloadSizeInBytes = 24; // sizeof FDefaultPayload declared in RayTracingCommon.ush

	// NOTE: GetTypeHash(const FRayTracingPipelineStateInitializer& Initializer) should also be updated when changing this function
	bool operator==(const FRayTracingPipelineStateSignature& rhs) const
	{
		return MaxAttributeSizeInBytes == rhs.MaxAttributeSizeInBytes
			&& MaxPayloadSizeInBytes == rhs.MaxPayloadSizeInBytes
			&& RayGenHash == rhs.RayGenHash
			&& MissHash == rhs.MissHash
			&& HitGroupHash == rhs.HitGroupHash
			&& CallableHash == rhs.CallableHash;
	}

	friend uint32 GetTypeHash(const FRayTracingPipelineStateSignature& Initializer)
	{
		return GetTypeHash(Initializer.MaxAttributeSizeInBytes) ^
			GetTypeHash(Initializer.MaxPayloadSizeInBytes) ^
			GetTypeHash(Initializer.GetRayGenHash()) ^
			GetTypeHash(Initializer.GetRayMissHash()) ^
			GetTypeHash(Initializer.GetHitGroupHash()) ^
			GetTypeHash(Initializer.GetCallableHash());
	}

	uint64 GetHitGroupHash() const { return HitGroupHash; }
	uint64 GetRayGenHash()   const { return RayGenHash; }
	uint64 GetRayMissHash()  const { return MissHash; }
	uint64 GetCallableHash() const { return CallableHash; }

protected:

	uint64 RayGenHash = 0;
	uint64 MissHash = 0;
	uint64 HitGroupHash = 0;
	uint64 CallableHash = 0;
};

class FRayTracingPipelineStateInitializer : public FRayTracingPipelineStateSignature
{
public:

	FRayTracingPipelineStateInitializer() = default;

	// Partial ray tracing pipelines can be used for run-time asynchronous shader compilation, but not for rendering.
	// Any number of shaders for any stage may be provided when creating partial pipelines, but 
	// at least one shader must be present in total (completely empty pipelines are not allowed).
	bool bPartial = false;

	// Hints to the RHI that this PSO is being compiled by a background task and will not be needed immediately for rendering.
	// Speculative PSO pre-caching or non-blocking PSO creation should set this flag.
	// This may be used by the RHI to decide if a hitch warning should be reported, change priority of any internally dispatched tasks, etc.
	// Does not affect the creation of the PSO itself.
	bool bBackgroundCompilation = false;

	// Ray tracing pipeline may be created by deriving from the existing base.
	// Base pipeline will be extended by adding new shaders into it, potentially saving substantial amount of CPU time.
	// Depends on GRHISupportsRayTracingPSOAdditions support at runtime (base pipeline is simply ignored if it is unsupported).
	FRayTracingPipelineStateRHIRef BasePipeline;

	// Shader binding table layout used during shader compilation which needs to be the same for all shaders in the RTPSO and defines
	// how uniform buffers needs to be bound at runtime (global(RayGen) vs local(miss/hit/callable) data)
	const FRHIShaderBindingLayout* ShaderBindingLayout = nullptr;

	const TArrayView<FRHIRayTracingShader*>& GetRayGenTable()   const { return RayGenTable; }
	const TArrayView<FRHIRayTracingShader*>& GetMissTable()     const { return MissTable; }
	const TArrayView<FRHIRayTracingShader*>& GetHitGroupTable() const { return HitGroupTable; }
	const TArrayView<FRHIRayTracingShader*>& GetCallableTable() const { return CallableTable; }

	// Shaders used as entry point to ray tracing work. At least one RayGen shader must be provided.
	void SetRayGenShaderTable(const TArrayView<FRHIRayTracingShader*>& InRayGenShaders, uint64 Hash = 0)
	{
		RayGenTable = InRayGenShaders;
		RayGenHash = Hash ? Hash : ComputeShaderTableHash(InRayGenShaders);
	}

	// Shaders that will be invoked if a ray misses all geometry.
	// If this table is empty, then a built-in default miss shader will be used that sets HitT member of FMinimalPayload to -1.
	// Desired miss shader can be selected by providing MissShaderIndex to TraceRay() function.
	void SetMissShaderTable(const TArrayView<FRHIRayTracingShader*>& InMissShaders, uint64 Hash = 0)
	{
		MissTable = InMissShaders;
		MissHash = Hash ? Hash : ComputeShaderTableHash(InMissShaders);
	}

	// Shaders that will be invoked when ray intersects geometry.
	// If this table is empty, then a built-in default shader will be used for all geometry, using FDefaultPayload.
	void SetHitGroupTable(const TArrayView<FRHIRayTracingShader*>& InHitGroups, uint64 Hash = 0)
	{
		HitGroupTable = InHitGroups;
		HitGroupHash = Hash ? Hash : ComputeShaderTableHash(HitGroupTable);
	}

	// Shaders that can be explicitly invoked from RayGen shaders by their Shader Binding Table (SBT) index.
	// SetRayTracingCallableShader() command must be used to fill SBT slots before a shader can be called.
	void SetCallableTable(const TArrayView<FRHIRayTracingShader*>& InCallableShaders, uint64 Hash = 0)
	{
		CallableTable = InCallableShaders;
		CallableHash = Hash ? Hash : ComputeShaderTableHash(CallableTable);
	}

	// Retrieve the max local binding size of all the raytracing shaders used in the RTPSO
	RHI_API uint32 GetMaxLocalBindingDataSize() const;

private:
	TArrayView<FRHIRayTracingShader*> RayGenTable;
	TArrayView<FRHIRayTracingShader*> MissTable;
	TArrayView<FRHIRayTracingShader*> HitGroupTable;
	TArrayView<FRHIRayTracingShader*> CallableTable;
};

// This PSO is used as a fallback for RHIs that dont support PSOs. It is used to set the graphics state using the legacy state setting APIs
class FRHIGraphicsPipelineStateFallBack : public FRHIGraphicsPipelineState
{
public:
	FRHIGraphicsPipelineStateFallBack() {}

	FRHIGraphicsPipelineStateFallBack(const FGraphicsPipelineStateInitializer& Init)
		: Initializer(Init)
	{
	}

	FRHIGraphicsShader* GetShader(EShaderFrequency Frequency) const override
	{
		switch (Frequency)
		{
		case SF_Vertex: return Initializer.BoundShaderState.GetVertexShader();
		case SF_Mesh: return Initializer.BoundShaderState.GetMeshShader();
		case SF_Amplification: return Initializer.BoundShaderState.GetAmplificationShader();
		case SF_Pixel: return Initializer.BoundShaderState.GetPixelShader();
		case SF_Geometry: return Initializer.BoundShaderState.GetGeometryShader();
		default: return nullptr;
		}
	}

	FGraphicsPipelineStateInitializer Initializer;
};

class FRHIComputePipelineStateFallback : public FRHIComputePipelineState
{
public:
	FRHIComputePipelineStateFallback(FRHIComputeShader* InComputeShader) : FRHIComputePipelineState(InComputeShader)
	{
	}
};

enum class ERenderTargetActions : uint8
{
	LoadOpMask = 2,

#define RTACTION_MAKE_MASK(Load, Store) (((uint8)ERenderTargetLoadAction::Load << (uint8)LoadOpMask) | (uint8)ERenderTargetStoreAction::Store)

	DontLoad_DontStore =	RTACTION_MAKE_MASK(ENoAction, ENoAction),

	DontLoad_Store =		RTACTION_MAKE_MASK(ENoAction, EStore),
	Clear_Store =			RTACTION_MAKE_MASK(EClear, EStore),
	Load_Store =			RTACTION_MAKE_MASK(ELoad, EStore),

	Clear_DontStore =		RTACTION_MAKE_MASK(EClear, ENoAction),
	Load_DontStore =		RTACTION_MAKE_MASK(ELoad, ENoAction),
	Clear_Resolve =			RTACTION_MAKE_MASK(EClear, EMultisampleResolve),
	Load_Resolve =			RTACTION_MAKE_MASK(ELoad, EMultisampleResolve),

#undef RTACTION_MAKE_MASK
};

inline ERenderTargetActions MakeRenderTargetActions(ERenderTargetLoadAction Load, ERenderTargetStoreAction Store)
{
	return (ERenderTargetActions)(((uint8)Load << (uint8)ERenderTargetActions::LoadOpMask) | (uint8)Store);
}

inline ERenderTargetLoadAction GetLoadAction(ERenderTargetActions Action)
{
	return (ERenderTargetLoadAction)((uint8)Action >> (uint8)ERenderTargetActions::LoadOpMask);
}

inline ERenderTargetStoreAction GetStoreAction(ERenderTargetActions Action)
{
	return (ERenderTargetStoreAction)((uint8)Action & ((1 << (uint8)ERenderTargetActions::LoadOpMask) - 1));
}

enum class EDepthStencilTargetActions : uint8
{
	DepthMask = 4,

#define RTACTION_MAKE_MASK(Depth, Stencil) (((uint8)ERenderTargetActions::Depth << (uint8)DepthMask) | (uint8)ERenderTargetActions::Stencil)

	DontLoad_DontStore =						RTACTION_MAKE_MASK(DontLoad_DontStore, DontLoad_DontStore),
	DontLoad_StoreDepthStencil =				RTACTION_MAKE_MASK(DontLoad_Store, DontLoad_Store),
	DontLoad_StoreStencilNotDepth =				RTACTION_MAKE_MASK(DontLoad_DontStore, DontLoad_Store),
	ClearDepthStencil_StoreDepthStencil =		RTACTION_MAKE_MASK(Clear_Store, Clear_Store),
	LoadDepthStencil_StoreDepthStencil =		RTACTION_MAKE_MASK(Load_Store, Load_Store),
	LoadDepthNotStencil_StoreDepthNotStencil =	RTACTION_MAKE_MASK(Load_Store, DontLoad_DontStore),
	LoadDepthNotStencil_DontStore =				RTACTION_MAKE_MASK(Load_DontStore, DontLoad_DontStore),
	LoadDepthStencil_StoreStencilNotDepth =		RTACTION_MAKE_MASK(Load_DontStore, Load_Store),

	ClearDepthStencil_DontStoreDepthStencil =	RTACTION_MAKE_MASK(Clear_DontStore, Clear_DontStore),
	LoadDepthStencil_DontStoreDepthStencil =	RTACTION_MAKE_MASK(Load_DontStore, Load_DontStore),
	ClearDepthStencil_StoreDepthNotStencil =	RTACTION_MAKE_MASK(Clear_Store, Clear_DontStore),
	ClearDepthStencil_StoreStencilNotDepth =	RTACTION_MAKE_MASK(Clear_DontStore, Clear_Store),
	ClearDepthStencil_ResolveDepthNotStencil =	RTACTION_MAKE_MASK(Clear_Resolve, Clear_DontStore),
	ClearDepthStencil_ResolveStencilNotDepth =	RTACTION_MAKE_MASK(Clear_DontStore, Clear_Resolve),
	LoadDepthClearStencil_StoreDepthStencil  =  RTACTION_MAKE_MASK(Load_Store, Clear_Store),

	ClearStencilDontLoadDepth_StoreStencilNotDepth = RTACTION_MAKE_MASK(DontLoad_DontStore, Clear_Store),

#undef RTACTION_MAKE_MASK
};

inline constexpr EDepthStencilTargetActions MakeDepthStencilTargetActions(const ERenderTargetActions Depth, const ERenderTargetActions Stencil)
{
	return (EDepthStencilTargetActions)(((uint8)Depth << (uint8)EDepthStencilTargetActions::DepthMask) | (uint8)Stencil);
}

inline ERenderTargetActions GetDepthActions(EDepthStencilTargetActions Action)
{
	return (ERenderTargetActions)((uint8)Action >> (uint8)EDepthStencilTargetActions::DepthMask);
}

inline ERenderTargetActions GetStencilActions(EDepthStencilTargetActions Action)
{
	return (ERenderTargetActions)((uint8)Action & ((1 << (uint8)EDepthStencilTargetActions::DepthMask) - 1));
}

struct FResolveRect
{
	int32 X1;
	int32 Y1;
	int32 X2;
	int32 Y2;

	// e.g. for a a full 256 x 256 area starting at (0, 0) it would be 
	// the values would be 0, 0, 256, 256
	FResolveRect(int32 InX1 = -1, int32 InY1 = -1, int32 InX2 = -1, int32 InY2 = -1)
		: X1(InX1)
		, Y1(InY1)
		, X2(InX2)
		, Y2(InY2)
	{}

	explicit FResolveRect(FIntRect Other)
		: X1(Other.Min.X)
		, Y1(Other.Min.Y)
		, X2(Other.Max.X)
		, Y2(Other.Max.Y)
	{}

	bool operator==(FResolveRect Other) const
	{
		return X1 == Other.X1 && Y1 == Other.Y1 && X2 == Other.X2 && Y2 == Other.Y2;
	}

	bool operator!=(FResolveRect Other) const
	{
		return !(*this == Other);
	}

	bool IsValid() const
	{
		return X1 >= 0 && Y1 >= 0 && X2 - X1 > 0 && Y2 - Y1 > 0;
	}
};

struct FRHIRenderPassInfo
{
	struct FColorEntry
	{
		FRHITexture*         RenderTarget      = nullptr;
		FRHITexture*         ResolveTarget     = nullptr;
		int32                ArraySlice        = -1;
		uint8                MipIndex          = 0;
		ERenderTargetActions Action            = ERenderTargetActions::DontLoad_DontStore;
	};
	TStaticArray<FColorEntry, MaxSimultaneousRenderTargets> ColorRenderTargets;

	struct FDepthStencilEntry
	{
		FRHITexture*         DepthStencilTarget = nullptr;
		FRHITexture*         ResolveTarget      = nullptr;
		EDepthStencilTargetActions Action       = EDepthStencilTargetActions::DontLoad_DontStore;
		FExclusiveDepthStencil ExclusiveDepthStencil;
	};
	FDepthStencilEntry DepthStencilRenderTarget;

	// Controls the area for a multisample resolve or raster UAV (i.e. no fixed-function targets) operation.
	FResolveRect ResolveRect;

	// Some RHIs can use a texture to control the sampling and/or shading resolution of different areas 
	FTextureRHIRef ShadingRateTexture = nullptr;
	EVRSRateCombiner ShadingRateTextureCombiner = VRSRB_Passthrough;

	// Some RHIs need to know the layout of all planes in the depth target
	ERHIAccess NopAccess = ERHIAccess::Unknown;

	// Some RHIs require a hint that occlusion queries will be used in this render pass
	uint32 NumOcclusionQueries = 0;
	bool bOcclusionQueries = false;

	// if this renderpass should be multiview, and if so how many views are required
	uint8 MultiViewCount = 0;

	// Hint for some RHI's that renderpass will have specific sub-passes 
	ESubpassHint SubpassHint = ESubpassHint::None;

	FRHIRenderPassInfo() = default;
	FRHIRenderPassInfo(const FRHIRenderPassInfo&) = default;
	FRHIRenderPassInfo& operator=(const FRHIRenderPassInfo&) = default;

	// Color, no depth, optional resolve, optional mip, optional array slice
	explicit FRHIRenderPassInfo(FRHITexture* ColorRT, ERenderTargetActions ColorAction, FRHITexture* ResolveRT = nullptr, uint8 InMipIndex = 0, int32 InArraySlice = -1)
	{
		check(!(ResolveRT && ResolveRT->IsMultisampled()));
		check(ColorRT);
		ColorRenderTargets[0].RenderTarget = ColorRT;
		ColorRenderTargets[0].ResolveTarget = ResolveRT;
		ColorRenderTargets[0].ArraySlice = InArraySlice;
		ColorRenderTargets[0].MipIndex = InMipIndex;
		ColorRenderTargets[0].Action = ColorAction;
	}

	// Color MRTs, no depth
	explicit FRHIRenderPassInfo(int32 NumColorRTs, FRHITexture* ColorRTs[], ERenderTargetActions ColorAction)
	{
		check(NumColorRTs > 0);
		for (int32 Index = 0; Index < NumColorRTs; ++Index)
		{
			check(ColorRTs[Index]);
			ColorRenderTargets[Index].RenderTarget = ColorRTs[Index];
			ColorRenderTargets[Index].ArraySlice = -1;
			ColorRenderTargets[Index].Action = ColorAction;
		}
		DepthStencilRenderTarget.DepthStencilTarget = nullptr;
		DepthStencilRenderTarget.Action = EDepthStencilTargetActions::DontLoad_DontStore;
		DepthStencilRenderTarget.ExclusiveDepthStencil = FExclusiveDepthStencil::DepthNop_StencilNop;
		DepthStencilRenderTarget.ResolveTarget = nullptr;
	}

	// Color MRTs, no depth
	explicit FRHIRenderPassInfo(int32 NumColorRTs, FRHITexture* ColorRTs[], ERenderTargetActions ColorAction, FRHITexture* ResolveTargets[])
	{
		check(NumColorRTs > 0);
		for (int32 Index = 0; Index < NumColorRTs; ++Index)
		{
			check(ColorRTs[Index]);
			ColorRenderTargets[Index].RenderTarget = ColorRTs[Index];
			ColorRenderTargets[Index].ResolveTarget = ResolveTargets[Index];
			ColorRenderTargets[Index].ArraySlice = -1;
			ColorRenderTargets[Index].MipIndex = 0;
			ColorRenderTargets[Index].Action = ColorAction;
		}
		DepthStencilRenderTarget.DepthStencilTarget = nullptr;
		DepthStencilRenderTarget.Action = EDepthStencilTargetActions::DontLoad_DontStore;
		DepthStencilRenderTarget.ExclusiveDepthStencil = FExclusiveDepthStencil::DepthNop_StencilNop;
		DepthStencilRenderTarget.ResolveTarget = nullptr;
	}

	// Color MRTs and depth
	explicit FRHIRenderPassInfo(int32 NumColorRTs, FRHITexture* ColorRTs[], ERenderTargetActions ColorAction, FRHITexture* DepthRT, EDepthStencilTargetActions DepthActions, FExclusiveDepthStencil InEDS = FExclusiveDepthStencil::DepthWrite_StencilWrite)
	{
		check(NumColorRTs > 0);
		for (int32 Index = 0; Index < NumColorRTs; ++Index)
		{
			check(ColorRTs[Index]);
			ColorRenderTargets[Index].RenderTarget = ColorRTs[Index];
			ColorRenderTargets[Index].ResolveTarget = nullptr;
			ColorRenderTargets[Index].ArraySlice = -1;
			ColorRenderTargets[Index].MipIndex = 0;
			ColorRenderTargets[Index].Action = ColorAction;
		}
		check(DepthRT);
		DepthStencilRenderTarget.DepthStencilTarget = DepthRT;
		DepthStencilRenderTarget.ResolveTarget = nullptr;
		DepthStencilRenderTarget.Action = DepthActions;
		DepthStencilRenderTarget.ExclusiveDepthStencil = InEDS;
	}

	// Color MRTs and depth
	explicit FRHIRenderPassInfo(int32 NumColorRTs, FRHITexture* ColorRTs[], ERenderTargetActions ColorAction, FRHITexture* ResolveRTs[], FRHITexture* DepthRT, EDepthStencilTargetActions DepthActions, FRHITexture* ResolveDepthRT, FExclusiveDepthStencil InEDS = FExclusiveDepthStencil::DepthWrite_StencilWrite)
	{
		check(NumColorRTs > 0);
		for (int32 Index = 0; Index < NumColorRTs; ++Index)
		{
			check(!ResolveRTs[Index] || ResolveRTs[Index]->IsMultisampled());
			check(ColorRTs[Index]);
			ColorRenderTargets[Index].RenderTarget = ColorRTs[Index];
			ColorRenderTargets[Index].ResolveTarget = ResolveRTs[Index];
			ColorRenderTargets[Index].ArraySlice = -1;
			ColorRenderTargets[Index].MipIndex = 0;
			ColorRenderTargets[Index].Action = ColorAction;
		}
		check(!ResolveDepthRT || ResolveDepthRT->IsMultisampled());
		check(DepthRT);
		DepthStencilRenderTarget.DepthStencilTarget = DepthRT;
		DepthStencilRenderTarget.ResolveTarget = ResolveDepthRT;
		DepthStencilRenderTarget.Action = DepthActions;
		DepthStencilRenderTarget.ExclusiveDepthStencil = InEDS;
	}

	// Depth, no color
	explicit FRHIRenderPassInfo(FRHITexture* DepthRT, EDepthStencilTargetActions DepthActions, FRHITexture* ResolveDepthRT = nullptr, FExclusiveDepthStencil InEDS = FExclusiveDepthStencil::DepthWrite_StencilWrite)
	{
		check(!ResolveDepthRT || ResolveDepthRT->IsMultisampled());
		check(DepthRT);
		DepthStencilRenderTarget.DepthStencilTarget = DepthRT;
		DepthStencilRenderTarget.ResolveTarget = ResolveDepthRT;
		DepthStencilRenderTarget.Action = DepthActions;
		DepthStencilRenderTarget.ExclusiveDepthStencil = InEDS;
	}

	// Depth, no color, occlusion queries
	explicit FRHIRenderPassInfo(FRHITexture* DepthRT, uint32 InNumOcclusionQueries, EDepthStencilTargetActions DepthActions, FRHITexture* ResolveDepthRT = nullptr, FExclusiveDepthStencil InEDS = FExclusiveDepthStencil::DepthWrite_StencilWrite)
		: NumOcclusionQueries(InNumOcclusionQueries)
	{
		check(!ResolveDepthRT || ResolveDepthRT->IsMultisampled());
		check(DepthRT);
		DepthStencilRenderTarget.DepthStencilTarget = DepthRT;
		DepthStencilRenderTarget.ResolveTarget = ResolveDepthRT;
		DepthStencilRenderTarget.Action = DepthActions;
		DepthStencilRenderTarget.ExclusiveDepthStencil = InEDS;
	}

	// Color and depth
	explicit FRHIRenderPassInfo(FRHITexture* ColorRT, ERenderTargetActions ColorAction, FRHITexture* DepthRT, EDepthStencilTargetActions DepthActions, FExclusiveDepthStencil InEDS = FExclusiveDepthStencil::DepthWrite_StencilWrite)
	{
		check(ColorRT);
		ColorRenderTargets[0].RenderTarget = ColorRT;
		ColorRenderTargets[0].ResolveTarget = nullptr;
		ColorRenderTargets[0].ArraySlice = -1;
		ColorRenderTargets[0].MipIndex = 0;
		ColorRenderTargets[0].Action = ColorAction;
		check(DepthRT);
		DepthStencilRenderTarget.DepthStencilTarget = DepthRT;
		DepthStencilRenderTarget.ResolveTarget = nullptr;
		DepthStencilRenderTarget.Action = DepthActions;
		DepthStencilRenderTarget.ExclusiveDepthStencil = InEDS;
		FMemory::Memzero(&ColorRenderTargets[1], sizeof(FColorEntry) * (MaxSimultaneousRenderTargets - 1));
	}

	// Color and depth with resolve
	explicit FRHIRenderPassInfo(FRHITexture* ColorRT, ERenderTargetActions ColorAction, FRHITexture* ResolveColorRT,
		FRHITexture* DepthRT, EDepthStencilTargetActions DepthActions, FRHITexture* ResolveDepthRT, FExclusiveDepthStencil InEDS = FExclusiveDepthStencil::DepthWrite_StencilWrite)
	{
		check(!ResolveColorRT || ResolveColorRT->IsMultisampled());
		check(!ResolveDepthRT || ResolveDepthRT->IsMultisampled());
		check(ColorRT);
		ColorRenderTargets[0].RenderTarget = ColorRT;
		ColorRenderTargets[0].ResolveTarget = ResolveColorRT;
		ColorRenderTargets[0].ArraySlice = -1;
		ColorRenderTargets[0].MipIndex = 0;
		ColorRenderTargets[0].Action = ColorAction;
		check(DepthRT);
		DepthStencilRenderTarget.DepthStencilTarget = DepthRT;
		DepthStencilRenderTarget.ResolveTarget = ResolveDepthRT;
		DepthStencilRenderTarget.Action = DepthActions;
		DepthStencilRenderTarget.ExclusiveDepthStencil = InEDS;
		FMemory::Memzero(&ColorRenderTargets[1], sizeof(FColorEntry) * (MaxSimultaneousRenderTargets - 1));
	}

	// Color and depth with resolve and optional sample density
	explicit FRHIRenderPassInfo(FRHITexture* ColorRT, ERenderTargetActions ColorAction, FRHITexture* ResolveColorRT,
		FRHITexture* DepthRT, EDepthStencilTargetActions DepthActions, FRHITexture* ResolveDepthRT, 
		FRHITexture* InShadingRateTexture, EVRSRateCombiner InShadingRateTextureCombiner,
		FExclusiveDepthStencil InEDS = FExclusiveDepthStencil::DepthWrite_StencilWrite)
	{
		check(!ResolveColorRT || ResolveColorRT->IsMultisampled());
		check(!ResolveDepthRT || ResolveDepthRT->IsMultisampled());
		check(ColorRT);
		ColorRenderTargets[0].RenderTarget = ColorRT;
		ColorRenderTargets[0].ResolveTarget = ResolveColorRT;
		ColorRenderTargets[0].ArraySlice = -1;
		ColorRenderTargets[0].MipIndex = 0;
		ColorRenderTargets[0].Action = ColorAction;
		check(DepthRT);
		DepthStencilRenderTarget.DepthStencilTarget = DepthRT;
		DepthStencilRenderTarget.ResolveTarget = ResolveDepthRT;
		DepthStencilRenderTarget.Action = DepthActions;
		DepthStencilRenderTarget.ExclusiveDepthStencil = InEDS;
		ShadingRateTexture = InShadingRateTexture;
		ShadingRateTextureCombiner = InShadingRateTextureCombiner;
		FMemory::Memzero(&ColorRenderTargets[1], sizeof(FColorEntry) * (MaxSimultaneousRenderTargets - 1));
	}

	inline int32 GetNumColorRenderTargets() const
	{
		int32 ColorIndex = 0;
		for (; ColorIndex < MaxSimultaneousRenderTargets; ++ColorIndex)
		{
			const FColorEntry& Entry = ColorRenderTargets[ColorIndex];
			if (!Entry.RenderTarget)
			{
				break;
			}
		}

		return ColorIndex;
	}

	FGraphicsPipelineRenderTargetsInfo ExtractRenderTargetsInfo() const
	{
		FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;

		RenderTargetsInfo.NumSamples = 1;
		int32 RenderTargetIndex = 0;

		for (; RenderTargetIndex < MaxSimultaneousRenderTargets; ++RenderTargetIndex)
		{
			FRHITexture* RenderTarget = ColorRenderTargets[RenderTargetIndex].RenderTarget;
			if (!RenderTarget)
			{
				break;
			}

			RenderTargetsInfo.RenderTargetFormats[RenderTargetIndex] = (uint8)RenderTarget->GetFormat();
			RenderTargetsInfo.RenderTargetFlags[RenderTargetIndex] = RenderTarget->GetFlags();
			RenderTargetsInfo.NumSamples |= RenderTarget->GetNumSamples();
		}

		RenderTargetsInfo.RenderTargetsEnabled = RenderTargetIndex;
		for (; RenderTargetIndex < MaxSimultaneousRenderTargets; ++RenderTargetIndex)
		{
			RenderTargetsInfo.RenderTargetFormats[RenderTargetIndex] = PF_Unknown;
		}

		if (DepthStencilRenderTarget.DepthStencilTarget)
		{
			RenderTargetsInfo.DepthStencilTargetFormat = DepthStencilRenderTarget.DepthStencilTarget->GetFormat();
			RenderTargetsInfo.DepthStencilTargetFlag = DepthStencilRenderTarget.DepthStencilTarget->GetFlags();
			RenderTargetsInfo.NumSamples |= DepthStencilRenderTarget.DepthStencilTarget->GetNumSamples();
		}
		else
		{
			RenderTargetsInfo.DepthStencilTargetFormat = PF_Unknown;
		}

		const ERenderTargetActions DepthActions = GetDepthActions(DepthStencilRenderTarget.Action);
		const ERenderTargetActions StencilActions = GetStencilActions(DepthStencilRenderTarget.Action);
		RenderTargetsInfo.DepthTargetLoadAction = GetLoadAction(DepthActions);
		RenderTargetsInfo.DepthTargetStoreAction = GetStoreAction(DepthActions);
		RenderTargetsInfo.StencilTargetLoadAction = GetLoadAction(StencilActions);
		RenderTargetsInfo.StencilTargetStoreAction = GetStoreAction(StencilActions);
		RenderTargetsInfo.DepthStencilAccess = DepthStencilRenderTarget.ExclusiveDepthStencil;

		RenderTargetsInfo.MultiViewCount = MultiViewCount;
		RenderTargetsInfo.bHasFragmentDensityAttachment = ShadingRateTexture != nullptr;

		return RenderTargetsInfo;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	RHI_API void Validate() const;
#else
	void Validate() const {}
#endif
	RHI_API void ConvertToRenderTargetsInfo(FRHISetRenderTargetsInfo& OutRTInfo) const;
};

class FRHIContextArray : public TRHIPipelineArray<IRHIComputeContext*>
{
	using Base = TRHIPipelineArray<IRHIComputeContext*>;
public:
	using Base::Base;
};

struct FRHIParallelRenderPassInfo : public FRHIRenderPassInfo
{
	FRHIContextArray Contexts{ InPlace, nullptr };
	IRHIUploadContext* UploadContext = nullptr;
	const TCHAR* PassName;
	
	// Contains platform specific data
	void* RHIPlatformData = nullptr;

	FRHIParallelRenderPassInfo(FRHIRenderPassInfo&& Info, const TCHAR* InPassName)
		: FRHIRenderPassInfo(MoveTemp(Info)), PassName(InPassName)
	{}
};

//UE_DEPRECATED(5.3, "Use FRHITextureSRVCreateDesc::SetDisableSRGB to create a view which explicitly disables SRGB.")
enum ERHITextureSRVOverrideSRGBType : uint8
{
	SRGBO_Default,
	SRGBO_ForceDisable,
};

//UE_DEPRECATED(5.3, "Use FRHITextureSRVCreateDesc rather than FRHITextureSRVCreateInfo.")
struct FRHITextureSRVCreateInfo
{
	explicit FRHITextureSRVCreateInfo(uint8 InMipLevel = 0u, uint8 InNumMipLevels = 1u, EPixelFormat InFormat = PF_Unknown)
		: Format(InFormat)
		, MipLevel(InMipLevel)
		, NumMipLevels(InNumMipLevels)
		, SRGBOverride(SRGBO_Default)
		, FirstArraySlice(0)
		, NumArraySlices(0)
	{}

	explicit FRHITextureSRVCreateInfo(uint8 InMipLevel, uint8 InNumMipLevels, uint16 InFirstArraySlice, uint16 InNumArraySlices, EPixelFormat InFormat = PF_Unknown)
		: Format(InFormat)
		, MipLevel(InMipLevel)
		, NumMipLevels(InNumMipLevels)
		, SRGBOverride(SRGBO_Default)
		, FirstArraySlice(InFirstArraySlice)
		, NumArraySlices(InNumArraySlices)
	{}

	/** View the texture with a different format. Leave as PF_Unknown to use original format. Useful when sampling stencil */
	EPixelFormat Format;

	/** Specify the mip level to use. Useful when rendering to one mip while sampling from another */
	uint8 MipLevel;

	/** Create a view to a single, or multiple mip levels */
	uint8 NumMipLevels;

	/** Potentially override the texture's sRGB flag */
	ERHITextureSRVOverrideSRGBType SRGBOverride;

	/** Specify first array slice index. By default 0. */
	uint16 FirstArraySlice;

	/** Specify number of array slices. If FirstArraySlice and NumArraySlices are both zero, the SRV is created for all array slices. By default 0. */
	uint16 NumArraySlices;

	/** Specify the metadata plane to use when creating a view. */
	ERHITextureMetaDataAccess MetaData = ERHITextureMetaDataAccess::None;
    
    /** Specify a dimension to use which overrides the default  */
    TOptional<ETextureDimension> DimensionOverride;

	inline bool operator==(const FRHITextureSRVCreateInfo& Other)const
	{
		return (
			Format == Other.Format &&
			MipLevel == Other.MipLevel &&
			NumMipLevels == Other.NumMipLevels &&
			SRGBOverride == Other.SRGBOverride &&
			FirstArraySlice == Other.FirstArraySlice &&
			NumArraySlices == Other.NumArraySlices &&
			MetaData == Other.MetaData &&
            DimensionOverride == Other.DimensionOverride);
	}

	inline bool operator!=(const FRHITextureSRVCreateInfo& Other)const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FRHITextureSRVCreateInfo& Info)
	{
		uint32 Hash = uint32(Info.Format) | uint32(Info.MipLevel) << 8 | uint32(Info.NumMipLevels) << 16 | uint32(Info.SRGBOverride) << 24;
		Hash = HashCombine(Hash, uint32(Info.FirstArraySlice) | uint32(Info.NumArraySlices) << 16);
        Hash = HashCombine(Hash, Info.DimensionOverride.IsSet() ? uint32(*Info.DimensionOverride) : MAX_uint32);
		Hash = HashCombine(Hash, uint32(Info.MetaData));
		return Hash;
	}

	/** Check the validity. */
	static bool CheckValidity(const FRHITextureDesc& TextureDesc, const FRHITextureSRVCreateInfo& TextureSRVDesc, const TCHAR* TextureName)
	{
		return FRHITextureSRVCreateInfo::Validate(TextureDesc, TextureSRVDesc, TextureName, /* bFatal = */ true);
	}

protected:
	RHI_API static bool Validate(const FRHITextureDesc& TextureDesc, const FRHITextureSRVCreateInfo& TextureSRVDesc, const TCHAR* TextureName, bool bFatal);
};

struct FRHITextureUAVCreateInfo
{
public:
	FRHITextureUAVCreateInfo() = default;

	explicit FRHITextureUAVCreateInfo(uint8 InMipLevel, EPixelFormat InFormat = PF_Unknown, uint16 InFirstArraySlice = 0, uint16 InNumArraySlices = 0)
		: Format(InFormat)
		, MipLevel(InMipLevel)
		, FirstArraySlice(InFirstArraySlice)
		, NumArraySlices(InNumArraySlices)
	{}

	explicit FRHITextureUAVCreateInfo(ERHITextureMetaDataAccess InMetaData)
		: MetaData(InMetaData)
	{}

	inline bool operator==(const FRHITextureUAVCreateInfo& Other)const
	{
		return Format == Other.Format && MipLevel == Other.MipLevel && MetaData == Other.MetaData &&
                FirstArraySlice == Other.FirstArraySlice && NumArraySlices == Other.NumArraySlices &&
                DimensionOverride == Other.DimensionOverride;
	}

	inline bool operator!=(const FRHITextureUAVCreateInfo& Other)const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FRHITextureUAVCreateInfo& Info)
	{
		uint32 Hash = uint32(Info.Format) | uint32(Info.MipLevel) << 8 | uint32(Info.FirstArraySlice) << 16;
        Hash = HashCombine(Hash, Info.DimensionOverride.IsSet() ? uint32(*Info.DimensionOverride) : MAX_uint32);
		Hash = HashCombine(Hash, uint32(Info.NumArraySlices) | uint32(Info.MetaData) << 16);
		return Hash;
	}

	EPixelFormat Format = PF_Unknown;
	uint8 MipLevel = 0;
	uint16 FirstArraySlice = 0;
	uint16 NumArraySlices = 0;	// When 0, the default behavior will be used, e.g. all slices mapped.
	ERHITextureMetaDataAccess MetaData = ERHITextureMetaDataAccess::None;
    
    /** Specify a dimension to use which overrides the default  */
    TOptional<ETextureDimension> DimensionOverride;
};

/** Descriptor used to create a buffer resource */
using FRHIBufferCreateInfo = FRHIBufferDesc;

struct FRHIBufferSRVCreateInfo
{
	explicit FRHIBufferSRVCreateInfo() = default;

	explicit FRHIBufferSRVCreateInfo(EPixelFormat InFormat)
		: Format(InFormat)
	{}

	FRHIBufferSRVCreateInfo(uint32 InStartOffsetBytes, uint32 InNumElements)
		: StartOffsetBytes(InStartOffsetBytes)
		, NumElements(InNumElements)
	{}

	FRHIBufferSRVCreateInfo(FRHIRayTracingScene* InRayTracingScene, uint32 InStartOffsetBytes)
		: StartOffsetBytes(InStartOffsetBytes)
		, RayTracingScene(InRayTracingScene)
	{}

	inline bool operator==(const FRHIBufferSRVCreateInfo& Other)const
	{
		return Format == Other.Format
			&& StartOffsetBytes == Other.StartOffsetBytes
			&& NumElements == Other.NumElements
			&& RayTracingScene == Other.RayTracingScene;
	}

	inline bool operator!=(const FRHIBufferSRVCreateInfo& Other)const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FRHIBufferSRVCreateInfo& Desc)
	{
		return HashCombine(
			HashCombine(
				HashCombine(GetTypeHash(Desc.Format), GetTypeHash(Desc.StartOffsetBytes)),
				GetTypeHash(Desc.NumElements)),
			GetTypeHash(Desc.RayTracingScene)
		);
	}

	/** Encoding format for the element. */
	EPixelFormat Format = PF_Unknown;

	/** Offset in bytes from the beginning of buffer */
	uint32 StartOffsetBytes = 0;

	/** Number of elements (whole buffer by default) */
	uint32 NumElements = UINT32_MAX;

	/** Ray tracing scene associated with the SRV (if BUF_AccelerationStructure) */
	FRHIRayTracingScene* RayTracingScene = nullptr;
};

struct FRHIBufferUAVCreateInfo
{
	FRHIBufferUAVCreateInfo() = default;

	explicit FRHIBufferUAVCreateInfo(EPixelFormat InFormat)
		: Format(InFormat)
	{}

	inline bool operator==(const FRHIBufferUAVCreateInfo& Other)const
	{
		return Format == Other.Format && bSupportsAtomicCounter == Other.bSupportsAtomicCounter && bSupportsAppendBuffer == Other.bSupportsAppendBuffer;
	}

	inline bool operator!=(const FRHIBufferUAVCreateInfo& Other)const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FRHIBufferUAVCreateInfo& Info)
	{
		return uint32(Info.Format) | uint32(Info.bSupportsAtomicCounter) << 8 | uint32(Info.bSupportsAppendBuffer) << 16;
	}

	/** Number of bytes per element (used for typed buffers). */
	EPixelFormat Format = PF_Unknown;

	/** Whether the uav supports atomic counter or append buffer operations (used for structured buffers) */
	bool bSupportsAtomicCounter = false;
	bool bSupportsAppendBuffer = false;
};

class FRHITextureViewCache
{
public:
	// Finds a UAV matching the descriptor in the cache or creates a new one and updates the cache.
	RHI_API FRHIUnorderedAccessView* GetOrCreateUAV(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const FRHITextureUAVCreateInfo& CreateInfo);

	// Finds a SRV matching the descriptor in the cache or creates a new one and updates the cache.
	RHI_API FRHIShaderResourceView* GetOrCreateSRV(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const FRHITextureSRVCreateInfo& CreateInfo);

	// Sets the debug name of the RHI view resources.
#if RHI_USE_RESOURCE_DEBUG_NAME
	RHI_API void SetDebugName(FRHICommandListBase& RHICmdList, const TCHAR* DebugName);
#else
	void SetDebugName(FRHICommandListBase& RHICmdList, const TCHAR* DebugName) {}
#endif

private:
	TArray<TPair<FRHITextureUAVCreateInfo, FUnorderedAccessViewRHIRef>, TInlineAllocator<1>> UAVs;
	TArray<TPair<FRHITextureSRVCreateInfo, FShaderResourceViewRHIRef>, TInlineAllocator<1>> SRVs;
};

class FRHIBufferViewCache
{
public:
	// Finds a UAV matching the descriptor in the cache or creates a new one and updates the cache.
	RHI_API FRHIUnorderedAccessView* GetOrCreateUAV(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, const FRHIBufferUAVCreateInfo& CreateInfo);

	// Finds a SRV matching the descriptor in the cache or creates a new one and updates the cache.
	RHI_API FRHIShaderResourceView* GetOrCreateSRV(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, const FRHIBufferSRVCreateInfo& CreateInfo);

	// Sets the debug name of the RHI view resources.
#if RHI_USE_RESOURCE_DEBUG_NAME
	RHI_API void SetDebugName(FRHICommandListBase& RHICmdList, const TCHAR* DebugName);
#else
	void SetDebugName(FRHICommandListBase& RHICmdList, const TCHAR* DebugName) {}
#endif
	inline int32 NumItems() const
	{
		return UAVs.Num() + SRVs.Num();
	}

private:
	TArray<TPair<FRHIBufferUAVCreateInfo, FUnorderedAccessViewRHIRef>, TInlineAllocator<1>> UAVs;
	TArray<TPair<FRHIBufferSRVCreateInfo, FShaderResourceViewRHIRef>, TInlineAllocator<1>> SRVs;
};
