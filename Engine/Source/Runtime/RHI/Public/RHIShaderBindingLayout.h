// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "Misc/Optional.h"
#include "RHIDefinitions.h"

/*
* FRHIUniformBufferShaderBindingLayout contains data on how a uniform buffer is used in a FRHIShaderBindingLayout
*/
struct FRHIUniformBufferShaderBindingLayout
{
	FString LayoutName;
	union
	{
		struct
		{
			uint32 RegisterSpace : 4;
			uint32 CBVResourceIndex : 6;
			uint32 BaseSRVResourceIndex : 8;
			uint32 BaseUAVResourceIndex : 8;
			uint32 BaseSamplerResourceIndex : 6;
		};
		uint32 Flags;
	};

	FRHIUniformBufferShaderBindingLayout() : Flags(0) {}
			
	friend uint32 GetTypeHash(const FRHIUniformBufferShaderBindingLayout& Entry)
	{
		uint32 Hash = GetTypeHash(Entry.LayoutName);
		Hash = HashCombineFast(Hash, GetTypeHash(Entry.Flags));
		return Hash;
	}
	
	bool operator==(const FRHIUniformBufferShaderBindingLayout& Other) const
	{
		return LayoutName == Other.LayoutName &&
			Flags == Other.Flags;			
	}

	friend inline FArchive& operator << (FArchive& Ar, FRHIUniformBufferShaderBindingLayout& F)
	{
		Ar << F.LayoutName;
		Ar << F.Flags;
		return Ar;
	}
};

enum class EShaderBindingLayoutFlags : uint8
{
	None = 0,
	AllowMeshShaders = 1 << 0,
	InputAssembler = 1 << 1,
	BindlessResources = 1 << 2,
	BindlessSamplers = 1 << 3,
	RootConstants = 1 << 4,
	ShaderBindingLayoutUsed = 1 << 5,
};
ENUM_CLASS_FLAGS(EShaderBindingLayoutFlags)


/*
* FRHIShaderBindingLayout contains data which is used during shader generation to build the shareable shader resource tables between multiple shaders.
* All shaders using the same FRHIShaderBindingLayout only have to bind the uniform buffers declared one at runtime. The shaders can have different PSOs
* but they will define the resources of the uniform buffers at a specific resource index or SRT offset.
* FRHIShaderBindingLayout is also used at runtime to know how/where the resources of the uniform buffers need to be bound.
*/
class FRHIShaderBindingLayout
{
public:

	enum
	{
		MaxUniformBufferEntries = 8,
	};

	FRHIShaderBindingLayout() = default;
	
	FRHIShaderBindingLayout(EShaderBindingLayoutFlags InFlags, TConstArrayView<FRHIUniformBufferShaderBindingLayout> InUniformBufferEntries) : Flags(InFlags)
	{
		NumUniformBufferEntries = InUniformBufferEntries.Num();
		check(NumUniformBufferEntries < MaxUniformBufferEntries);
		for (uint32 Index = 0; Index < NumUniformBufferEntries; ++Index)
		{
			UniformBufferEntries[Index] = InUniformBufferEntries[Index];
		}

		ComputeHash();
	}

	uint32 GetHash() const { return Hash; }
	EShaderBindingLayoutFlags GetFlags() const { return Flags; }
	uint32 GetNumUniformBufferEntries() const { return NumUniformBufferEntries; }
	const FRHIUniformBufferShaderBindingLayout& GetUniformBufferEntry(uint32 Index) const { check(Index < NumUniformBufferEntries); return UniformBufferEntries[Index]; }
	   	
	const FRHIUniformBufferShaderBindingLayout* FindEntry(const FString& LayoutName) const
	{
		for (const FRHIUniformBufferShaderBindingLayout& Entry : UniformBufferEntries)
		{
			if (Entry.LayoutName == LayoutName)
			{
				return &Entry;
			}
		}

		return nullptr;
	}

	friend uint32 GetTypeHash(const FRHIShaderBindingLayout& Desc)
	{
		return Desc.Hash;
	}

	bool operator==(const FRHIShaderBindingLayout& Other) const
	{
		return Hash == Other.Hash &&
			Flags == Other.Flags &&
			NumUniformBufferEntries == Other.NumUniformBufferEntries &&
			UniformBufferEntries == Other.UniformBufferEntries;			
	}

	friend inline FArchive& operator << (FArchive& Ar, FRHIShaderBindingLayout& F)
	{
		Ar << F.Hash;
		Ar << F.Flags;
		Ar << F.NumUniformBufferEntries;
		Ar << F.UniformBufferEntries;
		return Ar;
	}

private:

	void ComputeHash()
	{
		Hash = HashCombineFast(Hash, GetTypeHash(Flags));
		Hash = HashCombineFast(Hash, GetTypeHash(NumUniformBufferEntries));
		Hash = HashCombineFast(Hash, GetTypeHash(UniformBufferEntries));
	}
	
	uint32 Hash = 0;
	EShaderBindingLayoutFlags Flags = EShaderBindingLayoutFlags::None;
	uint32 NumUniformBufferEntries = 0;
	TStaticArray<FRHIUniformBufferShaderBindingLayout, MaxUniformBufferEntries> UniformBufferEntries = {};
};