// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHIShaderFormatDefinitions.h: Names for Shader Formats
		(that don't require linking).
=============================================================================*/

#pragma once

#include "RHIStaticShaderPlatformNames.h"

static FName NAME_PCD3D_SM6("PCD3D_SM6");
static FName NAME_PCD3D_SM5("PCD3D_SM5");
static FName NAME_PCD3D_ES3_1("PCD3D_ES31");

static FName NAME_GLSL_150_ES31("GLSL_150_ES31");
static FName NAME_GLSL_ES3_1_ANDROID("GLSL_ES3_1_ANDROID");

static FName NAME_SF_METAL_ES3_1_IOS_DEPRECATED("SF_METAL");
static FName NAME_SF_METAL_SM5_IOS_DEPRECATED("SF_METAL_MRT");
static FName NAME_SF_METAL_ES3_1_TVOS_DEPRECATED("SF_METAL_TVOS");
static FName NAME_SF_METAL_SM5_TVOS_DEPRECATED("SF_METAL_MRT_TVOS");
static FName NAME_SF_METAL_MRT_MAC_DEPRECATED("SF_METAL_MRT_MAC");

static FName NAME_SF_METAL_ES3_1_IOS("SF_METAL_ES3_1_IOS");
static FName NAME_SF_METAL_SM5_IOS("SF_METAL_SM5_IOS");
static FName NAME_SF_METAL_SM6_IOS("SF_METAL_SM6_IOS");
static FName NAME_SF_METAL_ES3_1_TVOS("SF_METAL_ES3_1_TVOS");
static FName NAME_SF_METAL_SM5_TVOS("SF_METAL_SM5_TVOS");
static FName NAME_SF_METAL_SM5("SF_METAL_SM5");
static FName NAME_SF_METAL_SM6("SF_METAL_SM6");
static FName NAME_SF_METAL_SIM("SF_METAL_SIM");
static FName NAME_SF_METAL_ES3_1("SF_METAL_ES3_1");

static FName NAME_VULKAN_ES3_1_ANDROID("SF_VULKAN_ES31_ANDROID");
static FName NAME_VULKAN_ES3_1("SF_VULKAN_ES31");
static FName NAME_VULKAN_SM5("SF_VULKAN_SM5");
static FName NAME_VULKAN_SM6("SF_VULKAN_SM6");
static FName NAME_VULKAN_SM5_ANDROID("SF_VULKAN_SM5_ANDROID");

static EShaderPlatform ShaderFormatNameToShaderPlatform(FName ShaderFormat)
{
	if (ShaderFormat == NAME_PCD3D_SM6)							return SP_PCD3D_SM6;
	if (ShaderFormat == NAME_PCD3D_SM5)							return SP_PCD3D_SM5;
	if (ShaderFormat == NAME_PCD3D_ES3_1)						return SP_PCD3D_ES3_1;

	if (ShaderFormat == NAME_GLSL_150_ES31)						return SP_OPENGL_PCES3_1;
	if (ShaderFormat == NAME_GLSL_ES3_1_ANDROID)				return SP_OPENGL_ES3_1_ANDROID;

	if (ShaderFormat == NAME_SF_METAL_ES3_1_IOS)				return SP_METAL_ES3_1_IOS;
	if (ShaderFormat == NAME_SF_METAL_ES3_1_IOS_DEPRECATED)		return SP_METAL_ES3_1_IOS;
	if (ShaderFormat == NAME_SF_METAL_SM5_IOS)					return SP_METAL_SM5_IOS;
	if (ShaderFormat == NAME_SF_METAL_SM5_IOS_DEPRECATED)		return SP_METAL_SM5_IOS;
	if (ShaderFormat == NAME_SF_METAL_SM6_IOS)					return SP_METAL_SM6_IOS;
	if (ShaderFormat == NAME_SF_METAL_ES3_1_TVOS)				return SP_METAL_ES3_1_TVOS;
	if (ShaderFormat == NAME_SF_METAL_ES3_1_TVOS_DEPRECATED)	return SP_METAL_ES3_1_TVOS;
	if (ShaderFormat == NAME_SF_METAL_SM5_TVOS)					return SP_METAL_SM5_TVOS;
	if (ShaderFormat == NAME_SF_METAL_SM5_TVOS_DEPRECATED)		return SP_METAL_SM5_TVOS;
	if (ShaderFormat == NAME_SF_METAL_SM5)						return SP_METAL_SM5;
	if (ShaderFormat == NAME_SF_METAL_SM6)						return SP_METAL_SM6;
	if (ShaderFormat == NAME_SF_METAL_SIM)						return SP_METAL_SIM;
	if (ShaderFormat == NAME_SF_METAL_ES3_1)					return SP_METAL_ES3_1;
	if (ShaderFormat == NAME_SF_METAL_MRT_MAC_DEPRECATED)				return SP_METAL_ES3_1;

	if (ShaderFormat == NAME_VULKAN_ES3_1_ANDROID)				return SP_VULKAN_ES3_1_ANDROID;
	if (ShaderFormat == NAME_VULKAN_ES3_1)						return SP_VULKAN_PCES3_1;
	if (ShaderFormat == NAME_VULKAN_SM5)						return SP_VULKAN_SM5;
	if (ShaderFormat == NAME_VULKAN_SM6)						return SP_VULKAN_SM6;
	if (ShaderFormat == NAME_VULKAN_SM5_ANDROID)				return SP_VULKAN_SM5_ANDROID;

	const FStaticShaderPlatformNames& StaticNames = FStaticShaderPlatformNames::Get();

	for (int32 StaticPlatform = SP_StaticPlatform_First; StaticPlatform <= SP_StaticPlatform_Last; ++StaticPlatform)
	{
		if (ShaderFormat == StaticNames.GetShaderFormat(EShaderPlatform(StaticPlatform)))
		{
			return EShaderPlatform(StaticPlatform);
		}
	}

	return SP_NumPlatforms;
}
