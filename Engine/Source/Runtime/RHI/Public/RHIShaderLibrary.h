// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIResources.h"
#include "Misc/CoreDelegates.h"

struct FShaderHash;

//
// Shader Library
//

class FRHIShaderLibrary : public FRHIResource
{
public:
	FRHIShaderLibrary(EShaderPlatform InPlatform, FString const& InName) : FRHIResource(RRT_ShaderLibrary), Platform(InPlatform), LibraryName(InName), LibraryId(GetTypeHash(InName)) {}
	virtual ~FRHIShaderLibrary() = default;

	inline EShaderPlatform GetPlatform(void) const { return Platform; }
	inline const FString& GetName(void) const { return LibraryName; }
	inline uint32 GetId(void) const { return LibraryId; }

	virtual bool IsNativeLibrary() const = 0;
	virtual int32 GetNumShaderMaps() const = 0;
	virtual int32 GetNumShaders() const = 0;
	virtual int32 GetNumShadersForShaderMap(int32 ShaderMapIndex) const = 0;
	virtual int32 GetShaderIndex(int32 ShaderMapIndex, int32 i) const = 0;
	virtual void GetAllShaderIndices(int32 ShaderMapIndex, TArray<int32>& ShaderIndices) const {}
	virtual uint32 GetSizeBytes() const = 0;
	virtual FShaderHash GetShaderHash(int32 ShaderMapIndex, int32 ShaderIndex) = 0;
	virtual FShaderHash GetShaderMapHash(int32 ShaderMapIndex) const = 0;
	virtual int32 FindShaderMapIndex(const FShaderHash& Hash) = 0;
	virtual int32 FindShaderIndex(const FShaderHash& Hash) = 0;
	virtual uint32 GetShaderSizeBytes(int32 ShaderIndex) const { return 0; }
	virtual bool IsPreloading(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents) { return false; }
	virtual bool ResolveShaderMap(int32 ShaderMapIndex, FCoreDelegates::FIoChunkIdResolvedFunc IoChunkIdResolvedFunc) const { return false; }
	virtual bool PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents) { return false; }
	virtual bool PreloadShaderMap(int32 ShaderMapIndex, FGraphEventArray& OutCompletionEvents) { return false; }
	virtual bool PreloadShaderMap(int32 ShaderMapIndex, FCoreDelegates::FAttachShaderReadRequestFunc AttachShaderReadRequestFunc) { return false; }
	virtual void ReleasePreloadedShader(int32 ShaderIndex) {}

	/**
	 * Returns true if all chunks of the specified shadermap resource are ready for use.
	 * This is only relevant for the IoStore shader library for containers that are loaded on demand.
	 * We expect those chunks to be loaded by the on-demand content managers.
	 * This function ensures the shadermap chunks are available when a material tries to load such a shadermap resource.
	 * Otherwise, the material can fallback to the default material.
	 */
	virtual bool IsShaderMapResolved(int32 ShaderMapIndex) const { return true; };

	virtual void AddRefPreloadedShaderGroup(int32 ShaderGroupIndex) {}
	virtual void ReleasePreloadedShaderGroup(int32 ShaderGroupIndex) {}
	virtual int32 GetGroupIndexForShader(int32 ShaderIndex) const { return INDEX_NONE; }
	virtual int32 GetLibraryId() { return LibraryId; }
	/*CreateShader can return a null shader when bRequired == false. Usefull to debug dynamic shader preloading or when shaders haven't finished loading.*/
	virtual TRefCountPtr<FRHIShader> CreateShader(int32 ShaderIndex, bool bRequired = true) { return nullptr; }
	virtual void Teardown() {}

	/** Called prior to Teardown(), giving subclasses a chance to perform additional cleanup. */
	virtual void OnCloseShaderCode() {}

protected:
	EShaderPlatform Platform;
	FString LibraryName;
	uint32 LibraryId;
};
