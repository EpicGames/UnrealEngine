// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "RHIResources.h"
#include "RHIResourceCollection.h"
#include "RHIDescriptorRange.h"

#define RHI_VALIDATE_BATCHED_SHADER_PARAMETERS DO_CHECK

class FRHICommandList;
class FRHIComputeCommandList;

/** Compact representation of a bound shader parameter (read: value). Its offsets are for referencing their data in an associated blob. */
struct FRHIShaderParameter
{
	FRHIShaderParameter(uint16 InBufferIndex, uint16 InBaseIndex, uint16 InByteOffset, uint16 InByteSize)
		: BufferIndex(InBufferIndex)
		, BaseIndex(InBaseIndex)
		, ByteOffset(InByteOffset)
		, ByteSize(InByteSize)
	{
	}
	uint16 BufferIndex;
	uint16 BaseIndex;
	uint16 ByteOffset;
	uint16 ByteSize;
};

/** Compact representation of a bound resource parameter (Texture, SRV, UAV, SamplerState, or UniformBuffer) */
struct FRHIShaderParameterResource
{
	enum class EType : uint8
	{
		Texture,
		ResourceView,
		UnorderedAccessView,
		Sampler,
		UniformBuffer,
		ResourceCollection,
		DescriptorRange,
	};

	FRHIShaderParameterResource() = default;
	FRHIShaderParameterResource(EType InType, FRHIResource* InResource, uint16 InIndex)
		: Resource(InResource)
		, Index(InIndex)
		, Type(InType)
	{
	}
	FRHIShaderParameterResource(FRHITexture* InTexture, uint16 InIndex)
		: FRHIShaderParameterResource(FRHIShaderParameterResource::EType::Texture, InTexture, InIndex)
	{
	}
	FRHIShaderParameterResource(FRHIShaderResourceView* InView, uint16 InIndex)
		: FRHIShaderParameterResource(FRHIShaderParameterResource::EType::ResourceView, InView, InIndex)
	{
	}
	FRHIShaderParameterResource(FRHIUnorderedAccessView* InUAV, uint16 InIndex)
		: FRHIShaderParameterResource(FRHIShaderParameterResource::EType::UnorderedAccessView, InUAV, InIndex)
	{
	}
	FRHIShaderParameterResource(FRHISamplerState* InSamplerState, uint16 InIndex)
		: FRHIShaderParameterResource(FRHIShaderParameterResource::EType::Sampler, InSamplerState, InIndex)
	{
	}
	FRHIShaderParameterResource(FRHIUniformBuffer* InUniformBuffer, uint16 InIndex)
		: FRHIShaderParameterResource(FRHIShaderParameterResource::EType::UniformBuffer, InUniformBuffer, InIndex)
	{
	}
	FRHIShaderParameterResource(FRHIResourceCollection* InResourceCollection, uint16 InIndex)
		: FRHIShaderParameterResource(FRHIShaderParameterResource::EType::ResourceCollection, InResourceCollection, InIndex)
	{
	}
	FRHIShaderParameterResource(FRHIDescriptorRange* InDescriptorRange, uint16 InIndex)
		: FRHIShaderParameterResource(FRHIShaderParameterResource::EType::DescriptorRange, InDescriptorRange, InIndex)
	{
	}

	bool operator == (const FRHIShaderParameterResource& Other) const
	{
		return Resource == Other.Resource
			&& Index == Other.Index
			&& Type == Other.Type;
	}

	FRHIResource* Resource = nullptr;
	uint16        Index = 0;
	EType         Type = EType::Texture;
};

struct FRHIBatchedShaderParameters;

enum class ERHIBatchedShaderParameterAllocatorPageSize
{
	Small,
	Large
};

class FRHIBatchedShaderParametersAllocator
{
public:
	FRHIBatchedShaderParametersAllocator* Next;
	FRHICommandListBase& RHICmdList;

private:
	friend class FRHICommandListBase;
	friend struct FRHIBatchedShaderParameters;

	FMemStackBase ParametersData;
	FMemStackBase Parameters;
	FMemStackBase ResourceParameters;
	FMemStackBase BindlessParameters;

	FRHIBatchedShaderParametersAllocator(FRHIBatchedShaderParametersAllocator*& InOutRootListLink, FRHICommandListBase& InRHICmdList, ERHIBatchedShaderParameterAllocatorPageSize PageSize)
		: FRHIBatchedShaderParametersAllocator(InOutRootListLink, InRHICmdList, PageSize == ERHIBatchedShaderParameterAllocatorPageSize::Small ? FMemStackBase::EPageSize::Small : FMemStackBase::EPageSize::Large)
	{}

	FRHIBatchedShaderParametersAllocator(FRHIBatchedShaderParametersAllocator*& InOutRootListLink, FRHICommandListBase& InRHICmdList, FMemStackBase::EPageSize PageSize)
		: Next(InOutRootListLink)
		, RHICmdList(InRHICmdList)
		, ParametersData(PageSize)
		, Parameters(PageSize)
		, ResourceParameters(PageSize)
		, BindlessParameters(PageSize)
	{
		InOutRootListLink = this;
	}

	inline void Attach(const FRHIBatchedShaderParameters* InParameters)
	{
#if RHI_VALIDATE_BATCHED_SHADER_PARAMETERS
		if (AttachedParameters != InParameters)
		{
			checkf(!AttachedParameters, TEXT("Only one FRHIBatchedShaderParameters instance can be used at a time with this allocator. You must call FRHIBatchedShaderParameters::{Reset, Finish} to start processing a new one."));
			AttachedParameters = InParameters;
		}
#endif
	}

	inline void Detach()
	{
#if RHI_VALIDATE_BATCHED_SHADER_PARAMETERS
		AttachedParameters = nullptr;
#endif
	}

	template <typename... ArgsType>
	inline void EmplaceParameter(TArrayView<FRHIShaderParameter>& InOutArray, ArgsType&& ...Args)
	{
		Emplace(Parameters, InOutArray, Forward<ArgsType&&>(Args)...);
	}

	template <typename... ArgsType>
	inline void AddResourceParameter(TArrayView<FRHIShaderParameterResource>& InOutArray, ArgsType&& ...Args)
	{
		Emplace(ResourceParameters, InOutArray, Forward<ArgsType&&>(Args)...);
	}

	template <typename... ArgsType>
	inline void AddBindlessParameter(TArrayView<FRHIShaderParameterResource>& InOutArray, ArgsType&& ...Args)
	{
		Emplace(BindlessParameters, InOutArray, Forward<ArgsType&&>(Args)...);
	}

#if RHI_VALIDATE_BATCHED_SHADER_PARAMETERS
	const FRHIBatchedShaderParameters* AttachedParameters = nullptr;
#endif

	template <typename ElementType, typename... ArgsType>
	void Emplace(FMemStackBase& MemStack, TArrayView<ElementType>& InOutArray, ArgsType&& ...Args)
	{
		static_assert(sizeof(ElementType) % alignof(ElementType) == 0, "Element size must be a multiple of its alignment");

		const size_t ElementSize = sizeof(ElementType);
		const size_t Alignment   = alignof(ElementType);
		const int32 NumElements  = InOutArray.Num() + 1;
		ElementType* Elements    = InOutArray.GetData();

		if (InOutArray.IsEmpty())
		{
			Elements = new (MemStack.Alloc(ElementSize, Alignment)) ElementType(Forward<ArgsType&>(Args)...);
		}
		else
		{
			// Sanity check that the top of the stack contains the last element that was allocated.
			check(MemStack.GetTop() == (uint8*)(InOutArray.GetData() + InOutArray.Num()));

			// Try to extend the size of the current array without resizing.
			if (MemStack.CanFitInPage(ElementSize, 1))
			{
				new (MemStack.Alloc(ElementSize, 1)) ElementType(Forward<ArgsType&>(Args)...);
			}
			// Reached the end of the page. Reallocate the entire array into a new page.
			else
			{
				Elements = reinterpret_cast<ElementType*>(MemStack.Alloc(NumElements * ElementSize, Alignment));
				ElementType* LastElement = Elements;
				for (int32 Index = 0; Index < InOutArray.Num(); ++Index, ++LastElement)
				{
					new (LastElement) ElementType(MoveTemp(InOutArray[Index]));
				}
				new (LastElement) ElementType(Forward<ArgsType&>(Args)...);
			}
		}

		InOutArray = TArrayView<ElementType>(Elements, NumElements);
	}

	void AppendParametersData(TArrayView<uint8>& InOutArray, uint32 NumBytes, const uint8* Bytes)
	{
		constexpr size_t Alignment = 1;
		const int32 NumArrayBytes  = InOutArray.Num() + NumBytes;
		uint8* ArrayBytes          = InOutArray.GetData();

		if (InOutArray.IsEmpty())
		{
			ArrayBytes = (uint8*)ParametersData.Alloc(NumBytes, Alignment);
			FMemory::Memcpy(ArrayBytes, Bytes, NumBytes);
		}
		else
		{
			// Sanity check that the top of the stack contains the last element that was allocated.
			check(ParametersData.GetTop() == InOutArray.GetData() + InOutArray.Num());

			// Try to extend the size of the current array without resizing.
			if (ParametersData.CanFitInPage(NumBytes, Alignment))
			{
				FMemory::Memcpy(ParametersData.Alloc(NumBytes, Alignment), Bytes, NumBytes);
			}
			// Reached the end of the page. Reallocate the entire array into a new page.
			else
			{
				ArrayBytes = (uint8*)ParametersData.Alloc(NumArrayBytes, Alignment);
				FMemory::Memcpy(ArrayBytes, InOutArray.GetData(), InOutArray.Num());
				FMemory::Memcpy(ArrayBytes + InOutArray.Num(), Bytes, NumBytes);
			}
		}

		InOutArray = TArrayView<uint8>(ArrayBytes, NumArrayBytes);
	}
};

/** Collection of parameters to set in the RHI. These parameters aren't bound to any specific shader until SetBatchedShaderParameters is called. */
struct FRHIBatchedShaderParameters
{
	FRHIBatchedShaderParametersAllocator& Allocator;
	TArrayView<uint8> ParametersData;
	TArrayView<FRHIShaderParameter> Parameters;
	TArrayView<FRHIShaderParameterResource> ResourceParameters;
	TArrayView<FRHIShaderParameterResource> BindlessParameters;

	FRHIBatchedShaderParameters(FRHIBatchedShaderParametersAllocator& InAllocator)
		: Allocator(InAllocator)
	{}

	inline bool HasParameters() const
	{
		return (Parameters.Num() + ResourceParameters.Num() + BindlessParameters.Num()) > 0;
	}

	// Marks the parameters as complete and retains the parameter contents.
	void Finish()
	{
		Allocator.Detach();
	}

	// Resets the parameters back to an empty state.
	void Reset()
	{
		Allocator.Detach();
		ParametersData = {};
		Parameters = {};
		ResourceParameters = {};
		BindlessParameters = {};
	}

	template <typename... ArgsType>
	inline void AddResourceParameter(ArgsType&& ...Args)
	{
		Allocator.Attach(this);
		Allocator.AddResourceParameter(ResourceParameters, Forward<ArgsType&>(Args)...);
	}
	
	template <typename... ArgsType>
	inline void AddBindlessParameter(ArgsType&& ...Args)
	{
		Allocator.Attach(this);
		Allocator.AddBindlessParameter(BindlessParameters, Forward<ArgsType&>(Args)...);
	}

	inline void SetShaderParameter(uint32 BufferIndex, uint32 BaseIndex, uint32 NumBytes, const void* NewValue)
	{
		const int32 DestDataOffset = ParametersData.Num();
		Allocator.Attach(this);
		Allocator.AppendParametersData(ParametersData, NumBytes, (const uint8*)NewValue);
		Allocator.EmplaceParameter(Parameters, (uint16)BufferIndex, (uint16)BaseIndex, (uint16)DestDataOffset, (uint16)NumBytes);
	}

	inline void SetShaderUniformBuffer(uint32 Index, FRHIUniformBuffer* UniformBuffer)
	{
		AddResourceParameter(UniformBuffer, (uint16)Index);
	}

	inline void SetBindlessShaderUniformBuffer(FRHIUniformBuffer* UniformBuffer)
	{
		AddBindlessParameter(UniformBuffer, (uint16)0);
	}

	inline void SetShaderTexture(uint32 Index, FRHITexture* Texture)
	{
		AddResourceParameter(Texture, (uint16)Index);
	}

	inline void SetShaderResourceViewParameter(uint32 Index, FRHIShaderResourceView* SRV)
	{
		AddResourceParameter(SRV, (uint16)Index);
	}

	inline void SetShaderSampler(uint32 Index, FRHISamplerState* State)
	{
		AddResourceParameter(State, (uint16)Index);
	}

	inline void SetUAVParameter(uint32 Index, FRHIUnorderedAccessView* UAV)
	{
		AddResourceParameter(UAV, (uint16)Index);
	}

	inline void SetResourceCollection(uint32 Index, FRHIResourceCollection* ResourceCollection)
	{
		AddResourceParameter(ResourceCollection, (uint16)Index);
	}

	inline void SetDescriptorRange(uint32 Index, FRHIDescriptorRange* DescriptorRange)
	{
		AddResourceParameter(DescriptorRange, (uint16)Index);
	}

	inline void SetBindlessTexture(uint32 Index, FRHITexture* Texture)
	{
		AddBindlessParameter(Texture, (uint16)Index);
	}

	inline void SetBindlessResourceView(uint32 Index, FRHIShaderResourceView* SRV)
	{
		AddBindlessParameter(SRV, (uint16)Index);
	}

	inline void SetBindlessSampler(uint32 Index, FRHISamplerState* State)
	{
		AddBindlessParameter(State, (uint16)Index);
	}

	inline void SetBindlessUAV(uint32 Index, FRHIUnorderedAccessView* UAV)
	{
		AddBindlessParameter(UAV, (uint16)Index);
	}

	inline void SetBindlessResourceCollection(uint32 Index, FRHIResourceCollection* ResourceCollection)
	{
		AddBindlessParameter(ResourceCollection, (uint16)Index);
	}
};

/** Compact representation of a resource parameter unbind, limited to  SRVs and UAVs */
struct FRHIShaderParameterUnbind
{
	enum class EType : uint8
	{
		ResourceView,
		UnorderedAccessView,
	};

	FRHIShaderParameterUnbind() = default;
	FRHIShaderParameterUnbind(EType InType, uint16 InIndex)
		: Index(InIndex)
		, Type(InType)
	{
	}

	uint16  Index = 0;
	EType   Type = EType::ResourceView;
};

/** Collection of parameters to unbind in the RHI. These unbinds aren't tied to any specific shader until SetBatchedShaderUnbinds is called. */
struct FRHIBatchedShaderUnbinds
{
	TArray<FRHIShaderParameterUnbind> Unbinds;

	bool HasParameters() const
	{
		return Unbinds.Num() > 0;
	}

	void Reset()
	{
		Unbinds.Reset();
	}

	void UnsetSRV(uint32 Index)
	{
		Unbinds.Emplace(FRHIShaderParameterUnbind::EType::ResourceView, (uint16)Index);
	}
	void UnsetUAV(uint32 Index)
	{
		Unbinds.Emplace(FRHIShaderParameterUnbind::EType::UnorderedAccessView, (uint16)Index);
	}
};

struct FRHIShaderBundleComputeDispatch
{
	uint32 RecordIndex = ~uint32(0u);
	class FComputePipelineState* PipelineState = nullptr;
	FRHIComputeShader* Shader = nullptr;
	FRHIWorkGraphShader* WorkGraphShader = nullptr;
	FRHIComputePipelineState* RHIPipeline = nullptr;
	TOptional<FRHIBatchedShaderParameters> Parameters;
	FUint32Vector4 Constants;

	inline bool IsValid() const
	{
		return RecordIndex != ~uint32(0u);
	}
};

struct FRHIShaderBundleGraphicsState
{
	FIntRect ViewRect;

	float DepthMin = 0.0f;
	float DepthMax = 1.0f;

	float BlendFactor[4] = { 1.0f, 1.0f, 1.0f, 1.0f };

	EPrimitiveType PrimitiveType = PT_TriangleList;

	uint8 StencilRef = 0;
};

struct FRHIShaderBundleGraphicsDispatch
{
	uint32 RecordIndex = ~uint32(0u);
	class FGraphicsPipelineState* PipelineState = nullptr;
	FRHIGraphicsPipelineState* RHIPipeline = nullptr;

	FGraphicsPipelineStateInitializer PipelineInitializer;

	TOptional<FRHIBatchedShaderParameters> Parameters_MSVS;
	TOptional<FRHIBatchedShaderParameters> Parameters_PS;

	FUint32Vector4 Constants;

	inline bool IsValid() const
	{
		return RecordIndex != ~uint32(0u);
	}
};