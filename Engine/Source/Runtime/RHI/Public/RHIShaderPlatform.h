// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"
#include "RHIDefinitions.h"
#include "HAL/Platform.h"

/** @warning: update *LegacyShaderPlatform* when the below changes */
enum EShaderPlatform : uint16
{
	SP_PCD3D_SM5 = 0,
	SP_METAL_ES3_1_IOS = 11,
	SP_METAL_SM5_IOS = 12,
	SP_PCD3D_ES3_1 = 14,
	SP_OPENGL_PCES3_1 = 15,
	SP_METAL_SM5 = 16,
	SP_VULKAN_PCES3_1 = 17,
	SP_VULKAN_SM5 = 20,
	SP_VULKAN_ES3_1_ANDROID = 21,
	SP_METAL_ES3_1 = 22,
	SP_OPENGL_ES3_1_ANDROID = 24,
	SP_METAL_ES3_1_TVOS = 30,
	SP_METAL_SM5_TVOS = 31,
	/**********************************************************************************/
	/* !! Do not add any new platforms here. Add them below SP_StaticPlatform_Last !! */
	/**********************************************************************************/

	//---------------------------------------------------------------------------------
	/** Pre-allocated block of shader platform enum values for platform extensions */
#define DDPI_NUM_STATIC_SHADER_PLATFORMS 16
	SP_StaticPlatform_First = 32,

	// Platform specific shaders are now stored in platform specific files called <platform>RHIShaderPlatform.h
	// These files are generated by UBT and can be included the same way as this file

	SP_StaticPlatform_Last = (SP_StaticPlatform_First + DDPI_NUM_STATIC_SHADER_PLATFORMS - 1),

	//  Add new platforms below this line, starting from (SP_StaticPlatform_Last + 1)
	//---------------------------------------------------------------------------------
	SP_VULKAN_SM5_ANDROID = SP_StaticPlatform_Last + 1,
	SP_PCD3D_SM6          = SP_StaticPlatform_Last + 2,
	SP_VULKAN_SM6         = SP_StaticPlatform_Last + 4,
    SP_METAL_SM6          = SP_StaticPlatform_Last + 5,
    SP_METAL_SIM          = SP_StaticPlatform_Last + 6,
	SP_METAL_SM6_IOS 	  = SP_StaticPlatform_Last + 7,

	SP_CUSTOM_PLATFORM_FIRST,
	SP_CUSTOM_PLATFORM_LAST = (SP_CUSTOM_PLATFORM_FIRST + 100),

	SP_NumPlatforms,
	SP_NumBits = 16,
};
static_assert(SP_NumPlatforms <= (1 << SP_NumBits), "SP_NumPlatforms will not fit on SP_NumBits");

#if USE_STATIC_SHADER_PLATFORM_ENUMS
#include COMPILED_PLATFORM_HEADER_GENERATED(RHIShaderPlatform.h)
#include COMPILED_PLATFORM_HEADER(StaticShaderPlatform.inl)
#else
struct FStaticShaderPlatform final
{
	inline FStaticShaderPlatform(const EShaderPlatform InPlatform) : Platform(InPlatform) {}
	inline operator EShaderPlatform() const
	{
		return Platform;
	}

	inline bool operator == (const EShaderPlatform Other) const
	{
		return Other == Platform;
	}
	inline bool operator != (const EShaderPlatform Other) const
	{
		return Other != Platform;
	}
private:
	const EShaderPlatform Platform;
};
#endif

// The maximum shader platform available on this system
extern RHI_API EShaderPlatform GMaxRHIShaderPlatform;

inline EShaderPlatform GetMaxShaderPlatformChecked()
{
	checkf(!IsRunningCookCommandlet(), TEXT("Calling GetMaxShaderPlatformChecked while inside a cook commandlet is not correct, as it will return the cookers GMaxRHIShaderPlatform and not the ShaderPlatform you are cooking for"));
	return GMaxRHIShaderPlatform;
}

inline bool IsCustomPlatform(const FStaticShaderPlatform Platform)
{
	return (Platform >= SP_CUSTOM_PLATFORM_FIRST && Platform < SP_CUSTOM_PLATFORM_LAST);
}
