// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIShaderPlatform.h"
#include "UObject/NameTypes.h"

class FStaticShaderPlatformNames
{
private:
	static const uint32 NumPlatforms = DDPI_NUM_STATIC_SHADER_PLATFORMS;

	struct FPlatform
	{
		FName Name;
		FName ShaderPlatform;
		FName ShaderFormat;
	} Platforms[NumPlatforms];

	FStaticShaderPlatformNames();

public:
	static RHI_API const FStaticShaderPlatformNames& Get();

	static inline bool IsStaticPlatform(EShaderPlatform Platform)
	{
		return Platform >= SP_StaticPlatform_First && Platform <= SP_StaticPlatform_Last;
	}

	inline const FName& GetShaderPlatform(EShaderPlatform Platform) const
	{
		return Platforms[GetStaticPlatformIndex(Platform)].ShaderPlatform;
	}

	inline const FName& GetShaderFormat(EShaderPlatform Platform) const
	{
		return Platforms[GetStaticPlatformIndex(Platform)].ShaderFormat;
	}

	inline const FName& GetPlatformName(EShaderPlatform Platform) const
	{
		return Platforms[GetStaticPlatformIndex(Platform)].Name;
	}

private:
	static inline uint32 GetStaticPlatformIndex(EShaderPlatform Platform)
	{
		check(IsStaticPlatform(Platform));
		return uint32(Platform) - SP_StaticPlatform_First;
	}
};
