// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "MultiGPU.h"
#include "RHIDefinitions.h"
#include "RHIGlobals.h"
#include "Stats/Stats.h"

struct FTextureMemoryStats
{
	// Hardware state (never change after device creation):

	// -1 if unknown, in bytes
	int64 DedicatedVideoMemory = -1;

	// -1 if unknown, in bytes
	int64 DedicatedSystemMemory = -1;

	// -1 if unknown, in bytes
	int64 SharedSystemMemory = -1;

	// Total amount of "graphics memory" that we think we can use for all our graphics resources, in bytes. -1 if unknown.
	int64 TotalGraphicsMemory = -1;

	// Size of memory allocated to streaming textures, in bytes
	uint64 StreamingMemorySize = 0;

	// Size of memory allocated to non-streaming textures, in bytes
	uint64 NonStreamingMemorySize = 0;

	// Size of the largest memory fragment, in bytes
	int64 LargestContiguousAllocation = 0;
	
	// 0 if streaming pool size limitation is disabled, in bytes
	int64 TexturePoolSize = 0;

	int64 GetTotalDeviceWorkingMemory() const
	{
		if (GRHIDeviceIsIntegrated)
		{
			// Max in case the device failed to report the available working memory
			return FMath::Max(TotalGraphicsMemory, DedicatedVideoMemory);
		}
		
		return DedicatedVideoMemory;
	}

	bool AreHardwareStatsValid() const
	{
		// pardon the redundancy, have a broken compiler (__EMSCRIPTEN__) that needs these types spelled out...
		return ((int64)DedicatedVideoMemory >= 0 && (int64)DedicatedSystemMemory >= 0 && (int64)SharedSystemMemory >= 0);
	}

	bool IsUsingLimitedPoolSize() const
	{
		return TexturePoolSize > 0;
	}

	int64 ComputeAvailableMemorySize() const
	{
		return FMath::Max<int64>(TexturePoolSize - StreamingMemorySize, 0);
	}
};


// GPU stats

extern RHI_API int32 GNumDrawCallsRHI[MAX_NUM_GPUS];
extern RHI_API int32 GNumPrimitivesDrawnRHI[MAX_NUM_GPUS];

#if HAS_GPU_STATS

struct FRHIDrawStatsCategory
{
	RHI_API FRHIDrawStatsCategory(FName InName);

	FName  const Name;
	uint32 const Index;

	static constexpr int32 MAX_DRAWCALL_CATEGORY = 31;

	struct FManager
	{
		TStaticArray<FRHIDrawStatsCategory*, MAX_DRAWCALL_CATEGORY> Array;

		// A backup of the counts that can be used to display on screen to avoid flickering.

		struct FDrawCountsPerGPU : public TStaticArray<int32, MAX_NUM_GPUS>
		{
			FDrawCountsPerGPU()
				: TStaticArray<int32, MAX_NUM_GPUS>(InPlace, 0)
			{}
		};

		struct FDrawCounts : public TStaticArray<FDrawCountsPerGPU, MAX_DRAWCALL_CATEGORY>
		{
			FDrawCounts()
				: TStaticArray<FDrawCountsPerGPU, MAX_DRAWCALL_CATEGORY>(InPlace)
			{}
		};

		// Used to track the max number of draws per frame.
		FDrawCounts CurrentCounts;

		// Copy of CurrentCounts updated periodically to use for displaying the "stat drawcount" text on screen.
		FDrawCounts DisplayCounts;
		mutable FCriticalSection DisplayCountsCS;

		int32 NumCategory;

		RHI_API FManager();

		void AccumulateFrameStats(FDrawCounts const& Counts);
	};

	RHI_API static FManager& GetManager();
};

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Draws"     ), STAT_RHIDraws     , STATGROUP_RHI, RHI_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Dispatches"), STAT_RHIDispatches, STATGROUP_RHI, RHI_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Primitives"), STAT_RHIPrimitives, STATGROUP_RHI, RHI_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Vertices"  ), STAT_RHIVertices  , STATGROUP_RHI, RHI_API);

#else

struct FRHIDrawStatsCategory
{
	static constexpr uint32 Index = 0;
};

#endif

// Macros for use inside RHI context Draw/Dispatch functions.
// Updates the Stats structure on the executing RHI command list
#define RHI_DRAW_CALL_STATS(Type,Verts,Prims,Instances)					\
	do																	\
	{																	\
		StatEvent.NumDraws++;											\
		StatEvent.NumPrimitives += Prims * FMath::Max(1u, Instances);	\
		StatEvent.NumVertices   += Verts * FMath::Max(1u, Instances);	\
	} while (false)

#define RHI_DRAW_CALL_INC()			do { StatEvent.NumDraws++;      } while (false)
#define RHI_DISPATCH_CALL_INC()		do { StatEvent.NumDispatches++; } while (false)

// RHI memory stats.
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Render Target 2D Memory"), STAT_RenderTargetMemory2D, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Render Target 3D Memory"), STAT_RenderTargetMemory3D, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Render Target Cube Memory"), STAT_RenderTargetMemoryCube, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);

DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("UAV Texture Memory"), STAT_UAVTextureMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);

DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Texture 2D Memory"), STAT_TextureMemory2D, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Texture 3D Memory"), STAT_TextureMemory3D, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Texture Cube Memory"), STAT_TextureMemoryCube, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);

DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Uniform Buffer Memory"), STAT_UniformBufferMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Index Buffer Memory"), STAT_IndexBufferMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Vertex Buffer Memory"), STAT_VertexBufferMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("RayTracing Acceleration Structure Memory"), STAT_RTAccelerationStructureMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Structured Buffer Memory"), STAT_StructuredBufferMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Byte Address Buffer Memory"), STAT_ByteAddressBufferMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Draw Indirect Buffer Memory"), STAT_DrawIndirectBufferMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Misc Buffer Memory"), STAT_MiscBufferMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);

DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Reserved Buffer Memory (Uncommitted)"), STAT_ReservedUncommittedBufferMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Reserved Buffer Memory (Committed)"), STAT_ReservedCommittedBufferMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Reserved Texture Memory (Uncommitted)"), STAT_ReservedUncommittedTextureMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Reserved Texture Memory (Committed)"), STAT_ReservedCommittedTextureMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Sampler Descriptors Allocated"), STAT_SamplerDescriptorsAllocated, STATGROUP_RHI, RHI_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Resource Descriptors Allocated"), STAT_ResourceDescriptorsAllocated, STATGROUP_RHI, RHI_API);

DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Bindless Sampler Heap"), STAT_BindlessSamplerHeapMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Bindless Resource Heap"), STAT_BindlessResourceHeapMemory, STATGROUP_RHI, FPlatformMemory::MCR_GPU, RHI_API);

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Bindless Sampler Descriptors Allocated"), STAT_BindlessSamplerDescriptorsAllocated, STATGROUP_RHI, RHI_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Bindless Resource Descriptors Allocated"), STAT_BindlessResourceDescriptorsAllocated, STATGROUP_RHI, RHI_API);

struct FRHIMemoryStats
{
	// Budget assigned by the OS/driver. Total memory the app should use.
	uint64 BudgetLocal = 0;   // GPU VRAM budget
	uint64 BudgetSystem = 0;  // System/host memory budget

	// Currently used memory.
	uint64 UsedLocal = 0;     // GPU VRAM used
	uint64 UsedSystem = 0;    // System/host memory used

	// Over-budget memory
	uint64 DemotedLocal = 0;
	uint64 DemotedSystem = 0;

	// Available memory within budget
	uint64 AvailableLocal = 0;
	uint64 AvailableSystem = 0;

	bool IsOverBudget() const
	{
		return DemotedLocal > 0 || DemotedSystem > 0;
	}

	bool IsValid() const
	{
		return BudgetLocal > 0 || BudgetSystem > 0;
	}
};

#if PLATFORM_MICROSOFT

DECLARE_STATS_GROUP(TEXT("D3D Video Memory"), STATGROUP_D3DMemory, STATCAT_Advanced);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Total Video Memory (Budget)"), STAT_D3DTotalVideoMemory, STATGROUP_D3DMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Total System Memory (Budget)"), STAT_D3DTotalSystemMemory, STATGROUP_D3DMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Available Video Memory"), STAT_D3DAvailableVideoMemory, STATGROUP_D3DMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Available System Memory"), STAT_D3DAvailableSystemMemory, STATGROUP_D3DMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Used Video Memory"), STAT_D3DUsedVideoMemory, STATGROUP_D3DMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Used System Memory"), STAT_D3DUsedSystemMemory, STATGROUP_D3DMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Demoted Video Memory"), STAT_D3DDemotedVideoMemory, STATGROUP_D3DMemory, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Demoted System Memory"), STAT_D3DDemotedSystemMemory, STATGROUP_D3DMemory, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Video Memory stats update time"), STAT_D3DUpdateVideoMemoryStats, STATGROUP_D3DMemory, RHI_API);

// Update D3D memory stat counters and CSV profiler stats, if enabled.
RHI_API void UpdateD3DMemoryStatsAndCSV(const FRHIMemoryStats& MemoryStats, bool bUpdateCSV);

#endif // PLATFORM_MICROSOFT