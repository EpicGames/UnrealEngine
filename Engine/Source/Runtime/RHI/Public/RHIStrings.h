// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "RHIDefinitions.h"

class FName;
class FString;

enum class ERHIAccess : uint32;
enum class ERHIPipeline : uint8;
enum EShaderPlatform : uint16;
namespace ERHIFeatureLevel { enum Type : int; }

// helper to convert GRHIVendorId into a printable string, or "Unknown" if unknown.
RHI_API const TCHAR* RHIVendorIdToString();

// helper to convert VendorId into a printable string, or "Unknown" if unknown.
RHI_API const TCHAR* RHIVendorIdToString(EGpuVendorId VendorId);

/** Finds a corresponding ERHIFeatureLevel::Type given a string, or returns false if one could not be found. */
RHI_API bool GetFeatureLevelFromName(const FString& Name, ERHIFeatureLevel::Type& OutFeatureLevel);

/** Finds a corresponding ERHIFeatureLevel::Type given an FName, or returns false if one could not be found. */
RHI_API bool GetFeatureLevelFromName(FName Name, ERHIFeatureLevel::Type& OutFeatureLevel);

/** Creates a string for the given feature level. */
RHI_API void GetFeatureLevelName(ERHIFeatureLevel::Type InFeatureLevel, FString& OutName);

/** Creates an FName for the given feature level. */
RHI_API void GetFeatureLevelName(ERHIFeatureLevel::Type InFeatureLevel, FName& OutName);

/** Finds a corresponding ERHIBindlessConfiguration given a string, or returns false if one could not be found. */
RHI_API bool GetBindlessConfigurationFromString(FStringView InString, ERHIBindlessConfiguration& OutBindlessConfiguration);

/** Get the string for the given BindlessConfiguration or return an empty string if invalid. */
RHI_API const TCHAR* GetBindlessConfigurationString(ERHIBindlessConfiguration InBindlessConfiguration);

RHI_API const TCHAR* GetDescriptorTypeString(ERHIDescriptorType InDescriptorType);
RHI_API FString GetDescriptorMaskString(ERHIDescriptorTypeMask InTypeMask);

/** Stringifies ERHIFeatureLevel */
RHI_API FString LexToString(ERHIFeatureLevel::Type Level);

RHI_API FName LegacyShaderPlatformToShaderFormat(EShaderPlatform Platform);
RHI_API EShaderPlatform ShaderFormatToLegacyShaderPlatform(FName ShaderFormat);
RHI_API FName ShaderPlatformToPlatformName(EShaderPlatform Platform);

/** Stringifies EShaderPlatform */
RHI_API FString LexToString(EShaderPlatform Platform, bool bError = true);

/** Stringifies ERHIDescriptorHeapType */
RHI_API const TCHAR* LexToString(ERHIDescriptorHeapType InHeapType);

/** Finds a corresponding ERHIShadingPath::Type given an FName, or returns false if one could not be found. */
RHI_API bool GetShadingPathFromName(FName Name, ERHIShadingPath::Type& OutShadingPath);

/** Creates a string for the given shading path. */
RHI_API void GetShadingPathName(ERHIShadingPath::Type InShadingPath, FString& OutName);

/** Creates an FName for the given shading path. */
RHI_API void GetShadingPathName(ERHIShadingPath::Type InShadingPath, FName& OutName);

/** Returns a string of friendly name bits for the buffer usage flags enum. */
RHI_API FString GetBufferUsageFlagsName(EBufferUsageFlags BufferUsage);

/** Returns a string of friendly name bits for the texture create flags enum. */
RHI_API FString GetTextureCreateFlagsName(ETextureCreateFlags TextureCreateFlags);

/** Returns a string of friendly name bits for the texture create flags enum. */
RHI_API const TCHAR* StringFromRHIResourceType(ERHIResourceType ResourceType);

RHI_API ERHIResourceType RHIResourceTypeFromString(const FString& InString);

RHI_API FString GetRHIAccessName(ERHIAccess Access);

RHI_API FString GetResourceTransitionFlagsName(EResourceTransitionFlags Flags);
RHI_API FString GetRHIPipelineName(ERHIPipeline Pipeline);

RHI_API const TCHAR* GetTextureDimensionString(ETextureDimension Dimension);
RHI_API const TCHAR* GetTextureCreateFlagString(ETextureCreateFlags TextureCreateFlag);
RHI_API const TCHAR* GetBufferUsageFlagString(EBufferUsageFlags BufferUsage);
RHI_API const TCHAR* GetUniformBufferBaseTypeString(EUniformBufferBaseType BaseType);
RHI_API const TCHAR* GetShaderCodeResourceBindingTypeName(EShaderCodeResourceBindingType BindingType);

// Needs to stay inline for shader formats which can't depend on the RHI module
inline const TCHAR* GetShaderFrequencyString(EShaderFrequency Frequency, bool bIncludePrefix = true)
{
	const TCHAR* String = TEXT("SF_NumFrequencies");
	switch (Frequency)
	{
	case SF_Vertex:					String = TEXT("SF_Vertex"); break;
	case SF_Mesh:					String = TEXT("SF_Mesh"); break;
	case SF_Amplification:			String = TEXT("SF_Amplification"); break;
	case SF_Geometry:				String = TEXT("SF_Geometry"); break;
	case SF_Pixel:					String = TEXT("SF_Pixel"); break;
	case SF_Compute:				String = TEXT("SF_Compute"); break;
	case SF_RayGen:					String = TEXT("SF_RayGen"); break;
	case SF_RayMiss:				String = TEXT("SF_RayMiss"); break;
	case SF_RayHitGroup:			String = TEXT("SF_RayHitGroup"); break;
	case SF_RayCallable:			String = TEXT("SF_RayCallable"); break;
	case SF_WorkGraphRoot:			String = TEXT("SF_WorkGraphRoot"); break;
	case SF_WorkGraphComputeNode:	String = TEXT("SF_WorkGraphComputeNode"); break;

	default:
		break;
	}

	// Skip SF_
	int32 Index = bIncludePrefix ? 0 : 3;
	String += Index;
	return String;
}

inline FName GetRenderQueryTypeName(ERenderQueryType QueryType)
{
	switch (QueryType)
	{
	default: checkNoEntry(); // fallthrough
	case RQT_Undefined:		{ static const FName Name(TEXT("Undefined"));      return Name; }
	case RQT_Occlusion:		{ static const FName Name(TEXT("GPUOcclusion"));    return Name; }
	case RQT_AbsoluteTime:	{ static const FName Name(TEXT("GPUAbsoluteTime")); return Name; }
	}
}
