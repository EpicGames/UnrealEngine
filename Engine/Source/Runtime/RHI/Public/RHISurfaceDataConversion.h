// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHISurfaceDataConversion.h: RHI surface data conversions.
=============================================================================*/

#pragma once

#include "Math/Float16Color.h"
#include "Math/PackedVector.h"
#include "Math/Plane.h"
#include "Math/UnrealMathUtility.h"
#include "RHI.h"
#include "RHITypes.h"

namespace
{

/** Helper for accessing R10G10B10A2 colors. */
struct FRHIR10G10B10A2
{
	uint32 R : 10;
	uint32 G : 10;
	uint32 B : 10;
	uint32 A : 2;
};

/** Helper for accessing R16G16B16A16 colors. */
struct FRHIRGBA16
{
	uint16 R;
	uint16 G;
	uint16 B;
	uint16 A;
};

/** Helper for accessing R16G16 colors. */
struct FRHIRG16
{
	uint16 R;
	uint16 G;
};


inline void ConvertRawR16DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out)
{
	// e.g. shadow maps
	for (uint32 Y = 0; Y < Height; Y++)
	{
		uint16* SrcPtr = (uint16*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			uint16 Value16 = *SrcPtr;
			int Value = FColor::Requantize16to8(Value16);

			*DestPtr = FColor(Value, Value, Value);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR8G8B8A8DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out)
{
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FColor* SrcPtr = (FColor*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = FColor(SrcPtr->B, SrcPtr->G, SrcPtr->R, SrcPtr->A);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawB8G8R8A8DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out)
{
	const uint32 DstPitch = Width * sizeof(FColor);

	// If source & dest pitch matches, perform a single memcpy.
	if (DstPitch == SrcPitch)
	{
		FPlatformMemory::Memcpy(Out, In, Width * Height * sizeof(FColor));
	}
	else
	{
		check(SrcPitch > DstPitch);

		// Need to copy row wise since the Pitch does not match the Width.
		for (uint32 Y = 0; Y < Height; Y++)
		{
			FColor* SrcPtr = (FColor*)(In + Y * SrcPitch);
			FColor* DestPtr = Out + Y * Width;
			FMemory::Memcpy(DestPtr, SrcPtr, DstPitch);
		}
	}
}

inline void ConvertRawR16G16B16A16FDataToFFloat16Color(uint32 Width, uint32 Height, uint8* In, uint32 SrcPitch, FFloat16Color* Out)
{
	const uint32 DstPitch = Width * sizeof(FFloat16Color);

	// If source & dest pitch matches, perform a single memcpy.
	if (DstPitch == SrcPitch)
	{
		FPlatformMemory::Memcpy(Out, In, Width * Height * sizeof(FFloat16Color));
	}
	else
	{
		check(SrcPitch > DstPitch);

		// Need to copy row wise since the Pitch does not match the Width.
		for (uint32 Y = 0; Y < Height; Y++)
		{
			FFloat16Color* SrcPtr = (FFloat16Color*)(In + Y * SrcPitch);
			FFloat16Color* DestPtr = Out + Y * Width;
			FMemory::Memcpy(DestPtr, SrcPtr, DstPitch);
		}
	}
}

inline void ConvertRawR10G10B10A2DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out)
{
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FRHIR10G10B10A2* SrcPtr = (FRHIR10G10B10A2*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = FColor::MakeRequantizeFrom1010102(
				SrcPtr->R,
				SrcPtr->G,
				SrcPtr->B,
				SrcPtr->A
			);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawB10G10R10A2DataToFColor(uint32 Width, uint32 Height, uint8* In, uint32 SrcPitch, FColor* Out)
{
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FRHIR10G10B10A2* SrcPtr = (FRHIR10G10B10A2*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = FColor::MakeRequantizeFrom1010102(
				SrcPtr->B,
				SrcPtr->G,
				SrcPtr->R,
				SrcPtr->A
			);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR16G16B16A16FDataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out, bool LinearToGamma)
{
	FPlane	MinValue(0.0f, 0.0f, 0.0f, 0.0f),
		MaxValue(1.0f, 1.0f, 1.0f, 1.0f);

	check(sizeof(FFloat16) == sizeof(uint16));

	for (uint32 Y = 0; Y < Height; Y++)
	{
		FFloat16* SrcPtr = (FFloat16*)(In + Y * SrcPitch);

		for (uint32 X = 0; X < Width; X++)
		{
			MinValue.X = FMath::Min<float>(SrcPtr[0], MinValue.X);
			MinValue.Y = FMath::Min<float>(SrcPtr[1], MinValue.Y);
			MinValue.Z = FMath::Min<float>(SrcPtr[2], MinValue.Z);
			MinValue.W = FMath::Min<float>(SrcPtr[3], MinValue.W);
			MaxValue.X = FMath::Max<float>(SrcPtr[0], MaxValue.X);
			MaxValue.Y = FMath::Max<float>(SrcPtr[1], MaxValue.Y);
			MaxValue.Z = FMath::Max<float>(SrcPtr[2], MaxValue.Z);
			MaxValue.W = FMath::Max<float>(SrcPtr[3], MaxValue.W);
			SrcPtr += 4;
		}
	}

	for (uint32 Y = 0; Y < Height; Y++)
	{
		FFloat16* SrcPtr = (FFloat16*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr =
				FLinearColor(
				(SrcPtr[0] - MinValue.X) / (MaxValue.X - MinValue.X),
					(SrcPtr[1] - MinValue.Y) / (MaxValue.Y - MinValue.Y),
					(SrcPtr[2] - MinValue.Z) / (MaxValue.Z - MinValue.Z),
					(SrcPtr[3] - MinValue.W) / (MaxValue.W - MinValue.W)
				).ToFColor(LinearToGamma);
			SrcPtr += 4;
			++DestPtr;
		}
	}
}

inline void ConvertRawR11G11B10DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out, bool LinearToGamma)
{
	check(sizeof(FFloat3Packed) == sizeof(uint32));

	for (uint32 Y = 0; Y < Height; Y++)
	{
		FFloat3Packed* SrcPtr = (FFloat3Packed*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			FLinearColor Value = (*SrcPtr).ToLinearColor();

			*DestPtr = Value.ToFColor(LinearToGamma);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR9G9B9E5DataToFColor(uint32 Width, uint32 Height, uint8* In, uint32 SrcPitch, FColor* Out, bool LinearToGamma)
{
	check(sizeof(FFloat3PackedSE) == sizeof(uint32));

	for (uint32 Y = 0; Y < Height; Y++)
	{
		FFloat3PackedSE* SrcPtr = (FFloat3PackedSE*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			FLinearColor Value = (*SrcPtr).ToLinearColor();

			*DestPtr = Value.ToFColor(LinearToGamma);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR32G32B32A32DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out, bool LinearToGamma)
{
	FPlane MinValue(0.0f, 0.0f, 0.0f, 0.0f);
	FPlane MaxValue(1.0f, 1.0f, 1.0f, 1.0f);

	for (uint32 Y = 0; Y < Height; Y++)
	{
		float* SrcPtr = (float*)(In + Y * SrcPitch);

		for (uint32 X = 0; X < Width; X++)
		{
			MinValue.X = FMath::Min<float>(SrcPtr[0], MinValue.X);
			MinValue.Y = FMath::Min<float>(SrcPtr[1], MinValue.Y);
			MinValue.Z = FMath::Min<float>(SrcPtr[2], MinValue.Z);
			MinValue.W = FMath::Min<float>(SrcPtr[3], MinValue.W);
			MaxValue.X = FMath::Max<float>(SrcPtr[0], MaxValue.X);
			MaxValue.Y = FMath::Max<float>(SrcPtr[1], MaxValue.Y);
			MaxValue.Z = FMath::Max<float>(SrcPtr[2], MaxValue.Z);
			MaxValue.W = FMath::Max<float>(SrcPtr[3], MaxValue.W);
			SrcPtr += 4;
		}
	}

	for (uint32 Y = 0; Y < Height; Y++)
	{
		float* SrcPtr = (float*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr =
				FLinearColor(
				(SrcPtr[0] - MinValue.X) / (MaxValue.X - MinValue.X),
					(SrcPtr[1] - MinValue.Y) / (MaxValue.Y - MinValue.Y),
					(SrcPtr[2] - MinValue.Z) / (MaxValue.Z - MinValue.Z),
					(SrcPtr[3] - MinValue.W) / (MaxValue.W - MinValue.W)
				).ToFColor(LinearToGamma);
			SrcPtr += 4;
			++DestPtr;
		}
	}
}


inline void ConvertRawR24G8DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out, FReadSurfaceDataFlags InFlags)
{
	bool bLinearToGamma = InFlags.GetLinearToGamma();
	// Depth stencil
	for (uint32 Y = 0; Y < Height; Y++)
	{
		uint32* SrcPtr = (uint32 *)In;
		FColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			FColor NormalizedColor;
			if (InFlags.GetOutputStencil())
			{
				uint8 DeviceStencil = (*SrcPtr & 0xFF000000) >> 24;
				NormalizedColor = FColor(DeviceStencil, DeviceStencil, DeviceStencil, 0xFF);
			}
			else
			{
				float DeviceZ = (*SrcPtr & 0xffffff) / (float)(1 << 24);
				float LinearValue = FMath::Min(InFlags.ComputeNormalizedDepth(DeviceZ), 1.0f);
				NormalizedColor = FLinearColor(LinearValue, LinearValue, LinearValue, 0).ToFColor(bLinearToGamma);
			}

			*DestPtr = NormalizedColor;
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawDepthStencil64DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out, FReadSurfaceDataFlags InFlags)
{
	bool bLinearToGamma = InFlags.GetLinearToGamma();
	for (uint32 Y = 0; Y < Height; Y++)
	{
		float* SrcPtr = (float *)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			float DeviceZ = (*SrcPtr);

			float LinearValue = FMath::Min(InFlags.ComputeNormalizedDepth(DeviceZ), 1.0f);

			*DestPtr = FLinearColor(LinearValue, LinearValue, LinearValue, 0).ToFColor(bLinearToGamma);
			SrcPtr += 1; // todo: copies only depth, need to check how this format is read
			++DestPtr;
			UE_LOGF(LogRHI, Warning, "CPU read of R32G8X24 is not tested and may not function.");
		}
	}
}

inline void ConvertRawR16G16B16A16DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out, bool bLinearToGamma = false)
{
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FRHIRGBA16* SrcPtr = (FRHIRGBA16*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = FLinearColor(
				FColor::DequantizeUNorm16ToFloat(SrcPtr->R),
				FColor::DequantizeUNorm16ToFloat(SrcPtr->G),
				FColor::DequantizeUNorm16ToFloat(SrcPtr->B),
				FColor::DequantizeUNorm16ToFloat(SrcPtr->A)
			).ToFColor(bLinearToGamma);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR16G16DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out)
{
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FRHIRG16* SrcPtr = (FRHIRG16*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = FColor(
				FColor::Requantize16to8(SrcPtr->R),
				FColor::Requantize16to8(SrcPtr->G),
				0);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR8DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out)
{
	for (uint32 Y = 0; Y < Height; Y++)
	{
		uint8* SrcPtr = (uint8*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = FColor(*SrcPtr, *SrcPtr, *SrcPtr, *SrcPtr);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR8G8DataToFColor(uint32 Width, uint32 Height, uint8* In, uint32 SrcPitch, FColor* Out)
{
	for (uint32 Y = 0; Y < Height; Y++)
	{
		uint8* SrcPtr = (uint8*)(In + Y * SrcPitch);
		FColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = FColor(*SrcPtr, *(SrcPtr + 1), 0);
			SrcPtr += 2;
			++DestPtr;
		}
	}
}

inline void ConvertRawD32S8DataToFColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out, FReadSurfaceDataFlags InFlags)
{
	bool bLinearToGamma = InFlags.GetLinearToGamma();
	// Depth
	if (!InFlags.GetOutputStencil())
	{
		for (uint32 Y = 0; Y < Height; Y++)
		{
			uint32* SrcPtr = (uint32*)(In + Y * SrcPitch);
			FColor* DestPtr = Out + Y * Width;

			for (uint32 X = 0; X < Width; X++)
			{
				float DeviceZ = *SrcPtr;
				float LinearValue = FMath::Min(InFlags.ComputeNormalizedDepth(DeviceZ), 1.0f);
				*DestPtr = FLinearColor(LinearValue, LinearValue, LinearValue, 0).ToFColor(bLinearToGamma);

				++DestPtr;
				++SrcPtr;
			}
		}
	}
	// Stencil
	else
	{
		// Depth stencil
		for (uint32 Y = 0; Y < Height; Y++)
		{
			uint8* SrcPtr = (uint8*)(In + Y * SrcPitch);
			FColor* DestPtr = Out + Y * Width;

			for (uint32 X = 0; X < Width; X++)
			{
				uint8 DeviceStencil = *SrcPtr;
				*DestPtr = FColor(DeviceStencil, DeviceStencil, DeviceStencil, 0xFF);

				++SrcPtr;
				++DestPtr;
			}
		}
	}
}

// Linear functions

inline void ConvertRawR16UDataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out)
{
	// e.g. shadow maps
	for (uint32 Y = 0; Y < Height; Y++)
	{
		uint16* SrcPtr = (uint16*)(In + Y * SrcPitch);
		FLinearColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			uint16 Value16 = *SrcPtr;
			float Value = Value16 * (1.f/0xffff);

			*DestPtr = FLinearColor(Value, Value, Value);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR16FDataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out)
{
	// e.g. shadow maps
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FFloat16 * SrcPtr = (FFloat16 *)(In + Y * SrcPitch);
		FLinearColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			float Value = SrcPtr->GetFloat();

			*DestPtr = FLinearColor(Value, Value, Value);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR8G8B8A8DataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out)
{
	// Read the data out of the buffer, converting it from ABGR to ARGB.
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FColor* SrcPtr = (FColor*)(In + Y * SrcPitch);
		FLinearColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			FColor sRGBColor = FColor(SrcPtr->B, SrcPtr->G, SrcPtr->R, SrcPtr->A); // swap RB
			// FLinearColor(FColor) does SRGB -> Linear
			*DestPtr = FLinearColor(sRGBColor);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawB8G8R8A8DataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out)
{
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FColor* SrcPtr = (FColor*)(In + Y * SrcPitch);
		FLinearColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			// FLinearColor(FColor) does SRGB -> Linear
			*DestPtr = FLinearColor(*SrcPtr);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawA2B10G10R10DataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out)
{
	// Read the data out of the buffer, converting it from R10G10B10A2 to FLinearColor.
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FRHIR10G10B10A2* SrcPtr = (FRHIR10G10B10A2*)(In + Y * SrcPitch);
		FLinearColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = FLinearColor(
				(float)SrcPtr->R / 1023.0f,
				(float)SrcPtr->G / 1023.0f,
				(float)SrcPtr->B / 1023.0f,
				(float)SrcPtr->A / 3.0f
			);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR16G16B16A16FDataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out, FReadSurfaceDataFlags InFlags)
{
	if (InFlags.GetCompressionMode() == RCM_MinMax)
	{
		for (uint32 Y = 0; Y < Height; Y++)
		{
			FFloat16* SrcPtr = (FFloat16*)(In + Y * SrcPitch);
			FLinearColor* DestPtr = Out + Y * Width;

			for (uint32 X = 0; X < Width; X++)
			{
				*DestPtr = FLinearColor((float)SrcPtr[0], (float)SrcPtr[1], (float)SrcPtr[2], (float)SrcPtr[3]);
				++DestPtr;
				SrcPtr += 4;
			}
		}
	}
	else
	{
		FPlane	MinValue(0.0f, 0.0f, 0.0f, 0.0f);
		FPlane	MaxValue(1.0f, 1.0f, 1.0f, 1.0f);

		check(sizeof(FFloat16) == sizeof(uint16));

		for (uint32 Y = 0; Y < Height; Y++)
		{
			FFloat16* SrcPtr = (FFloat16*)(In + Y * SrcPitch);

			for (uint32 X = 0; X < Width; X++)
			{
				MinValue.X = FMath::Min<float>(SrcPtr[0], MinValue.X);
				MinValue.Y = FMath::Min<float>(SrcPtr[1], MinValue.Y);
				MinValue.Z = FMath::Min<float>(SrcPtr[2], MinValue.Z);
				MinValue.W = FMath::Min<float>(SrcPtr[3], MinValue.W);
				MaxValue.X = FMath::Max<float>(SrcPtr[0], MaxValue.X);
				MaxValue.Y = FMath::Max<float>(SrcPtr[1], MaxValue.Y);
				MaxValue.Z = FMath::Max<float>(SrcPtr[2], MaxValue.Z);
				MaxValue.W = FMath::Max<float>(SrcPtr[3], MaxValue.W);
				SrcPtr += 4;
			}
		}

		for (uint32 Y = 0; Y < Height; Y++)
		{
			FFloat16* SrcPtr = (FFloat16*)(In + Y * SrcPitch);
			FLinearColor* DestPtr = Out + Y * Width;

			for (uint32 X = 0; X < Width; X++)
			{
				*DestPtr = FLinearColor(
					(SrcPtr[0] - MinValue.X) / (MaxValue.X - MinValue.X),
					(SrcPtr[1] - MinValue.Y) / (MaxValue.Y - MinValue.Y),
					(SrcPtr[2] - MinValue.Z) / (MaxValue.Z - MinValue.Z),
					(SrcPtr[3] - MinValue.W) / (MaxValue.W - MinValue.W)
				);
				++DestPtr;
				SrcPtr += 4;
			}
		}
	}
}


inline void ConvertRawR11G11B10FDataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out)
{
	check(sizeof(FFloat3Packed) == sizeof(uint32));

	for (uint32 Y = 0; Y < Height; Y++)
	{
		FFloat3Packed* SrcPtr = (FFloat3Packed*)(In + Y * SrcPitch);
		FLinearColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = (*SrcPtr).ToLinearColor();
			++DestPtr;
			++SrcPtr;
		}
	}
}

inline void ConvertRawR32G32B32A32DataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out, FReadSurfaceDataFlags InFlags)
{
	if (InFlags.GetCompressionMode() == RCM_MinMax)
	{
		// Copy data directly, respecting existing min-max values
		FLinearColor* SrcPtr = (FLinearColor*)In;
		FLinearColor* DestPtr = (FLinearColor*)Out;
		const int32 ImageSize = sizeof(FLinearColor) * Height * Width;

		FMemory::Memcpy(DestPtr, SrcPtr, ImageSize);
	}
	else
	{
		// Normalize data
		FPlane MinValue(0.0f, 0.0f, 0.0f, 0.0f);
		FPlane MaxValue(1.0f, 1.0f, 1.0f, 1.0f);

		for (uint32 Y = 0; Y < Height; Y++)
		{
			float* SrcPtr = (float*)(In + Y * SrcPitch);

			for (uint32 X = 0; X < Width; X++)
			{
				MinValue.X = FMath::Min<float>(SrcPtr[0], MinValue.X);
				MinValue.Y = FMath::Min<float>(SrcPtr[1], MinValue.Y);
				MinValue.Z = FMath::Min<float>(SrcPtr[2], MinValue.Z);
				MinValue.W = FMath::Min<float>(SrcPtr[3], MinValue.W);
				MaxValue.X = FMath::Max<float>(SrcPtr[0], MaxValue.X);
				MaxValue.Y = FMath::Max<float>(SrcPtr[1], MaxValue.Y);
				MaxValue.Z = FMath::Max<float>(SrcPtr[2], MaxValue.Z);
				MaxValue.W = FMath::Max<float>(SrcPtr[3], MaxValue.W);
				SrcPtr += 4;
			}
		}

		float* SrcPtr = (float*)In;

		for (uint32 Y = 0; Y < Height; Y++)
		{
			FLinearColor* DestPtr = Out + Y * Width;

			for (uint32 X = 0; X < Width; X++)
			{
				*DestPtr = FLinearColor(
					(SrcPtr[0] - MinValue.X) / (MaxValue.X - MinValue.X),
					(SrcPtr[1] - MinValue.Y) / (MaxValue.Y - MinValue.Y),
					(SrcPtr[2] - MinValue.Z) / (MaxValue.Z - MinValue.Z),
					(SrcPtr[3] - MinValue.W) / (MaxValue.W - MinValue.W)
				);
				++DestPtr;
				SrcPtr += 4;
			}
		}
	}
}

inline void ConvertRawR24G8DataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out, FReadSurfaceDataFlags InFlags)
{
	// Depth stencil
	for (uint32 Y = 0; Y < Height; Y++)
	{
		uint32* SrcPtr = (uint32 *)In;
		FLinearColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			float DeviceStencil = 0.0f;
			DeviceStencil = (float)((*SrcPtr & 0xFF000000) >> 24) / 255.0f;
			float DeviceZ = (*SrcPtr & 0xffffff) / (float)(1 << 24);
			float LinearValue = FMath::Min(InFlags.ComputeNormalizedDepth(DeviceZ), 1.0f);
			*DestPtr = FLinearColor(LinearValue, DeviceStencil, 0.0f, 0.0f);
			++DestPtr;
			++SrcPtr;
		}
	}
}

inline void ConvertRawDepthStencil64DataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out, FReadSurfaceDataFlags InFlags)
{
	// Depth stencil
	for (uint32 Y = 0; Y < Height; Y++)
	{
		uint8* SrcStart = (uint8 *)(In + Y * SrcPitch);
		FLinearColor* DestPtr = Out + Y * Width;

		for (uint32 X = 0; X < Width; X++)
		{
			float DeviceZ = *((float *)(SrcStart));
			float LinearValue = FMath::Min(InFlags.ComputeNormalizedDepth(DeviceZ), 1.0f);
			float DeviceStencil = (float)(*(SrcStart + 4)) / 255.0f;
			*DestPtr = FLinearColor(LinearValue, DeviceStencil, 0.0f, 0.0f);
			SrcStart += 8; //64 bit format with the last 24 bit ignore
			++DestPtr;
		}
	}
}

inline void ConvertRawR16G16B16A16DataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out)
{
	// Read the data out of the buffer, converting it to FLinearColor.
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FRHIRGBA16* SrcPtr = (FRHIRGBA16*)(In + Y * SrcPitch);
		FLinearColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = FLinearColor(
				(float)SrcPtr->R / 65535.0f,
				(float)SrcPtr->G / 65535.0f,
				(float)SrcPtr->B / 65535.0f,
				(float)SrcPtr->A / 65535.0f
			);
			++SrcPtr;
			++DestPtr;
		}
	}
}

inline void ConvertRawR16G16DataToFLinearColor(uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out)
{
	// Read the data out of the buffer, converting it to FLinearColor.
	for (uint32 Y = 0; Y < Height; Y++)
	{
		FRHIRG16* SrcPtr = (FRHIRG16*)(In + Y * SrcPitch);
		FLinearColor* DestPtr = Out + Y * Width;
		for (uint32 X = 0; X < Width; X++)
		{
			*DestPtr = FLinearColor(
				(float)SrcPtr->R / 65535.0f,
				(float)SrcPtr->G / 65535.0f,
				0);
			++SrcPtr;
			++DestPtr;
		}
	}
}

bool ConvertRAWSurfaceDataToFLinearColor(EPixelFormat Format, uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FLinearColor* Out, FReadSurfaceDataFlags InFlags)
{
	// InFlags.GetLinearToGamma() is ignored by the FLinearColor reader

	// Flags RCM_MinMax means pass the values out unchanged
	//	default flags RCM_UNorm rescales them to [0,1] if they were outside that range

	if (Format == PF_G16 || Format == PF_R16_UINT )
	{
		ConvertRawR16UDataToFLinearColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_R16F || Format == PF_R16F_FILTER)
	{
		ConvertRawR16FDataToFLinearColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_R8G8B8A8)
	{
		ConvertRawR8G8B8A8DataToFLinearColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_B8G8R8A8)
	{
		ConvertRawB8G8R8A8DataToFLinearColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_A2B10G10R10)
	{
		ConvertRawA2B10G10R10DataToFLinearColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_FloatRGBA)
	{
		ConvertRawR16G16B16A16FDataToFLinearColor(Width, Height, In, SrcPitch, Out, InFlags);
		return true;
	}
	else if (Format == PF_FloatRGB || Format == PF_FloatR11G11B10)
	{
		// assume FloatRGB == PF_FloatR11G11B10 always here
		check( GPixelFormats[Format].BlockBytes == 4 );
		ConvertRawR11G11B10FDataToFLinearColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_A32B32G32R32F)
	{
		ConvertRawR32G32B32A32DataToFLinearColor(Width, Height, In, SrcPitch, Out, InFlags);
		return true;
	}
	else if ( Format == PF_D24 ||
		( (Format == PF_X24_G8 || Format == PF_DepthStencil ) && GPixelFormats[Format].BlockBytes == 4 )
		)
	{
		//	see CVarD3D11UseD24/CVarD3D12UseD24
		ConvertRawR24G8DataToFLinearColor(Width, Height, In, SrcPitch, Out, InFlags);
		return true;
	}
	else if ( (Format == PF_X24_G8 || Format == PF_DepthStencil ) && GPixelFormats[Format].BlockBytes > 4 )
	{
		// @@ D3D 11/12 different?

		/**
		
		D3D11 and 12 formats are almost the same, one difference :

		DepthStencil formats are interleaved	DepthStencil formats are planar.
		For example a format of 24 bits of depth, 8 bits of stencil would be
		stored in the format 24/8/24/8... etc in Direct3D 11, but as 24/24/24...
		followed by 8/8/8... in Direct3D 12. Note that each plane is its own
		subresource in D3D12

		**/

		ConvertRawDepthStencil64DataToFLinearColor(Width, Height, In, SrcPitch, Out, InFlags);
		return true;
	}
	else if (Format == PF_A16B16G16R16)
	{
		ConvertRawR16G16B16A16DataToFLinearColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_G16R16)
	{
		ConvertRawR16G16DataToFLinearColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_G16R16F)
	{
		// Read the data out of the buffer, converting it to FLinearColor.
		for (uint32 Y = 0; Y < Height; Y++)
		{
			FFloat16 * SrcPtr = (FFloat16*)(In + Y * SrcPitch);
			FLinearColor* DestPtr = Out + Y * Width;
			for (uint32 X = 0; X < Width; X++)
			{
				*DestPtr = FLinearColor( SrcPtr[0].GetFloat(), SrcPtr[1].GetFloat(), 0.f,1.f);
				SrcPtr += 2;
				++DestPtr;
			}
		}
		return true;
	}
	else if (Format == PF_G32R32F)
	{
		// not doing MinMax/Unorm remap here
	
		// Read the data out of the buffer, converting it to FLinearColor.
		for (uint32 Y = 0; Y < Height; Y++)
		{
			float * SrcPtr = (float *)(In + Y * SrcPitch);
			FLinearColor* DestPtr = Out + Y * Width;
			for (uint32 X = 0; X < Width; X++)
			{
				*DestPtr = FLinearColor( SrcPtr[0], SrcPtr[1], 0.f, 1.f );
				SrcPtr += 2;
				++DestPtr;
			}
		}
		return true;
	}
	else if (Format == PF_R32_FLOAT)
	{
		// not doing MinMax/Unorm remap here
	
		// Read the data out of the buffer, converting it to FLinearColor.
		for (uint32 Y = 0; Y < Height; Y++)
		{
			float * SrcPtr = (float *)(In + Y * SrcPitch);
			FLinearColor* DestPtr = Out + Y * Width;
			for (uint32 X = 0; X < Width; X++)
			{
				*DestPtr = FLinearColor( SrcPtr[0], 0.f, 0.f, 1.f );
				++SrcPtr;
				++DestPtr;
			}
		}
		return true;
	}
	else
	{
		// not supported yet
		checkf(0, TEXT("Surface data to linear color conversion not supported for format %s"), GetPixelFormatString(Format));
		return false;
	}
}

bool ConvertRAWSurfaceDataToFColor(EPixelFormat Format, uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out, FReadSurfaceDataFlags InFlags)
{
	bool bLinearToGamma = InFlags.GetLinearToGamma();

	if (Format == PF_G16 || Format == PF_R16_UINT)
	{
		ConvertRawR16DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_R8G8B8A8)
	{
		ConvertRawR8G8B8A8DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_B8G8R8A8)
	{
		ConvertRawB8G8R8A8DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_A2B10G10R10)
	{
		ConvertRawR10G10B10A2DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_FloatRGBA)
	{
		ConvertRawR16G16B16A16FDataToFColor(Width, Height, In, SrcPitch, Out, bLinearToGamma);
		return true;
	}
	else if (Format == PF_FloatRGB || Format == PF_FloatR11G11B10)
	{
		ConvertRawR11G11B10DataToFColor(Width, Height, In, SrcPitch, Out, bLinearToGamma);
		return true;
	}
	else if (Format == PF_A32B32G32R32F)
	{
		ConvertRawR32G32B32A32DataToFColor(Width, Height, In, SrcPitch, Out, bLinearToGamma);
		return true;
	}
	else if (Format == PF_D24 || ((Format == PF_X24_G8 || Format == PF_DepthStencil) && GPixelFormats[Format].BlockBytes == 4))
	{
		ConvertRawR24G8DataToFColor(Width, Height, In, SrcPitch, Out, InFlags);
		return true;
	}
	else if (Format == PF_A16B16G16R16)
	{
		ConvertRawR16G16B16A16DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_G16R16)
	{
		ConvertRawR16G16DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_G8)
	{
		ConvertRawR8DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else if (Format == PF_R8G8)
	{
		ConvertRawR8G8DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	}
	else
	{
		// not supported yet
		checkf(0, TEXT("Surface data to color conversion not supported for format %s"), GetPixelFormatString(Format));
		return false;
	}
}

#if defined(DXGI_FORMAT_DEFINED) && DXGI_FORMAT_DEFINED

// switching on platform format
// could switch on EPixelFormat instead and make this more generic
// have to be a little careful with that as the mapping is not one to one

/** Convert D3D format type to general pixel format type*/
bool ConvertDXGIToFColor(DXGI_FORMAT Format, uint32 Width, uint32 Height, uint8 *In, uint32 SrcPitch, FColor* Out, FReadSurfaceDataFlags InFlags)
{
	// todo : use a helper to map all the typeless to unorm before switching here ; see DXGIUtilities

	bool bLinearToGamma = InFlags.GetLinearToGamma();
	switch (Format)
	{
	case DXGI_FORMAT_R16_TYPELESS:
	case DXGI_FORMAT_R16_UNORM:
		ConvertRawR16DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	case DXGI_FORMAT_R8G8B8A8_TYPELESS:
	case DXGI_FORMAT_R8G8B8A8_UNORM:
	case DXGI_FORMAT_R8G8B8A8_UNORM_SRGB:
		ConvertRawR8G8B8A8DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	case DXGI_FORMAT_B8G8R8A8_TYPELESS:
	case DXGI_FORMAT_B8G8R8A8_UNORM:
	case DXGI_FORMAT_B8G8R8A8_UNORM_SRGB:
		ConvertRawB8G8R8A8DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	case DXGI_FORMAT_R10G10B10A2_UNORM:
		ConvertRawR10G10B10A2DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	case DXGI_FORMAT_R16G16B16A16_FLOAT:
		ConvertRawR16G16B16A16FDataToFColor(Width, Height, In, SrcPitch, Out, bLinearToGamma);
		return true;
	case DXGI_FORMAT_R11G11B10_FLOAT:
		ConvertRawR11G11B10DataToFColor(Width, Height, In, SrcPitch, Out, bLinearToGamma);
		return true;
	case DXGI_FORMAT_R9G9B9E5_SHAREDEXP:
		ConvertRawR9G9B9E5DataToFColor(Width, Height, In, SrcPitch, Out, bLinearToGamma);
		return true;
	case DXGI_FORMAT_R32G32B32A32_FLOAT:
		ConvertRawR32G32B32A32DataToFColor(Width, Height, In, SrcPitch, Out, bLinearToGamma);
		return true;
	case DXGI_FORMAT_R24G8_TYPELESS:
		ConvertRawR24G8DataToFColor(Width, Height, In, SrcPitch, Out, InFlags);
		return true;
	case DXGI_FORMAT_R32G8X24_TYPELESS:
		ConvertRawDepthStencil64DataToFColor(Width, Height, In, SrcPitch, Out, InFlags);
		return true;
	case DXGI_FORMAT_R16G16B16A16_UNORM:
		ConvertRawR16G16B16A16DataToFColor(Width, Height, In, SrcPitch, Out, bLinearToGamma);
		return true;
	case DXGI_FORMAT_R16G16_UNORM:
		ConvertRawR16G16DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	case DXGI_FORMAT_R8_UNORM:
		ConvertRawR8DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	case DXGI_FORMAT_R8G8_UNORM:
		ConvertRawR8G8DataToFColor(Width, Height, In, SrcPitch, Out);
		return true;
	default:
		return false;
	}
}

#endif // DXGI_FORMAT_DEFINED

}; // namespace
