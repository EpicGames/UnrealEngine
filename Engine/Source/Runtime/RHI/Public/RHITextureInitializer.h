// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIResources.h"
#include "Templates/Function.h"

// 'Minimal' description of an RHITexture, primarily used by FRHITextureInitializer to keep overhead low.
struct FRHITextureMinimalDesc
{
	FIntPoint         Extent = FIntPoint(1, 1);
	uint16            Depth = 1;
	uint16            ArraySize = 1;
	ETextureDimension Dimension = ETextureDimension::Texture2D;
	EPixelFormat      Format = PF_Unknown;

	FRHITextureMinimalDesc() = default;
	FRHITextureMinimalDesc(const FRHITextureDesc& InDesc)
		: Extent(InDesc.Extent)
		, Depth(InDesc.Depth)
		, ArraySize(InDesc.ArraySize)
		, Dimension(InDesc.Dimension)
		, Format(InDesc.Format)
	{
	}
};

struct FRHITextureSubresourceInitializer
{
	void WriteData(const void* InSource, size_t InSize)
	{
		FMemory::Memcpy(Data, InSource, InSize);
	}

	void WriteColor(FColor InColor)
	{
		WriteData(&InColor, sizeof(InColor));
	}

	void*  Data;
	uint64 Size;
	uint64 Stride;
};

// Structure used to allow optimal texture initialization at creation time.
// Should only ever be obtained by calling RHICreateTextureInitializer().
// Texture data writing has to be done on individual subresources via the GetSubresource accesors.
// NO COPIES ALLOWED
struct FRHITextureInitializer
{
	FRHITextureInitializer() = default;
	FRHITextureInitializer(FRHITextureInitializer&& InOther)
		: FinalizeCallback(MoveTemp(InOther.FinalizeCallback))
		, GetSubresourceCallback(MoveTemp(InOther.GetSubresourceCallback))
		, CommandList(InOther.CommandList)
		, Texture(InOther.Texture)
		, WritableData(InOther.WritableData)
		, WritableSize(InOther.WritableSize)
		, Desc(InOther.Desc)
	{
		InOther.Reset();
	}
	~FRHITextureInitializer()
	{
		if (CommandList && Texture)
		{
			RemovePendingTextureUpload();
		}
	}

	const FRHITextureMinimalDesc& GetDesc() const
	{
		return Desc;
	}

	struct FSubresourceIndex
	{
		int32 FaceIndex = 0;
		int32 ArrayIndex = 0;
		int32 MipIndex = 0;
	};

	// Get a subresource for any texture type.
	FRHITextureSubresourceInitializer GetSubresource(FSubresourceIndex SubresourceIndex)
	{
		checkf(GetSubresourceCallback, TEXT("Attempting to call GetSubresourceCallback when it is null, make sure you set your InitAction to Initializer."));
		return GetSubresourceCallback(SubresourceIndex);
	}

	// Get a subresource for a 2D texture. The only subresources are mip levels.
	FRHITextureSubresourceInitializer GetTexture2DSubresource(int32 MipIndex)
	{
		FSubresourceIndex SubresourceIndex;
		SubresourceIndex.MipIndex = MipIndex;
		return GetSubresource(SubresourceIndex);
	}

	// Get a subresource for a 2D texture array. Subresources are individual array slice mip levels.
	FRHITextureSubresourceInitializer GetTexture2DArraySubresource(int32 ArrayIndex, int32 MipIndex)
	{
		FSubresourceIndex SubresourceIndex;
		SubresourceIndex.ArrayIndex = ArrayIndex;
		SubresourceIndex.MipIndex = MipIndex;
		return GetSubresource(SubresourceIndex);
	}

	// Get a subresource for a 3D texture. The only subresources are mip levels.
	FRHITextureSubresourceInitializer GetTexture3DSubresource(int32 MipIndex)
	{
		FSubresourceIndex SubresourceIndex;
		SubresourceIndex.MipIndex = MipIndex;
		return GetSubresource(SubresourceIndex);
	}

	// Get a subresource for a cube texture. Subresources are individual face mip levels.
	FRHITextureSubresourceInitializer GetTextureCubeSubresource(int32 FaceIndex, int32 MipIndex)
	{
		FSubresourceIndex SubresourceIndex;
		SubresourceIndex.FaceIndex = FaceIndex;
		SubresourceIndex.MipIndex = MipIndex;
		return GetSubresource(SubresourceIndex);
	}

	// Get a subresource for a cube texture array. Subresources are individual array slice face mip levels.
	FRHITextureSubresourceInitializer GetTextureCubeArraySubresource(int32 FaceIndex, int32 ArrayIndex, int32 MipIndex)
	{
		FSubresourceIndex SubresourceIndex;
		SubresourceIndex.FaceIndex = FaceIndex;
		SubresourceIndex.ArrayIndex = ArrayIndex;
		SubresourceIndex.MipIndex = MipIndex;
		return GetSubresource(SubresourceIndex);
	}

	// 'Finalizes' the initializer and returns the created FRHITexture. The initializer will be reset to an invalid state and should not be used after calling this.
	RHI_API FTextureRHIRef Finalize();

protected:
	// @todo dev-pr switch to using IRHIUploadContext
	using FFinalizeCallback = TUniqueFunction<FTextureRHIRef(FRHICommandListBase&)>;

	// FaceIndex, ArrayIndex, MipIndex
	using FGetSubresourceCallback = TUniqueFunction<FRHITextureSubresourceInitializer(FSubresourceIndex)>;

	// Should only be called by RHI derived types.
	RHI_API FRHITextureInitializer(FRHICommandListBase& RHICmdList, FRHITexture* InTexture, void* InWritableData, uint64 InWritableSize, FFinalizeCallback&& InFinalizeCallback, FGetSubresourceCallback&& InGetSubresourceCallback);

	// Remove the texture from the command list. Has to be in cpp file to prevent circular header dependency.
	RHI_API void RemovePendingTextureUpload();

	// Allow copies only for RHI derived types.
	FRHITextureInitializer(const FRHITextureInitializer&) = delete;
	FRHITextureInitializer& operator=(const FRHITextureInitializer&) = delete;
	FRHITextureInitializer& operator=(FRHITextureInitializer&&) = delete;

	void Reset()
	{
		FinalizeCallback = {};
		GetSubresourceCallback = {};
		CommandList = {};
		Texture = {};
		WritableData = {};
		WritableSize = {};
		Desc = FRHITextureMinimalDesc();
	}

protected:
	// RHI provided lambda to call when done writing data, returns the created Texture
	// Only used by the RHI internals, should not be accessed outside of RHIs
	FFinalizeCallback FinalizeCallback = nullptr;

	// RHI provided lambda to provide pointers to and sizes of individual subresources for writing.
	// Only used by the RHI internals, should not be accessed outside of RHIs
	FGetSubresourceCallback GetSubresourceCallback = nullptr;

	// Command list provided on construction, used in finalize.
	FRHICommandListBase* CommandList = nullptr;

	// Current RHI Texture being initialized. Will only be used for command list validation since each RHI implementation will manage their own texture type.
	FRHITexture* Texture = nullptr;

	// Pointer to the writable data provided by the RHI
	void* WritableData = nullptr;

	// Size of the writable data provided by the RHI
	uint64 WritableSize = 0;

	// Description of the texture being created.
	FRHITextureMinimalDesc Desc{};
};
