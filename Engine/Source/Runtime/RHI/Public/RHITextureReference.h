// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIResources.h"

class FRHITextureReference : public FRHITexture
{
public:
	FRHITextureReference() = delete;
	RHI_API FRHITextureReference(FRHITexture* InReferencedTexture);
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	RHI_API FRHITextureReference(FRHITexture* InReferencedTexture, FRHIDescriptorHandle InBindlessHandle);
#endif
	RHI_API ~FRHITextureReference();

	RHI_API virtual class FRHITextureReference* GetTextureReference() override;
	RHI_API virtual FRHIDescriptorHandle GetDefaultBindlessHandle() const override;

	RHI_API virtual void* GetNativeResource() const override;
	RHI_API virtual void* GetNativeShaderResourceView() const override;
	RHI_API virtual void* GetTextureBaseRHI() override;
	RHI_API virtual void GetWriteMaskProperties(void*& OutData, uint32& OutSize) override;
	RHI_API virtual const FRHITextureDesc& GetDesc() const override;

#if ENABLE_RHI_VALIDATION
	// Implement RHIValidation::FTextureResource::GetTrackerResource to use the tracker info
	// for the referenced texture.
	RHI_API virtual RHIValidation::FResource* GetTrackerResource() final override;
#endif

	inline FRHITexture* GetReferencedTexture() const { return ReferencedTexture.GetReference(); }

	static inline FRHITexture* GetDefaultTexture() { return DefaultTexture; }

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FRHIDescriptorHandle GetBindlessHandle() const { return BindlessHandle; }
	bool                 IsBindless()        const { return GetBindlessHandle().IsValid(); }
#endif

protected:
	friend class FDynamicRHI;

	// Only called from Platform Context
	RHI_API virtual void SetTrackedAccessFromContext(FRHITrackedAccess TrackedAccess) override;

	// Called only from FDynamicRHI::RHIUpdateTextureReference
	void SetReferencedTexture(FRHITexture* InTexture)
	{
		ReferencedTexture = InTexture;
	}

	TRefCountPtr<FRHITexture> ReferencedTexture;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	const FRHIDescriptorHandle BindlessHandle;
#endif

	// This pointer is set by the InitRHI() function on the FBlackTextureWithSRV global resource,
	// to allow FRHITextureReference to use the global black texture when the reference is nullptr.
	// A pointer is required since FBlackTextureWithSRV is defined in RenderCore.
	friend class FBlackTextureWithSRV;
	RHI_API static TRefCountPtr<FRHITexture> DefaultTexture;
};
