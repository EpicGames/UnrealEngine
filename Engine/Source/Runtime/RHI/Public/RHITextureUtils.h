// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIResources.h"

namespace UE::RHITextureUtils
{
	static uint32 CalculateMipExtent(uint32 Extent, uint32 MipIndex)
	{
		return FMath::Max<uint32>(Extent >> MipIndex, 1);
	}

	static uint32 CalculateMipBlockCount(uint32 Extent, uint32 MipIndex, uint32 BlockSize)
	{
		return FMath::Max<uint32>(1, FMath::DivideAndRoundUp(CalculateMipExtent(Extent, MipIndex), BlockSize));
	}

	static FUintVector3 CalculateMipBlockCounts(const FRHITextureDesc& Desc, uint32 MipIndex, const FPixelFormatInfo& PixelFormat)
	{
		const uint32 NumBlocksX = UE::RHITextureUtils::CalculateMipBlockCount(Desc.Extent.X, MipIndex, PixelFormat.BlockSizeX);
		const uint32 NumBlocksY = UE::RHITextureUtils::CalculateMipBlockCount(Desc.Extent.Y, MipIndex, PixelFormat.BlockSizeY);
		const uint32 NumBlocksZ = UE::RHITextureUtils::CalculateMipBlockCount(Desc.Depth,    MipIndex, PixelFormat.BlockSizeZ);

		return FUintVector3(NumBlocksX, NumBlocksY, NumBlocksZ);
	}

	static FUintVector3 CalculateMipBlockCounts(const FRHITextureDesc& Desc, uint32 MipIndex)
	{
		const FPixelFormatInfo& PixelFormat = GPixelFormats[Desc.Format];
		return CalculateMipBlockCounts(Desc, MipIndex, PixelFormat);
	}

	static FUintVector3 CalculateMipExtents(const FRHITextureDesc& Desc, uint32 MipIndex)
	{
		return FUintVector3(
			CalculateMipExtent(Desc.Extent.X, MipIndex),
			CalculateMipExtent(Desc.Extent.Y, MipIndex),
			CalculateMipExtent(Desc.Depth,    MipIndex)
		);
	}

	static uint64 CalculateTextureMipSize(const FRHITextureDesc& Desc, uint32 MipIndex, uint64& OutStride)
	{
		const FPixelFormatInfo& PixelFormat = GPixelFormats[Desc.Format];

		const FUintVector3 BlockCounts = CalculateMipBlockCounts(Desc, MipIndex, PixelFormat);
		const uint64 MipStride = BlockCounts.X * PixelFormat.BlockBytes;
		const uint64 SubresourceSize = MipStride * BlockCounts.Y * BlockCounts.Z;

		OutStride = MipStride;
		return SubresourceSize;
	}

	static uint64 CalculateTextureMipSize(const FRHITextureDesc& Desc, uint32 MipIndex)
	{
		uint64 TempStride{};
		return CalculateTextureMipSize(Desc, MipIndex, TempStride);
	}

	static uint64 CalculateTexturePlaneSize(const FRHITextureDesc& Desc)
	{
		uint64 PlaneSize = 0;
		uint64 MipStride = 0;
		for (uint32 MipIndex = 0; MipIndex < Desc.NumMips; MipIndex++)
		{
			PlaneSize += CalculateTextureMipSize(Desc, MipIndex, MipStride);
		}
		return PlaneSize;
	}

	static uint64 CalculateTextureSize(const FRHITextureDesc& Desc)
	{
		const uint64 PlaneSize = CalculateTexturePlaneSize(Desc);
		const uint64 FaceCount = Desc.IsTextureCube() ? 6 : 1;
		return PlaneSize * Desc.ArraySize * Desc.Depth * FaceCount;
	}

	inline void SubresourceIndexToIndices(uint32 SubresourceIndex, uint32 NumMips, uint32 NumSlices, uint32& OutFaceIndex, uint32& OutArrayIndex, uint32& OutMipIndex)
	{
		OutMipIndex = (SubresourceIndex % NumMips);
		OutArrayIndex = (SubresourceIndex / NumMips) % NumSlices;
		OutFaceIndex = (SubresourceIndex / (NumMips % NumSlices));
	}

	static uint64 CalculateSubresourceOffset(const FRHITextureDesc& Desc, int32 FaceIndex, int32 ArrayIndex, int32 MipIndex, uint64& OutStride, uint64& OutSize)
	{
		const uint64 FaceCount = Desc.IsTextureCube() ? 6 : 1;
		const uint64 PlaneIndex = FaceIndex + ArrayIndex * FaceCount;

		uint64 PlaneSize = 0;
		if (PlaneIndex > 0)
		{
			PlaneSize = CalculateTexturePlaneSize(Desc);
		}

		const uint64 PlaneOffset = PlaneIndex * PlaneSize;

		uint64 MipOffset = 0;
		uint64 MipStride = 0;

		for (int32 Index = 0; Index <= MipIndex; Index++)
		{
			const uint64 MipSize = CalculateTextureMipSize(Desc, Index, MipStride);

			if (Index == MipIndex)
			{
				OutSize = MipSize;
				OutStride = MipStride;
			}
			else
			{
				MipOffset += MipSize;
			}
		}

		return PlaneOffset + MipOffset;
	}
}
