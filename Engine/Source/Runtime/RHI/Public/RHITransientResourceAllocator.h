// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"
#include "RHIResources.h"
#include "Tasks/Task.h"

class FRHICommandListBase;
class FRHICommandListImmediate;
class FRHITransientHeap;
class FRHITransientPagePool;

/** This data structure contains fence values used for allocating / deallocating transient memory regions for transient resources. A memory region can
 *  be re-used if the deallocation fences from the discarding resource and the allocation fences for the acquiring resource are not executing simultaneously
 *  on both the graphics | async compute pipe on the GPU timeline.
 *
 *  Allocation events are always on a single pipeline, while deallocation events can happen on multiple pipelines at the same time. Async compute is represented
 *  using three fence values: one for the async compute pipe, and two for the fork / join points on the graphics pipe. If fences are active on both pipes at the
 *  same time, the graphics fence must be contained within the async compute fork / join region.
 */
class FRHITransientAllocationFences
{
public:
	// Returns the fence at which the Acquire operation can occur for the given pair of resources transitioning from Discard -> Acquire.
	static uint32 GetAcquireFence(const FRHITransientAllocationFences& Discard, const FRHITransientAllocationFences& Acquire)
	{
		check(Acquire.IsSinglePipeline());

		if (Discard.Graphics != Invalid)
		{
			// Graphics -> Graphics | AsyncCompute
			if (Discard.AsyncCompute == Invalid)
			{
				return Discard.Graphics;
			}

			// All -> AsyncCompute
			if (Acquire.AsyncCompute != Invalid)
			{
				// All -> AsyncCompute - The acquire graphics fork pass is used because a fence from Graphics -> AsyncCompute after the discard's graphics pass must be present.
				return Acquire.GraphicsForkJoin.Min;
			}
		}

		// AsyncCompute -> AsyncCompute
		if (Acquire.AsyncCompute != Invalid)
		{
			return Discard.AsyncCompute;
		}

		// All | AsyncCompute -> Graphics - The discard graphics fork pass is used because because a fence from AsyncCompute -> Graphics after the discard's async compute pass must be present.
		return Discard.GraphicsForkJoin.Max;
	}

	// Returns whether two regions described by the discard and acquire fences contain each other. If they do, that means the memory would being used by both pipes simultaneously and cannot be aliased.
	static bool Contains(const FRHITransientAllocationFences& Discard, const FRHITransientAllocationFences& Acquire)
	{
		return
			Contains(Discard.GraphicsForkJoin, Acquire.Graphics) ||
			Contains(Acquire.GraphicsForkJoin, Discard.Graphics) ||
			// We have to discard on the graphics pipe, which means we can't alias with async compute resources that have a graphics fork pass earlier than the discard join pass.
			(GRHIGlobals.NeedsTransientDiscardOnGraphicsWorkaround &&
				Discard.GetPipelines() == ERHIPipeline::All &&
				Acquire.GetPipelines() == ERHIPipeline::AsyncCompute &&
				Acquire.GraphicsForkJoin.Min < Discard.GraphicsForkJoin.Max);
	}

	FRHITransientAllocationFences() = default;

	// Expects the bitmask of the pipes this transient allocation was accessed on, which can be different from the fences themselves.
	FRHITransientAllocationFences(ERHIPipeline InPipelines)
		: Pipelines(InPipelines)
	{}

	void SetGraphics(uint32 InGraphics)
	{
		check(EnumHasAnyFlags(Pipelines, ERHIPipeline::Graphics));
		check(!GraphicsForkJoin.IsValid() || Contains(GraphicsForkJoin, InGraphics));
		Graphics = InGraphics;
	}

	void SetAsyncCompute(uint32 InAsyncCompute, TInterval<uint32> InGraphicsForkJoin)
	{
		check(EnumHasAnyFlags(Pipelines, ERHIPipeline::AsyncCompute));
		check(InGraphicsForkJoin.IsValid() && Contains(InGraphicsForkJoin, InAsyncCompute));
		check(Graphics == Invalid || Contains(InGraphicsForkJoin, Graphics));
		AsyncCompute = InAsyncCompute;
		GraphicsForkJoin = InGraphicsForkJoin;
	}

	ERHIPipeline GetPipelines() const
	{
		return Pipelines;
	}

	uint32 GetSinglePipeline() const
	{
		check(IsSinglePipeline());
		return Graphics != Invalid ? Graphics : AsyncCompute;
	}

	bool IsSinglePipeline() const
	{
		return Pipelines != ERHIPipeline::All;
	}

private:
	static bool Contains(TInterval<uint32> Interval, uint32 Element)
	{
		return Interval.IsValid() && Element > Interval.Min && Element < Interval.Max;
	}

	static const uint32 Invalid = std::numeric_limits<uint32>::max();
	uint32 Graphics = Invalid;
	uint32 AsyncCompute = Invalid;
	TInterval<uint32> GraphicsForkJoin;
	ERHIPipeline Pipelines = ERHIPipeline::None;
};

struct FRHITransientPageSpan
{
	// Offset of the span in the page pool in pages. 
	uint16 Offset = 0;

	// Number of pages in the span.
	uint16 Count = 0;
};

/** Represents an allocation from a transient page pool. */
struct FRHITransientPagePoolAllocation
{
	bool IsValid() const { return Pool != nullptr; }

	// The transient page pool which made the allocation.
	FRHITransientPagePool* Pool = nullptr;

	// A unique hash identifying this allocation to the allocator implementation.
	uint64 Hash = 0;

	// The index identifying the allocation to the page pool.
	uint16 SpanIndex = 0;

	// Offsets into the array of spans for the allocator implementation.
	uint16 SpanOffsetMin = 0;
	uint16 SpanOffsetMax = 0;
};

/** Represents a full set of page allocations from multiple page pools. */
struct FRHITransientPageAllocation
{
	// The list of allocations by pool.
	TArray<FRHITransientPagePoolAllocation> PoolAllocations;

	// The full list of spans indexed by each allocation.
	TArray<FRHITransientPageSpan> Spans;
};

/** Represents an allocation from the transient heap. */
struct FRHITransientHeapAllocation
{
	bool IsValid() const { return Size != 0; }

	// Transient heap which made the allocation.
	FRHITransientHeap* Heap = nullptr;

	// Size of the allocation made from the allocator (aligned).
	uint64 Size = 0;

	// Offset in the transient heap; front of the heap starts at 0.
	uint64 Offset = 0;

	// Number of bytes of padding were added to the offset.
	uint32 AlignmentPad = 0;
};

enum class ERHITransientResourceType : uint8
{
	Texture,
	Buffer
};

enum class ERHITransientAllocationType : uint8
{
	Heap,
	Page
};

class FRHITransientResource
{
public:
	static const uint32 kInvalidPassIndex = TNumericLimits<uint32>::Max();

	struct FResourceTaskResult
	{
		TRefCountPtr<FRHIResource> Resource; 
		uint64 GpuVirtualAddress;
	};

	using FResourceTask = UE::Tasks::TTask<FResourceTaskResult>;

	RHI_API FRHITransientResource(
		FRHIResource* InResource,
		uint64 InGpuVirtualAddress,
		uint64 InHash,
		uint64 InSize,
		ERHITransientAllocationType InAllocationType,
		ERHITransientResourceType InResourceType);

	RHI_API FRHITransientResource(
		const FResourceTask& InResourceTask,
		uint64 InHash,
		uint64 InSize,
		ERHITransientAllocationType InAllocationType,
		ERHITransientResourceType InResourceType);

	RHI_API virtual ~FRHITransientResource();

	//////////////////////////////////////////////////////////////////////////////////////////////////
	//! Internal Allocator API

	void Acquire(const TCHAR* InName, uint32 InAcquirePassIndex, ERHIPipeline InAcquirePipeline, uint64 InAllocatorCycle)
	{
		Name = InName;
		AcquirePasses = TInterval<uint32>(0, InAcquirePassIndex);
		DiscardPass = kInvalidPassIndex;
		bAcquired = true;
		AcquirePipeline = InAcquirePipeline;
		AcquireCycle = InAllocatorCycle;
		AcquireCount++;
		AliasingOverlaps.Reset();
	}

	void Discard(const FRHITransientAllocationFences& Fences)
	{
		bAcquired = false;

		if (GRHIGlobals.NeedsTransientDiscardOnGraphicsWorkaround && !Fences.IsSinglePipeline())
		{
			bDiscardOnGraphicsWorkaround = true;
		}
	}

	void AddAliasingOverlap(FRHITransientResource* InBeforeResource, uint32 InAcquirePassIndex)
	{
		check(!InBeforeResource->IsAcquired());

		// Aliasing overlaps are currently only tracked with RHI validation, as no RHI is actually using them.
		if (GRHIValidationEnabled)
		{
			AliasingOverlaps.Emplace(InBeforeResource->GetRHI(), InBeforeResource->IsTexture() ? FRHITransientAliasingOverlap::EType::Texture : FRHITransientAliasingOverlap::EType::Buffer);
		}

		AcquirePasses.Min = FMath::Max(AcquirePasses.Min, InAcquirePassIndex);

		if (AcquirePipeline == ERHIPipeline::AsyncCompute && InBeforeResource->bDiscardOnGraphicsWorkaround)
		{
			InBeforeResource->DiscardPass = FMath::Min(InBeforeResource->DiscardPass, AcquirePasses.Min);
		}
		else
		{
			InBeforeResource->DiscardPass = FMath::Min(InBeforeResource->DiscardPass, AcquirePasses.Max);
		}

		check(AcquirePasses.Min <= AcquirePasses.Max);
	}

	void Finish(FRHICommandListBase& RHICmdList)
	{
		if (ResourceTask.IsValid())
		{
			FResourceTaskResult Result = MoveTemp(ResourceTask.GetResult());
			Resource = MoveTemp(Result.Resource);
			GpuVirtualAddress = Result.GpuVirtualAddress;
			ResourceTask = {};
		}
		BindDebugLabelName(RHICmdList);
	}

	FRHITransientHeapAllocation& GetHeapAllocation()
	{
		check(AllocationType == ERHITransientAllocationType::Heap);
		return HeapAllocation;
	}

	const FRHITransientHeapAllocation& GetHeapAllocation() const
	{
		check(AllocationType == ERHITransientAllocationType::Heap);
		return HeapAllocation;
	}

	FRHITransientPageAllocation& GetPageAllocation()
	{
		check(IsPageAllocated());
		return PageAllocation;
	}

	const FRHITransientPageAllocation& GetPageAllocation() const
	{
		check(IsPageAllocated());
		return PageAllocation;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////

	// Returns the underlying RHI resource.
	FRHIResource* GetRHI() const { check(!ResourceTask.IsValid()); return Resource; }

	// Returns the gpu virtual address of the transient resource.
	uint64 GetGpuVirtualAddress() const { return GpuVirtualAddress; }

	// Returns whether a resource has a pending task.
	bool HasResourceTask() const { return ResourceTask.IsValid(); }

	// Returns the name assigned to the transient resource at allocation time.
	const TCHAR* GetName() const { return Name; }

	// Returns the hash used to uniquely identify this resource if cached.
	uint64 GetHash() const { return Hash; }

	// Returns the required size in bytes of the resource.
	uint64 GetSize() const { return Size; }

	// Returns the last allocator cycle this resource was acquired.
	uint64 GetAcquireCycle() const { return AcquireCycle; }

	// Returns the number of times Acquire has been called.
	uint32 GetAcquireCount() const { return AcquireCount; }

	// Returns the list of aliasing overlaps used when transitioning the resource.
	TConstArrayView<FRHITransientAliasingOverlap> GetAliasingOverlaps() const { return AliasingOverlaps; }

	// Returns the pass index which may end acquiring this resource.
	uint32 GetAcquirePass() const { return AcquirePasses.Min; }

	// Returns the pass index which discarded this resource.
	uint32 GetDiscardPass() const { return DiscardPass; }

	// Returns whether this resource is still in an acquired state.
	bool IsAcquired() const { return bAcquired; }
	bool IsDiscarded() const { return !bAcquired; }

	ERHITransientResourceType GetResourceType() const { return ResourceType; }

	bool IsTexture() const { return ResourceType == ERHITransientResourceType::Texture; }
	bool IsBuffer()  const { return ResourceType == ERHITransientResourceType::Buffer; }

	ERHITransientAllocationType GetAllocationType() const { return AllocationType; }

	bool IsHeapAllocated() const { return AllocationType == ERHITransientAllocationType::Heap; }
	bool IsPageAllocated() const { return AllocationType == ERHITransientAllocationType::Page; }

private:
	virtual void BindDebugLabelName(FRHICommandListBase& RHICmdList) = 0;

	// Underlying RHI resource.
	TRefCountPtr<FRHIResource> Resource;
	FResourceTask ResourceTask;

	// The Gpu virtual address of the RHI resource.
	uint64 GpuVirtualAddress = 0;

	// The hash used to uniquely identify this resource if cached.
	uint64 Hash;

	// Size of the resource in bytes.
	uint64 Size;

	// Alignment of the resource in bytes.
	uint32 Alignment;

	// Tracks the number of times Acquire has been called.
	uint32 AcquireCount = 0;

	// Cycle count used to deduce age of the resource.
	uint64 AcquireCycle = 0;

	// Debug name of the resource. Updated with each allocation.
	const TCHAR* Name = nullptr;

	FRHITransientHeapAllocation HeapAllocation;
	FRHITransientPageAllocation PageAllocation;

	// List of aliasing resources overlapping with this one.
	TArray<FRHITransientAliasingOverlap> AliasingOverlaps;

	// Start -> End split pass index intervals for acquire / discard operations.
	TInterval<uint32> AcquirePasses = TInterval<uint32>(0, 0);
	uint32 DiscardPass = 0;
	ERHIPipeline AcquirePipeline = ERHIPipeline::None;
	bool bAcquired = false;
	bool bDiscardOnGraphicsWorkaround = false;

	ERHITransientAllocationType AllocationType;
	ERHITransientResourceType ResourceType;
};

class FRHITransientTexture final : public FRHITransientResource
{
public:
	RHI_API FRHITransientTexture(
		const FResourceTask& InResourceTask,
		uint64 InHash,
		uint64 InSize,
		ERHITransientAllocationType InAllocationType,
		const FRHITextureCreateInfo& InCreateInfo);

	RHI_API FRHITransientTexture(
		FRHIResource* InTexture,
		uint64 InGpuVirtualAddress,
		uint64 InHash,
		uint64 InSize,
		ERHITransientAllocationType InAllocationType,
		const FRHITextureCreateInfo& InCreateInfo);

	RHI_API virtual ~FRHITransientTexture();

	// Returns the underlying RHI texture.
	FRHITexture* GetRHI() const { return static_cast<FRHITexture*>(FRHITransientResource::GetRHI()); }

	// Returns the create info struct used when creating this texture.
	const FRHITextureCreateInfo& GetCreateInfo() const { return CreateInfo; }

	// Finds a UAV matching the descriptor in the cache or creates a new one and updates the cache.
	FRHIUnorderedAccessView* GetOrCreateUAV(FRHICommandListBase& RHICmdList, const FRHITextureUAVCreateInfo& InCreateInfo) { return ViewCache.GetOrCreateUAV(RHICmdList, GetRHI(), InCreateInfo); }

	// Finds a SRV matching the descriptor in the cache or creates a new one and updates the cache.
	FRHIShaderResourceView* GetOrCreateSRV(FRHICommandListBase& RHICmdList, const FRHITextureSRVCreateInfo& InCreateInfo) { return ViewCache.GetOrCreateSRV(RHICmdList, GetRHI(), InCreateInfo); }

	// The create info describing the texture.
	const FRHITextureCreateInfo CreateInfo;

	// The persistent view cache containing all views created for this texture.
	FRHITextureViewCache ViewCache;

private:
	RHI_API void BindDebugLabelName(FRHICommandListBase& RHICmdList) override;
};

class FRHITransientBuffer final : public FRHITransientResource
{
public:
	FRHITransientBuffer(
		const FResourceTask& InResourceTask,
		uint64 InHash,
		uint64 InSize,
		ERHITransientAllocationType InAllocationType,
		const FRHIBufferCreateInfo& InCreateInfo)
		: FRHITransientResource(InResourceTask, InHash, InSize, InAllocationType, ERHITransientResourceType::Buffer)
		, CreateInfo(InCreateInfo)
	{}

	FRHITransientBuffer(
		FRHIResource* InBuffer,
		uint64 InGpuVirtualAddress,
		uint64 InHash,
		uint64 InSize,
		ERHITransientAllocationType InAllocationType,
		const FRHIBufferCreateInfo& InCreateInfo)
		: FRHITransientResource(InBuffer, InGpuVirtualAddress, InHash, InSize, InAllocationType, ERHITransientResourceType::Buffer)
		, CreateInfo(InCreateInfo)
	{}

	RHI_API virtual ~FRHITransientBuffer();

	// Returns the underlying RHI buffer.
	FRHIBuffer* GetRHI() const { return static_cast<FRHIBuffer*>(FRHITransientResource::GetRHI()); }

	// Returns the create info used when creating this buffer.
	const FRHIBufferCreateInfo& GetCreateInfo() const { return CreateInfo; }

	// Finds a UAV matching the descriptor in the cache or creates a new one and updates the cache.
	FRHIUnorderedAccessView* GetOrCreateUAV(FRHICommandListBase& RHICmdList, const FRHIBufferUAVCreateInfo& InCreateInfo) { return ViewCache.GetOrCreateUAV(RHICmdList, GetRHI(), InCreateInfo); }

	// Finds a SRV matching the descriptor in the cache or creates a new one and updates the cache.
	FRHIShaderResourceView* GetOrCreateSRV(FRHICommandListBase& RHICmdList, const FRHIBufferSRVCreateInfo& InCreateInfo) { return ViewCache.GetOrCreateSRV(RHICmdList, GetRHI(), InCreateInfo); }

	// The create info describing the texture.
	const FRHIBufferCreateInfo CreateInfo;

	// The persistent view cache containing all views created for this buffer.
	FRHIBufferViewCache ViewCache;

private:
	RHI_API void BindDebugLabelName(FRHICommandListBase& RHICmdList) override;
};

class FRHITransientAllocationStats
{
public:
	struct FAllocation
	{
		uint64 OffsetMin = 0;
		uint64 OffsetMax = 0;
		uint32 MemoryRangeIndex = 0;
	};

	using FAllocationArray = TArray<FAllocation, TInlineAllocator<2>>;

	enum class EMemoryRangeFlags
	{
		None = 0,

		// The memory range references platform specific fast RAM.
		FastVRAM = 1 << 0
	};

	struct FMemoryRange
	{
		// Number of bytes available for use in the memory range.
		uint64 Capacity = 0;

		// Number of bytes allocated for use in the memory range.
		uint64 CommitSize = 0;

		// Flags specified for this memory range.
		EMemoryRangeFlags Flags = EMemoryRangeFlags::None;
	};

	TArray<FMemoryRange> MemoryRanges;
	TMap<const FRHITransientResource*, FAllocationArray> Resources;
};

ENUM_CLASS_FLAGS(FRHITransientAllocationStats::EMemoryRangeFlags);

enum class ERHITransientResourceCreateMode
{
	// Transient resources are always created inline inside of the Create call.
	Inline,

	// Transient resource creation may be offloaded to a task (dependent on platform), in which case FRHITransientResource::Finish must be called prior to accessing the underlying RHI resource.
	Task
};

class IRHITransientResourceAllocator
{
public:
	virtual ~IRHITransientResourceAllocator() = default;

	// Supports transient allocations of given resource type
	virtual bool SupportsResourceType(ERHITransientResourceType Type) const = 0;

	// Sets the create mode for allocations.
	virtual void SetCreateMode(ERHITransientResourceCreateMode CreateMode) {};

	// Allocates a new transient resource with memory backed by the transient allocator.
	virtual FRHITransientTexture* CreateTexture(const FRHITextureCreateInfo& CreateInfo, const TCHAR* DebugName, const FRHITransientAllocationFences& Fences) = 0;
	virtual FRHITransientBuffer* CreateBuffer(const FRHIBufferCreateInfo& CreateInfo, const TCHAR* DebugName, const FRHITransientAllocationFences& Fences) = 0;

	// Deallocates the underlying memory for use by a future resource creation call.
	virtual void DeallocateMemory(FRHITransientTexture* Texture, const FRHITransientAllocationFences& Fences) = 0;
	virtual void DeallocateMemory(FRHITransientBuffer* Buffer, const FRHITransientAllocationFences& Fences) = 0;

	// Flushes any pending allocations prior to rendering. Optionally emits stats if OutStats is valid.
	virtual void Flush(FRHICommandListImmediate& RHICmdList, FRHITransientAllocationStats* OutStats = nullptr) = 0;

	// Releases this instance of the transient allocator. Invalidates any outstanding transient resources.
	virtual void Release(FRHICommandListImmediate& RHICmdList) { delete this; }
};
