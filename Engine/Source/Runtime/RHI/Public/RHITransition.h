// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Containers/ArrayView.h"
#include "Misc/Optional.h"
#include "RHIDefinitions.h"
#include "RHIAllocators.h"
#include "RHIAccess.h"
#include "RHIPipeline.h"
#include "RHIValidationCommon.h"


// The size in bytes of the storage required by the platform RHI for each resource transition.
extern RHI_API uint64 GRHITransitionPrivateData_SizeInBytes;
extern RHI_API uint64 GRHITransitionPrivateData_AlignInBytes;

struct FRHISubresourceRange
{
	static const uint16 kDepthPlaneSlice = 0;
	static const uint16 kStencilPlaneSlice = 1;
	static const uint16 kAllSubresources = TNumericLimits<uint16>::Max();

	uint16 MipIndex = kAllSubresources;
	uint16 ArraySlice = kAllSubresources;
	uint16 PlaneSlice = kAllSubresources;

	FRHISubresourceRange() = default;

	FRHISubresourceRange(
		uint32 InMipIndex,
		uint32 InArraySlice,
		uint32 InPlaneSlice)
	{
		checkSlow(InMipIndex <= TNumericLimits<uint16>::Max());
		checkSlow(InArraySlice <= TNumericLimits<uint16>::Max());
		checkSlow(InPlaneSlice <= TNumericLimits<uint16>::Max());

		MipIndex = (uint16)InMipIndex;
		ArraySlice = (uint16)InArraySlice;
		PlaneSlice = (uint16)InPlaneSlice;
	}

	FRHISubresourceRange(
		uint16 InMipIndex,
		uint16 InArraySlice,
		uint16 InPlaneSlice)
		: MipIndex(InMipIndex)
		, ArraySlice(InArraySlice)
		, PlaneSlice(InPlaneSlice)
	{}

	inline bool IsAllMips() const
	{
		return MipIndex == kAllSubresources;
	}

	inline bool IsAllArraySlices() const
	{
		return ArraySlice == kAllSubresources;
	}

	inline bool IsAllPlaneSlices() const
	{
		return PlaneSlice == kAllSubresources;
	}

	inline bool IsWholeResource() const
	{
		return IsAllMips() && IsAllArraySlices() && IsAllPlaneSlices();
	}

	inline bool IsZeroIndexResource() const
	{
		return (MipIndex == 0) && (ArraySlice == 0) && (PlaneSlice == 0);
	}

	inline bool IgnoreDepthPlane() const
	{
		return PlaneSlice == kStencilPlaneSlice;
	}

	inline bool IgnoreStencilPlane() const
	{
		return PlaneSlice == kDepthPlaneSlice;
	}

	inline bool operator == (FRHISubresourceRange const& RHS) const
	{
		return MipIndex == RHS.MipIndex
			&& ArraySlice == RHS.ArraySlice
			&& PlaneSlice == RHS.PlaneSlice;
	}

	inline bool operator != (FRHISubresourceRange const& RHS) const
	{
		return !(*this == RHS);
	}
};

/**
* Represents a change in physical memory allocation for a resource that was created with TexCreate/BUF_ReservedResource flag.
* Physical memory is allocated in tiles/pages and mapped to the tail of the currently committed region of the resource.
* This API may be used to grow or shrink reserved resources without moving the bulk of the data or re-creating SRVs/UAVs.
* The contents of the newly committed region of the resource is undefined and must be overwritten by the application before use.
* Reserved resources must be created with maximum expected size, which will not cost any memory until committed.
* Commit size must be smaller or equal to the maximum resource size specified at creation.
* Check GRHIGlobals.ReservedResources.Supported before using this API or TexCreate/BUF_ReservedResource flag.
*/
struct FRHICommitResourceInfo
{
	uint64 SizeInBytes = 0;
	explicit FRHICommitResourceInfo(uint64 InSizeInBytes) : SizeInBytes(InSizeInBytes) {}
};

struct FRHITransitionInfo : public FRHISubresourceRange
{
	union
	{
		class FRHIResource* Resource = nullptr;
		class FRHIViewableResource* ViewableResource;
		class FRHITexture* Texture;
		class FRHIBuffer* Buffer;
		class FRHIUnorderedAccessView* UAV;
		class FRHIRayTracingAccelerationStructure* BVH;
	};

	enum class EType : uint8
	{
		Unknown,
		Texture,
		Buffer,
		UAV,
		BVH,
	} Type = EType::Unknown;

	ERHIAccess AccessBefore = ERHIAccess::Unknown;
	ERHIAccess AccessAfter = ERHIAccess::Unknown;
	EResourceTransitionFlags Flags = EResourceTransitionFlags::None;
	TOptional<FRHICommitResourceInfo> CommitInfo;

	FRHITransitionInfo() = default;

	FRHITransitionInfo(
		class FRHITexture* InTexture,
		ERHIAccess InPreviousState,
		ERHIAccess InNewState,
		EResourceTransitionFlags InFlags = EResourceTransitionFlags::None,
		uint32 InMipIndex = kAllSubresources,
		uint32 InArraySlice = kAllSubresources,
		uint32 InPlaneSlice = kAllSubresources)
		: FRHISubresourceRange(InMipIndex, InArraySlice, InPlaneSlice)
		, Texture(InTexture)
		, Type(EType::Texture)
		, AccessBefore(InPreviousState)
		, AccessAfter(InNewState)
		, Flags(InFlags)
	{}

	FRHITransitionInfo(class FRHIUnorderedAccessView* InUAV, ERHIAccess InPreviousState, ERHIAccess InNewState, EResourceTransitionFlags InFlags = EResourceTransitionFlags::None)
		: UAV(InUAV)
		, Type(EType::UAV)
		, AccessBefore(InPreviousState)
		, AccessAfter(InNewState)
		, Flags(InFlags)
	{}

	FRHITransitionInfo(class FRHIBuffer* InRHIBuffer, ERHIAccess InPreviousState, ERHIAccess InNewState, EResourceTransitionFlags InFlags = EResourceTransitionFlags::None)
		: Buffer(InRHIBuffer)
		, Type(EType::Buffer)
		, AccessBefore(InPreviousState)
		, AccessAfter(InNewState)
		, Flags(InFlags)
	{}

	FRHITransitionInfo(class FRHIBuffer* InRHIBuffer, ERHIAccess InPreviousState, ERHIAccess InNewState, FRHICommitResourceInfo InCommitInfo)
		: Buffer(InRHIBuffer)
		, Type(EType::Buffer)
		, AccessBefore(InPreviousState)
		, AccessAfter(InNewState)
		, CommitInfo(InCommitInfo)
	{}

	FRHITransitionInfo(class FRHIRayTracingAccelerationStructure* InBVH, ERHIAccess InPreviousState, ERHIAccess InNewState, EResourceTransitionFlags InFlags = EResourceTransitionFlags::None)
		: BVH(InBVH)
		, Type(EType::BVH)
		, AccessBefore(InPreviousState)
		, AccessAfter(InNewState)
		, Flags(InFlags)
	{}

	FRHITransitionInfo(class FRHITexture* InTexture, ERHIAccess InNewState)
		: Texture(InTexture)
		, Type(EType::Texture)
		, AccessAfter(InNewState)
	{}

	FRHITransitionInfo(class FRHIUnorderedAccessView* InUAV, ERHIAccess InNewState)
		: UAV(InUAV)
		, Type(EType::UAV)
		, AccessAfter(InNewState)
	{}

	FRHITransitionInfo(class FRHIBuffer* InRHIBuffer, ERHIAccess InNewState)
		: Buffer(InRHIBuffer)
		, Type(EType::Buffer)
		, AccessAfter(InNewState)
	{}

	inline bool operator == (FRHITransitionInfo const& RHS) const
	{
		return Resource == RHS.Resource
			&& Type == RHS.Type
			&& AccessBefore == RHS.AccessBefore
			&& AccessAfter == RHS.AccessAfter
			&& Flags == RHS.Flags
			&& FRHISubresourceRange::operator==(RHS);
	}

	inline bool operator != (FRHITransitionInfo const& RHS) const
	{
		return !(*this == RHS);
	}
};

struct FRHITransientAliasingOverlap
{
	union
	{
		class FRHIResource* Resource = nullptr;
		class FRHITexture* Texture;
		class FRHIBuffer* Buffer;
	};

	enum class EType : uint8
	{
		Texture,
		Buffer
	} Type = EType::Texture;

	FRHITransientAliasingOverlap() = default;

	FRHITransientAliasingOverlap(FRHIResource* InResource, EType InType)
		: Resource(InResource)
		, Type(InType)
	{}

	FRHITransientAliasingOverlap(FRHITexture* InTexture)
		: Texture(InTexture)
		, Type(EType::Texture)
	{}

	FRHITransientAliasingOverlap(FRHIBuffer* InBuffer)
		: Buffer(InBuffer)
		, Type(EType::Buffer)
	{}

	bool IsTexture() const
	{
		return Type == EType::Texture;
	}

	bool IsBuffer() const
	{
		return Type == EType::Buffer;
	}

	bool operator == (const FRHITransientAliasingOverlap& Other) const
	{
		return Resource == Other.Resource;
	}

	inline bool operator != (const FRHITransientAliasingOverlap& RHS) const
	{
		return !(*this == RHS);
	}
};

struct FRHITransientAliasingInfo
{
	union
	{
		class FRHIResource* Resource = nullptr;
		class FRHITexture* Texture;
		class FRHIBuffer* Buffer;
	};

	// List of prior resource overlaps to use when acquiring. Must be empty for discard operations.
	TArrayView<const FRHITransientAliasingOverlap> Overlaps;

	enum class EType : uint8
	{
		Texture,
		Buffer
	} Type = EType::Texture;

	enum class EAction : uint8
	{
		Acquire,
		Discard
	} Action = EAction::Acquire;

	FRHITransientAliasingInfo() = default;

	static FRHITransientAliasingInfo Acquire(class FRHITexture* Texture, TArrayView<const FRHITransientAliasingOverlap> InOverlaps)
	{
		FRHITransientAliasingInfo Info;
		Info.Texture = Texture;
		Info.Overlaps = InOverlaps;
		Info.Type = EType::Texture;
		Info.Action = EAction::Acquire;
		return Info;
	}

	static FRHITransientAliasingInfo Acquire(class FRHIBuffer* Buffer, TArrayView<const FRHITransientAliasingOverlap> InOverlaps)
	{
		FRHITransientAliasingInfo Info;
		Info.Buffer = Buffer;
		Info.Overlaps = InOverlaps;
		Info.Type = EType::Buffer;
		Info.Action = EAction::Acquire;
		return Info;
	}

	bool IsAcquire() const
	{
		return Action == EAction::Acquire;
	}

	bool IsTexture() const
	{
		return Type == EType::Texture;
	}

	bool IsBuffer() const
	{
		return Type == EType::Buffer;
	}

	inline bool operator == (const FRHITransientAliasingInfo& RHS) const
	{
		return Resource == RHS.Resource
			&& Type == RHS.Type
			&& Action == RHS.Action;
	}

	inline bool operator != (const FRHITransientAliasingInfo& RHS) const
	{
		return !(*this == RHS);
	}
};

struct FRHITransitionCreateInfo
{
	FRHITransitionCreateInfo() = default;

	FRHITransitionCreateInfo(
		ERHIPipeline InSrcPipelines,
		ERHIPipeline InDstPipelines,
		ERHITransitionCreateFlags InFlags = ERHITransitionCreateFlags::None,
		TArrayView<const FRHITransitionInfo> InTransitionInfos = {},
		TArrayView<const FRHITransientAliasingInfo> InAliasingInfos = {})
		: SrcPipelines(InSrcPipelines)
		, DstPipelines(InDstPipelines)
		, Flags(InFlags)
		, TransitionInfos(InTransitionInfos)
		, AliasingInfos(InAliasingInfos)
	{}

	ERHIPipeline SrcPipelines = ERHIPipeline::None;
	ERHIPipeline DstPipelines = ERHIPipeline::None;
	ERHITransitionCreateFlags Flags = ERHITransitionCreateFlags::None;
	TArrayView<const FRHITransitionInfo> TransitionInfos;
	TArrayView<const FRHITransientAliasingInfo> AliasingInfos;

	inline bool operator == (FRHITransitionCreateInfo const& RHS) const
	{
		if (SrcPipelines != RHS.SrcPipelines)
			return false;

		if (DstPipelines != RHS.DstPipelines)
			return false;

		if (Flags != RHS.Flags)
			return false;

		for (int32 i = 0; i < RHS.TransitionInfos.Num(); i++)
		{
			if (TransitionInfos[i] != RHS.TransitionInfos[i])
				return false;
		}

		for (int32 i = 0; i < RHS.AliasingInfos.Num(); i++)
		{
			if (AliasingInfos[i] != RHS.AliasingInfos[i])
				return false;
		}

		return true;
	}

};

struct FRHITrackedAccess
{
	FRHITrackedAccess() = default;

	FRHITrackedAccess(ERHIAccess InAccess)
		: Access(InAccess)
	{}

	FRHITrackedAccess(ERHIAccess InAccess, ERHIPipeline InPipelines)
		: Access(InAccess)
		, Pipelines(InPipelines)
	{}

	ERHIAccess Access = ERHIAccess::Unknown;
	ERHIPipeline Pipelines = ERHIPipeline::None;
};

struct FRHITrackedAccessInfo : FRHITrackedAccess
{
	FRHITrackedAccessInfo() = default;

	FRHITrackedAccessInfo(FRHIViewableResource* InResource, ERHIAccess InAccess, ERHIPipeline InPipelines)
		: FRHITrackedAccess(InAccess, InPipelines)
		, Resource(InResource)
	{}

	FRHIViewableResource* Resource = nullptr;
};

struct FRHITransition;
RHI_API FRHITransition* RHICreateTransition(const FRHITransitionCreateInfo& CreateInfo);

enum class ETransitionFlag : uint8
{
	None = 0,
	AllowDuringRenderPass = 1 << 0,			// Specific transition are happening during RenderPass and are reported by the validation layer, we need to allow some of them
};

// Opaque data structure used to represent a pending resource transition in the RHI.
struct FRHITransition
#if ENABLE_RHI_VALIDATION
	: public RHIValidation::FTransitionResource
#endif
{
public:
	template <typename T>
	inline T* GetPrivateData()
	{
		checkSlow(sizeof(T) == GRHITransitionPrivateData_SizeInBytes && GRHITransitionPrivateData_AlignInBytes != 0);
		uintptr_t Addr = Align(uintptr_t(this + 1), GRHITransitionPrivateData_AlignInBytes);
		checkSlow(Addr + GRHITransitionPrivateData_SizeInBytes - (uintptr_t)this == GetTotalAllocationSize());
		return reinterpret_cast<T*>(Addr);
	}

	template <typename T>
	inline const T* GetPrivateData() const
	{
		return const_cast<FRHITransition*>(this)->GetPrivateData<T>();
	}

	inline bool AllowInRenderingPass() const { return EnumHasAnyFlags(Flags, ERHITransitionCreateFlags::AllowDuringRenderPass); }

private:
	// Prevent copying and moving. Only pointers to these structures are allowed.
	FRHITransition(const FRHITransition&) = delete;
	FRHITransition(FRHITransition&&) = delete;

	// Private constructor. Memory for transitions is allocated manually with extra space at the tail of the structure for RHI use.
	FRHITransition(ERHIPipeline SrcPipelines, ERHIPipeline DstPipelines, ERHITransitionCreateFlags InFlags)
		: State(int8(int32(SrcPipelines) | (int32(DstPipelines) << int32(ERHIPipeline::Num))))
		, Flags(InFlags)
#if DO_CHECK || USING_CODE_ANALYSIS
		, AllowedSrc(SrcPipelines)
		, AllowedDst(DstPipelines)
#endif
	{}

	// Give private access to specific functions/RHI commands that need to allocate or control transitions.
	friend RHI_API FRHITransition* RHICreateTransition(const FRHITransitionCreateInfo& CreateInfo);
	friend class FRHICommandListBase;
	friend class FRHIComputeCommandList;
	friend struct FRHICommandBeginTransitions;
	friend struct FRHICommandEndTransitions;
	friend struct FRHICommandResourceTransition;

	static uint64 GetTotalAllocationSize()
	{
		// Allocate extra space at the end of this structure for private RHI use. This is determined by GRHITransitionPrivateData_SizeInBytes.
		return Align(sizeof(FRHITransition), FMath::Max(GRHITransitionPrivateData_AlignInBytes, 1ull)) + GRHITransitionPrivateData_SizeInBytes;
	}

	static uint64 GetAlignment()
	{
		return FMath::Max((uint64)alignof(FRHITransition), GRHITransitionPrivateData_AlignInBytes);
	}

	inline void MarkBegin(ERHIPipeline Pipeline) const
	{
#if DO_CHECK || USING_CODE_ANALYSIS
		checkf(EnumHasAllFlags(AllowedSrc, Pipeline), TEXT("Transition is being used on a source pipeline that it wasn't created for."));
#endif

		int8 Mask = int8(Pipeline);
		int8 PreviousValue = FPlatformAtomics::InterlockedAnd(&State, ~Mask);
		checkf((PreviousValue & Mask) == Mask, TEXT("RHIBeginTransitions has been called twice on this transition for at least one pipeline."));

		if (PreviousValue == Mask)
		{
			Cleanup();
		}
	}

	inline void MarkEnd(ERHIPipeline Pipeline) const
	{
#if DO_CHECK || USING_CODE_ANALYSIS
		checkf(EnumHasAllFlags(AllowedDst, Pipeline), TEXT("Transition is being used on a destination pipeline that it wasn't created for."));
#endif

		int8 Mask = int8(int32(Pipeline) << int32(ERHIPipeline::Num));
		int8 PreviousValue = FPlatformAtomics::InterlockedAnd(&State, ~Mask);
		checkf((PreviousValue & Mask) == Mask, TEXT("RHIEndTransitions has been called twice on this transition for at least one pipeline."));

		if (PreviousValue == Mask)
		{
			Cleanup();
		}
	}
	
	RHI_API void Cleanup() const;

	mutable int8 State;
	static_assert((int32(ERHIPipeline::Num) * 2) < (sizeof(State) * 8), "Not enough bits to hold pipeline state.");
	mutable ERHITransitionCreateFlags Flags;

#if DO_CHECK || USING_CODE_ANALYSIS
	mutable ERHIPipeline AllowedSrc;
	mutable ERHIPipeline AllowedDst;
#endif
};

RHI_API FRHIViewableResource* GetViewableResource(const FRHITransitionInfo& Info);