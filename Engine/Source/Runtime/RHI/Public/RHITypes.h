// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHITypes.h: Render Hardware Interface types
		(that require linking or heavier includes).
=============================================================================*/

#pragma once

#include "RHIDefinitions.h"
#include "Math/PerspectiveMatrix.h"
#include "Math/IntVector.h"

/** to customize the RHIReadSurfaceData() output */
class FReadSurfaceDataFlags
{
public:
	// @param InCompressionMode defines the value input range that is mapped to output range
	//			The default CompressionMode of RCM_UNorm will modify values to fit in [0,1]
	//			it is recommended to always use RCM_MinMax instead, which leaves values unchanged
	//			if you do want scaling into [0,1] do it after the fact using ScaleChannelsSoMinMaxIsInZeroToOne
	// @param InCubeFace defined which cubemap side is used, only required for cubemap content, then it needs to be a valid side
	FReadSurfaceDataFlags(ERangeCompressionMode InCompressionMode = RCM_UNorm, ECubeFace InCubeFace = CubeFace_MAX)
		:CubeFace(InCubeFace), CompressionMode(InCompressionMode)
	{
	}

	ECubeFace GetCubeFace() const
	{
		return CubeFace;
	}
	void SetCubeFace(ECubeFace InCubeFace)
	{
		CubeFace = InCubeFace;
	}

	ERangeCompressionMode GetCompressionMode() const
	{
		return CompressionMode;
	}

	void SetLinearToGamma(bool Value)
	{
		bLinearToGamma = Value;
	}

	bool GetLinearToGamma() const
	{
		return bLinearToGamma;
	}

	void SetOutputStencil(bool Value)
	{
		bOutputStencil = Value;
	}

	bool GetOutputStencil() const
	{
		return bOutputStencil;
	}

	void SetMip(uint8 InMipLevel)
	{
		MipLevel = InMipLevel;
	}

	uint8 GetMip() const
	{
		return MipLevel;
	}

	void SetMaxDepthRange(float Value)
	{
		MaxDepthRange = Value;
	}

	float ComputeNormalizedDepth(float DeviceZ) const
	{
		return FMath::Abs(ConvertFromDeviceZ(DeviceZ) / MaxDepthRange);
	}

	void SetGPUIndex(uint32 InGPUIndex)
	{
		GPUIndex = InGPUIndex;
	}

	uint32 GetGPUIndex() const
	{
		return GPUIndex;
	}

	void SetArrayIndex(int32 InArrayIndex)
	{
		ArrayIndex = InArrayIndex;
	}

	int32 GetArrayIndex() const
	{
		return ArrayIndex;
	}

private:

	// @return SceneDepth
	float ConvertFromDeviceZ(float DeviceZ) const
	{
		DeviceZ = FMath::Min(DeviceZ, 1 - Z_PRECISION);

		// for depth to linear conversion
		const FVector2f InvDeviceZToWorldZ(0.1f, 0.1f);

		return 1.0f / (DeviceZ * InvDeviceZToWorldZ.X - InvDeviceZToWorldZ.Y);
	}

	ECubeFace CubeFace = CubeFace_MAX;
	ERangeCompressionMode CompressionMode = RCM_UNorm;
	bool bLinearToGamma = true;
	float MaxDepthRange = 16000.0f;
	bool bOutputStencil = false;
	uint8 MipLevel = 0;
	int32 ArrayIndex = 0;
	uint32 GPUIndex = 0;
};

/** specifies an update region for a texture */
struct FUpdateTextureRegion2D
{
	/** offset in texture */
	uint32 DestX;
	uint32 DestY;

	/** offset in source image data */
	int32 SrcX;
	int32 SrcY;

	/** size of region to copy */
	uint32 Width;
	uint32 Height;

	FUpdateTextureRegion2D()
	{}

	FUpdateTextureRegion2D(uint32 InDestX, uint32 InDestY, int32 InSrcX, int32 InSrcY, uint32 InWidth, uint32 InHeight)
		: DestX(InDestX)
		, DestY(InDestY)
		, SrcX(InSrcX)
		, SrcY(InSrcY)
		, Width(InWidth)
		, Height(InHeight)
	{}
};

/** specifies an update region for a texture */
struct FUpdateTextureRegion3D
{
	/** offset in texture */
	uint32 DestX;
	uint32 DestY;
	uint32 DestZ;

	/** offset in source image data */
	int32 SrcX;
	int32 SrcY;
	int32 SrcZ;

	/** size of region to copy */
	uint32 Width;
	uint32 Height;
	uint32 Depth;

	FUpdateTextureRegion3D()
	{}

	FUpdateTextureRegion3D(uint32 InDestX, uint32 InDestY, uint32 InDestZ, int32 InSrcX, int32 InSrcY, int32 InSrcZ, uint32 InWidth, uint32 InHeight, uint32 InDepth)
		: DestX(InDestX)
		, DestY(InDestY)
		, DestZ(InDestZ)
		, SrcX(InSrcX)
		, SrcY(InSrcY)
		, SrcZ(InSrcZ)
		, Width(InWidth)
		, Height(InHeight)
		, Depth(InDepth)
	{}

	FUpdateTextureRegion3D(FIntVector InDest, FIntVector InSource, FIntVector InSourceSize)
		: DestX(InDest.X)
		, DestY(InDest.Y)
		, DestZ(InDest.Z)
		, SrcX(InSource.X)
		, SrcY(InSource.Y)
		, SrcZ(InSource.Z)
		, Width(InSourceSize.X)
		, Height(InSourceSize.Y)
		, Depth(InSourceSize.Z)
	{}
};
