// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"
#include "RHIMemoryLayout.h"
#include "RHIResources.h"
#include "Serialization/MemoryImage.h"
#include "Serialization/MemoryLayout.h"

/** Data structure to store information about resource parameter in a shader parameter structure. */
struct FRHIUniformBufferResourceInitializer
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FRHIUniformBufferResourceInitializer, RHI_API, NonVirtual);

	/** Byte offset to each resource in the uniform buffer memory. */
	LAYOUT_FIELD(uint16, MemberOffset);

	/** Type of the member that allow (). */
	LAYOUT_FIELD(EUniformBufferBaseType, MemberType);

	/** Compare two uniform buffer layout resources. */
	friend inline bool operator==(const FRHIUniformBufferResourceInitializer& A, const FRHIUniformBufferResourceInitializer& B)
	{
		return A.MemberOffset == B.MemberOffset
			&& A.MemberType == B.MemberType;
	}
};

inline FArchive& operator<<(FArchive& Ar, FRHIUniformBufferResourceInitializer& Ref)
{
	uint8 Type = (uint8)Ref.MemberType;
	Ar << Ref.MemberOffset;
	Ar << Type;
	Ref.MemberType = (EUniformBufferBaseType)Type;
	return Ar;
}

/** Initializer for the layout of a uniform buffer in memory. */
struct FRHIUniformBufferLayoutInitializer
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FRHIUniformBufferLayoutInitializer, RHI_API, NonVirtual);

	FRHIUniformBufferLayoutInitializer() = default;

	explicit FRHIUniformBufferLayoutInitializer(const TCHAR* InName)
		: Name(InName)
	{}
	explicit FRHIUniformBufferLayoutInitializer(const TCHAR* InName, uint32 InConstantBufferSize)
		: Name(InName)
		, ConstantBufferSize(InConstantBufferSize)
	{
		ComputeHash();
	}

	inline uint32 GetHash() const
	{
		checkSlow(Hash != 0);
		return Hash;
	}

	/** 
	 * This hash is stored with compiled shaders and is used for two things. 
	 * 1. Validation that the data layout did not change between compilation and runtime.
	 * 2. Passing to FindUniformBufferStructByLayoutHash() to find the associated uniform buffer layouts (and specifically the runtime slot mappings) for the shader.
	 * Possibly it would be better to separate those concerns, but that would require more data stored per shader.
	 */
	void ComputeHash()
	{
		uint32 TmpHash = 0;

		TmpHash |= (ConstantBufferSize << 16);
		TmpHash |= (static_cast<uint32>(BindingFlags) << 8);

		// Static slot is not stable. Just track whether we have one at all.
		TmpHash |= (StaticSlot != MAX_UNIFORM_BUFFER_STATIC_SLOTS ? 1 << 0 : 0);

		// These flags aren't based on the resources, so make sure they change the hash
		TmpHash |= (EnumHasAnyFlags(Flags, ERHIUniformBufferFlags::NoEmulatedUniformBuffer) ? 1 << 1 : 0);
		TmpHash |= (EnumHasAnyFlags(Flags, ERHIUniformBufferFlags::NeedsReflectedMembers)   ? 1 << 2 : 0);
		TmpHash |= (EnumHasAnyFlags(Flags, ERHIUniformBufferFlags::UniformView)             ? 1 << 3 : 0);

		for (int32 ResourceIndex = 0; ResourceIndex < Resources.Num(); ResourceIndex++)
		{
			// Offset and therefore hash must be the same regardless of pointer size
			checkSlow(Resources[ResourceIndex].MemberOffset == Align(Resources[ResourceIndex].MemberOffset, SHADER_PARAMETER_POINTER_ALIGNMENT));
			TmpHash ^= Resources[ResourceIndex].MemberOffset;
		}

		uint32 N = Resources.Num();
		while (N >= 4)
		{
			TmpHash ^= (Resources[--N].MemberType << 0);
			TmpHash ^= (Resources[--N].MemberType << 8);
			TmpHash ^= (Resources[--N].MemberType << 16);
			TmpHash ^= (Resources[--N].MemberType << 24);
		}
		while (N >= 2)
		{
			TmpHash ^= Resources[--N].MemberType << 0;
			TmpHash ^= Resources[--N].MemberType << 16;
		}
		while (N > 0)
		{
			TmpHash ^= Resources[--N].MemberType;
		}

		// Name is necessary to distinguish different uniform buffer declarations with matching layouts.
		// Without it FindUniformBufferStructByLayoutHash() is vulnerable to hash collisions.
		TmpHash ^= FCrc::StrCrc32(*Name);
		
		Hash = TmpHash;
	}

	void CopyFrom(const FRHIUniformBufferLayoutInitializer& Source)
	{
		ConstantBufferSize = Source.ConstantBufferSize;
		StaticSlot = Source.StaticSlot;
		BindingFlags = Source.BindingFlags;
		Flags = Source.Flags;
		Resources = Source.Resources;
		Name = Source.Name;
		Hash = Source.Hash;
	}

	const FMemoryImageString& GetDebugName() const
	{
		return Name;
	}

	bool HasRenderTargets() const
	{
		return RenderTargetsOffset != kUniformBufferInvalidOffset;
	}

	bool HasStaticSlot() const
	{
		return IsUniformBufferStaticSlotValid(StaticSlot);
	}

	friend FArchive& operator<<(FArchive& Ar, FRHIUniformBufferLayoutInitializer& Ref)
	{
		Ar << Ref.ConstantBufferSize;
		Ar << Ref.StaticSlot;
		Ar << Ref.RenderTargetsOffset;
		Ar << Ref.BindingFlags;
		Ar << Ref.Flags;
		Ar << Ref.Resources;
		Ar << Ref.GraphResources;
		Ar << Ref.GraphTextures;
		Ar << Ref.GraphBuffers;
		Ar << Ref.GraphUniformBuffers;
		Ar << Ref.UniformBuffers;
		Ar << Ref.Name;
		Ar << Ref.Hash;
		return Ar;
	}

private:
	// for debugging / error message
	LAYOUT_FIELD(FMemoryImageString, Name);

public:
	/** The list of all resource inlined into the shader parameter structure. */
	LAYOUT_FIELD(TMemoryImageArray<FRHIUniformBufferResourceInitializer>, Resources);

	/** The list of all RDG resource references inlined into the shader parameter structure. */
	LAYOUT_FIELD(TMemoryImageArray<FRHIUniformBufferResourceInitializer>, GraphResources);

	/** The list of all RDG texture references inlined into the shader parameter structure. */
	LAYOUT_FIELD(TMemoryImageArray<FRHIUniformBufferResourceInitializer>, GraphTextures);

	/** The list of all RDG buffer references inlined into the shader parameter structure. */
	LAYOUT_FIELD(TMemoryImageArray<FRHIUniformBufferResourceInitializer>, GraphBuffers);

	/** The list of all RDG uniform buffer references inlined into the shader parameter structure. */
	LAYOUT_FIELD(TMemoryImageArray<FRHIUniformBufferResourceInitializer>, GraphUniformBuffers);

	/** The list of all non-RDG uniform buffer references inlined into the shader parameter structure. */
	LAYOUT_FIELD(TMemoryImageArray<FRHIUniformBufferResourceInitializer>, UniformBuffers);

private:
	LAYOUT_FIELD_INITIALIZED(uint32, Hash, 0);

public:
	/** The size of the constant buffer in bytes. */
	LAYOUT_FIELD_INITIALIZED(uint32, ConstantBufferSize, 0);

	/** The render target binding slots offset, if it exists. */
	LAYOUT_FIELD_INITIALIZED(uint16, RenderTargetsOffset, kUniformBufferInvalidOffset);

	/** The static slot (if applicable). */
	LAYOUT_FIELD_INITIALIZED(FUniformBufferStaticSlot, StaticSlot, MAX_UNIFORM_BUFFER_STATIC_SLOTS);

	/** The binding flags describing how this resource can be bound to the RHI. */
	LAYOUT_FIELD_INITIALIZED(EUniformBufferBindingFlags, BindingFlags, EUniformBufferBindingFlags::Shader);

	/** Flags for the uniform buffer */
	LAYOUT_FIELD_INITIALIZED(ERHIUniformBufferFlags, Flags, ERHIUniformBufferFlags::None);
	
	/** Compare two uniform buffer layout initializers. */
	friend inline bool operator==(const FRHIUniformBufferLayoutInitializer& A, const FRHIUniformBufferLayoutInitializer& B)
	{
		return A.ConstantBufferSize == B.ConstantBufferSize
			&& A.StaticSlot == B.StaticSlot
			&& A.BindingFlags == B.BindingFlags
			&& A.Resources == B.Resources;
	}
};

