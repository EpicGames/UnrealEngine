// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Async/Mutex.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/ResourceArray.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "HAL/IConsoleManager.h"
#include "RHIDefinitions.h"
#include "RHICommandList.h"
#include "Tasks/Task.h"
#include "Containers/Map.h"

#if !UE_BUILD_SHIPPING && PLATFORM_SUPPORTS_FLIP_TRACKING
	#define UE_INPUT_LATENCY_TRACKING 1
#else
	#define UE_INPUT_LATENCY_TRACKING 0
#endif

inline EPixelFormat GetBlockCompressedFormatUAVAliasFormat(EPixelFormat Format)
{
	switch (Format)
	{
	case PF_DXT1:
	case PF_BC4:
	case PF_ETC1:
	case PF_ETC2_RGB:
	case PF_ETC2_R11_EAC:
		return PF_R32G32_UINT;

	case PF_DXT3:
	case PF_DXT5:
	case PF_BC5:
	case PF_BC6H:
	case PF_BC7:
	case PF_ETC2_RGBA:
	case PF_ETC2_RG11_EAC:
		return PF_R32G32B32A32_UINT;
	}

	return Format;
}

inline bool IsUnormFormat(EPixelFormat Format)
{
	switch (Format)
	{
	case PF_R5G6B5_UNORM:
	case PF_R16G16B16A16_UNORM:
	case PF_B5G5R5A1_UNORM:
		return true;

	default:
		break;
	}
	return false;
}

inline bool IsSnormFormat(EPixelFormat Format)
{
	switch (Format)
	{
	case PF_R8G8B8A8_SNORM:
	case PF_R16G16B16A16_SNORM:
	case PF_G16R16_SNORM:
		return true;

	default:
		break;
	}
	return false;
}

inline bool IsUintFormat(EPixelFormat Format)
{
	switch (Format)
	{
	case PF_R32_UINT:
	case PF_R16_UINT:
	case PF_R16G16B16A16_UINT:
	case PF_R32G32B32A32_UINT:
	case PF_R16G16_UINT:
	case PF_R8_UINT:
	case PF_R8G8B8A8_UINT:
	case PF_R32G32_UINT:
		return true;

	default:
		break;
	}
	return false;
}

inline bool IsSintFormat(EPixelFormat Format)
{
	switch (Format)
	{
	case PF_R32_SINT:
	case PF_R16_SINT:
	case PF_R16G16B16A16_SINT:
		return true;

	default:
		break;
	}
	return false;
}

/** Get the best default resource state for the given texture creation flags */
extern RHI_API ERHIAccess RHIGetDefaultResourceState(ETextureCreateFlags InUsage, bool bInHasInitialData);

/** Get the best default resource state for the given buffer creation flags */
extern RHI_API ERHIAccess RHIGetDefaultResourceState(EBufferUsageFlags InUsage, bool bInHasInitialData);

/** Encapsulates a GPU read/write texture 2D with its UAV and SRV. */
struct FTextureRWBuffer
{
	FTextureRHIRef Buffer;
	FUnorderedAccessViewRHIRef UAV;
	FShaderResourceViewRHIRef SRV;
	uint32 NumBytes = 0;

	FTextureRWBuffer() = default;

	~FTextureRWBuffer()
	{
		Release();
	}

	static constexpr ETextureCreateFlags DefaultTextureInitFlag = ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV;

	void Initialize2D(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 BytesPerElement, uint32 SizeX, uint32 SizeY, EPixelFormat Format, ETextureCreateFlags Flags = DefaultTextureInitFlag)
	{
		NumBytes = SizeX * SizeY * BytesPerElement;

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(InDebugName, SizeX, SizeY, Format)
			.SetFlags(Flags);

		Buffer = RHICmdList.CreateTexture(Desc);
		UAV = RHICmdList.CreateUnorderedAccessView(Buffer, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(Buffer));
		SRV = RHICmdList.CreateShaderResourceView(Buffer, FRHIViewDesc::CreateTextureSRV().SetDimensionFromTexture(Buffer));
	}

	UE_DEPRECATED(5.7, "A command list argument is now required")
	void Initialize2D(const TCHAR* InDebugName, uint32 BytesPerElement, uint32 SizeX, uint32 SizeY, EPixelFormat Format, ETextureCreateFlags Flags = DefaultTextureInitFlag)
	{
		FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();
		Initialize2D(RHICmdList, InDebugName, BytesPerElement, SizeX, SizeY, Format, Flags);
	}

	void Initialize3D(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 BytesPerElement, uint32 SizeX, uint32 SizeY, uint32 SizeZ, EPixelFormat Format, ETextureCreateFlags Flags = DefaultTextureInitFlag)
	{
		NumBytes = SizeX * SizeY * SizeZ * BytesPerElement;

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create3D(InDebugName, SizeX, SizeY, SizeZ, Format)
			.SetFlags(Flags);

		Buffer = RHICmdList.CreateTexture(Desc);
		UAV = RHICmdList.CreateUnorderedAccessView(Buffer, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(Buffer));
		SRV = RHICmdList.CreateShaderResourceView(Buffer, FRHIViewDesc::CreateTextureSRV().SetDimensionFromTexture(Buffer));
	}

	UE_DEPRECATED(5.7, "A command list argument is now required")
	void Initialize3D(const TCHAR* InDebugName, uint32 BytesPerElement, uint32 SizeX, uint32 SizeY, uint32 SizeZ, EPixelFormat Format, ETextureCreateFlags Flags = DefaultTextureInitFlag)
	{
		FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();
		Initialize3D(RHICmdList, InDebugName, BytesPerElement, SizeX, SizeY, SizeZ, Format, Flags);
	}

	void Release()
	{
		NumBytes = 0;
		Buffer.SafeRelease();
		UAV.SafeRelease();
		SRV.SafeRelease();
	}
};

/** Encapsulates a GPU read/write buffer with its UAV and SRV. */
struct FRWBuffer
{
	FBufferRHIRef Buffer;
	FUnorderedAccessViewRHIRef UAV;
	FShaderResourceViewRHIRef SRV;
	uint32 NumBytes;
	FName ClassName = NAME_None;	// The owner class of Buffer used for Insight asset metadata tracing, set it before calling Initialize()
	FName OwnerName = NAME_None;	// The owner name used for Insight asset metadata tracing, set it before calling Initialize()

	FRWBuffer()
		: NumBytes(0)
	{}

	FRWBuffer(FRWBuffer&& Other)
		: Buffer(MoveTemp(Other.Buffer))
		, UAV(MoveTemp(Other.UAV))
		, SRV(MoveTemp(Other.SRV))
		, NumBytes(Other.NumBytes)
	{
		Other.NumBytes = 0;
	}

	FRWBuffer(const FRWBuffer& Other)
		: Buffer(Other.Buffer)
		, UAV(Other.UAV)
		, SRV(Other.SRV)
		, NumBytes(Other.NumBytes)
	{
	}

	FRWBuffer& operator=(FRWBuffer&& Other)
	{
		Buffer = MoveTemp(Other.Buffer);
		UAV = MoveTemp(Other.UAV);
		SRV = MoveTemp(Other.SRV);
		NumBytes = Other.NumBytes;
		Other.NumBytes = 0;

		return *this;
	}

	FRWBuffer& operator=(const FRWBuffer& Other)
	{
		Buffer = Other.Buffer;
		UAV = Other.UAV;
		SRV = Other.SRV;
		NumBytes = Other.NumBytes;

		return *this;
	}

	~FRWBuffer()
	{
		Release();
	}

	// @param AdditionalUsage passed down to RHICreateBuffer(), get combined with "BUF_UnorderedAccess | BUF_ShaderResource" e.g. BUF_Static
	void Initialize(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 BytesPerElement, uint32 NumElements, EPixelFormat Format, ERHIAccess InResourceState, EBufferUsageFlags AdditionalUsage = BUF_None, FResourceArrayUploadInterface* InResourceArray = nullptr)
	{
		// Provide a debug name if using Fast VRAM so the allocators diagnostics will work
		ensure(!(EnumHasAnyFlags(AdditionalUsage, BUF_FastVRAM) && !InDebugName));

		NumBytes = BytesPerElement * NumElements;

		FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(InDebugName, NumBytes)
			.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | AdditionalUsage)
			.SetInitialState(InResourceState)
			.SetClassName(ClassName)
			.SetOwnerName(OwnerName);

		if (InResourceArray)
		{
			CreateDesc.SetInitActionResourceArray(InResourceArray);
		}

		Buffer = RHICmdList.CreateBuffer(CreateDesc);
		UAV = RHICmdList.CreateUnorderedAccessView(
			Buffer,
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format));

		SRV = RHICmdList.CreateShaderResourceView(
			Buffer,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format));
	}

	void Initialize(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 BytesPerElement, uint32 NumElements, EPixelFormat Format, EBufferUsageFlags AdditionalUsage = BUF_None, FResourceArrayUploadInterface* InResourceArray = nullptr)
	{
		Initialize(RHICmdList, InDebugName, BytesPerElement, NumElements, Format, ERHIAccess::UAVCompute, AdditionalUsage, InResourceArray);
	}

	template<typename LAMBDA>
	void InitializeWithData(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 BytesPerElement, uint32 NumElements, EPixelFormat Format, ERHIAccess InResourceState, EBufferUsageFlags AdditionalUsage, LAMBDA&& InitializerFunction)
	{
		// Provide a debug name if using Fast VRAM so the allocators diagnostics will work
		ensure(!(EnumHasAnyFlags(AdditionalUsage, BUF_FastVRAM) && !InDebugName));

		NumBytes = BytesPerElement * NumElements;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(InDebugName, NumBytes)
			.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | AdditionalUsage)
			.SetInitialState(InResourceState)
			.SetClassName(ClassName)
			.SetOwnerName(OwnerName)
			.SetInitActionInitializer();

		FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
		InitializerFunction(Initializer);
		Buffer = Initializer.Finalize();

		UAV = RHICmdList.CreateUnorderedAccessView(
			Buffer,
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format));
		SRV = RHICmdList.CreateShaderResourceView(
			Buffer,
			FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(Format));
	}

	void Release()
	{
		NumBytes = 0;
		Buffer.SafeRelease();
		UAV.SafeRelease();
		SRV.SafeRelease();
	}
};

/** Encapsulates a GPU read only texture 2D with its SRV. */
struct FTextureReadBuffer2D
{
	FTextureRHIRef Buffer;
	FShaderResourceViewRHIRef SRV;
	uint32 NumBytes = 0;

	FTextureReadBuffer2D() = default;
	~FTextureReadBuffer2D()
	{
		Release();
	}

	static constexpr ETextureCreateFlags DefaultTextureInitFlag = ETextureCreateFlags::ShaderResource;

	void Initialize(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 BytesPerElement, uint32 SizeX, uint32 SizeY, EPixelFormat Format, ETextureCreateFlags Flags = DefaultTextureInitFlag, FResourceBulkDataInterface* InBulkData = nullptr)
	{
		NumBytes = SizeX * SizeY * BytesPerElement;

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(InDebugName, SizeX, SizeY, Format)
			.SetFlags(Flags)
			.SetInitActionBulkData(InBulkData);

		Buffer = RHICmdList.CreateTexture(Desc);
		SRV = RHICmdList.CreateShaderResourceView(Buffer, FRHIViewDesc::CreateTextureSRV().SetDimensionFromTexture(Buffer));
	}

	UE_DEPRECATED(5.7, "A command list argument is now required")
	void Initialize(const TCHAR* InDebugName, uint32 BytesPerElement, uint32 SizeX, uint32 SizeY, EPixelFormat Format, ETextureCreateFlags Flags = DefaultTextureInitFlag, FResourceBulkDataInterface* InBulkData = nullptr)
	{
		FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();
		Initialize(RHICmdList, InDebugName, BytesPerElement, SizeX, SizeY, Format, Flags, InBulkData);
	}

	void Release()
	{
		NumBytes = 0;
		Buffer.SafeRelease();		
		SRV.SafeRelease();
	}
};

/** Encapsulates a GPU read buffer with its SRV. */
struct FReadBuffer
{
	FBufferRHIRef Buffer;
	FShaderResourceViewRHIRef SRV;
	uint32 NumBytes;

	FReadBuffer(): NumBytes(0) {}

	void Initialize(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 BytesPerElement, uint32 NumElements, EPixelFormat Format, EBufferUsageFlags AdditionalUsage = BUF_None, FResourceArrayUploadInterface* InResourceArray = nullptr)
	{
		NumBytes = BytesPerElement * NumElements;

		FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(InDebugName, NumBytes)
			.AddUsage(EBufferUsageFlags::ShaderResource | AdditionalUsage)
			.SetInitialState(ERHIAccess::SRVMask);

		if (InResourceArray)
		{
			CreateDesc.SetInitActionResourceArray(InResourceArray);
		}

		Buffer = RHICmdList.CreateBuffer(CreateDesc);
		SRV = RHICmdList.CreateShaderResourceView(
			Buffer, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format));
	}

	template<typename LAMBDA>
	void InitializeWithData(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 BytesPerElement, uint32 NumElements, EPixelFormat Format, EBufferUsageFlags AdditionalUsage, LAMBDA&& InitializerFunction)
	{
		NumBytes = BytesPerElement * NumElements;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(InDebugName, NumBytes)
			.AddUsage(EBufferUsageFlags::ShaderResource | AdditionalUsage)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
		InitializerFunction(Initializer);
		Buffer = Initializer.Finalize();


		SRV = RHICmdList.CreateShaderResourceView(
			Buffer, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format));
	}

	void Release()
	{
		NumBytes = 0;
		Buffer.SafeRelease();
		SRV.SafeRelease();
	}
};

/** Encapsulates a GPU read/write structured buffer with its UAV and SRV. */
struct FRWBufferStructured
{
	FBufferRHIRef Buffer;
	FUnorderedAccessViewRHIRef UAV;
	FShaderResourceViewRHIRef SRV;
	uint32 NumBytes;

	FRWBufferStructured(): NumBytes(0) {}

	~FRWBufferStructured()
	{
		Release();
	}

	void Initialize(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 BytesPerElement, uint32 NumElements, EBufferUsageFlags AdditionalUsage = EBufferUsageFlags::None, bool bUseUavCounter = false, bool bAppendBuffer = false, ERHIAccess InitialState = ERHIAccess::UAVMask)
	{
		// Provide a debug name if using Fast VRAM so the allocators diagnostics will work
		ensure(!(EnumHasAnyFlags(AdditionalUsage, EBufferUsageFlags::FastVRAM) && !InDebugName));

		NumBytes = BytesPerElement * NumElements;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured(InDebugName, NumBytes, BytesPerElement)
			.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | AdditionalUsage)
			.SetInitialState(InitialState);

		Buffer = RHICmdList.CreateBuffer(CreateDesc);
		UAV = RHICmdList.CreateUnorderedAccessView(
			Buffer,
			FRHIViewDesc::CreateBufferUAV()
				.SetTypeFromBuffer(Buffer)
				.SetAtomicCounter(bUseUavCounter)
				.SetAppendBuffer(bAppendBuffer));
		SRV = RHICmdList.CreateShaderResourceView(Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(Buffer));
	}

	template<typename LAMBDA>
	void InitializeWithData(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 BytesPerElement, uint32 NumElements, LAMBDA&& InitializerFunction, EBufferUsageFlags AdditionalUsage = EBufferUsageFlags::None, bool bUseUavCounter = false, bool bAppendBuffer = false, ERHIAccess InitialState = ERHIAccess::UAVMask)
	{
		// Provide a debug name if using Fast VRAM so the allocators diagnostics will work
		ensure(!(EnumHasAnyFlags(AdditionalUsage, EBufferUsageFlags::FastVRAM) && !InDebugName));

		NumBytes = BytesPerElement * NumElements;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured(InDebugName, NumBytes, BytesPerElement)
			.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | AdditionalUsage)
			.SetInitialState(InitialState)
			.SetInitActionInitializer();

		FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
		InitializerFunction(Initializer);
		Buffer = Initializer.Finalize();

		UAV = RHICmdList.CreateUnorderedAccessView(
			Buffer,
			FRHIViewDesc::CreateBufferUAV()
				.SetTypeFromBuffer(Buffer)
				.SetAtomicCounter(bUseUavCounter)
				.SetAppendBuffer(bAppendBuffer));
		SRV = RHICmdList.CreateShaderResourceView(Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(Buffer));
	}

	void Release()
	{
		NumBytes = 0;
		Buffer.SafeRelease();
		UAV.SafeRelease();
		SRV.SafeRelease();
	}
};

struct FByteAddressBuffer
{
	FBufferRHIRef Buffer;
	FShaderResourceViewRHIRef SRV;
	uint32 NumBytes = 0;

	FByteAddressBuffer() = default;

	void Initialize(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, uint32 InNumBytes, EBufferUsageFlags AdditionalUsage = EBufferUsageFlags::None)
	{
		NumBytes = InNumBytes;
		check( NumBytes % 4 == 0 );

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured(InDebugName, NumBytes, 4)
			.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::ByteAddressBuffer | AdditionalUsage)
			.SetInitialState(ERHIAccess::SRVMask);

		Buffer = RHICmdList.CreateBuffer(CreateDesc);
		SRV = RHICmdList.CreateShaderResourceView(Buffer, FRHIViewDesc::CreateBufferSRV().SetType(FRHIViewDesc::EBufferType::Raw));
	}

	template<typename TElement>
	void Initialize(FRHICommandListBase& RHICmdList, const TCHAR* InDebugName, TConstArrayView<TElement> InData)
	{
		NumBytes = InData.NumBytes();
		check( NumBytes % 4 == 0 );

		FResourceArrayUploadArrayView UploadView(InData);
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured(InDebugName, NumBytes, 4)
			.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::ByteAddressBuffer)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetInitActionResourceArray(&UploadView);

		Buffer = RHICmdList.CreateBuffer(CreateDesc);
		SRV = RHICmdList.CreateShaderResourceView(Buffer, FRHIViewDesc::CreateBufferSRV().SetType(FRHIViewDesc::EBufferType::Raw));
	}

	void Release()
	{
		NumBytes = 0;
		Buffer.SafeRelease();
		SRV.SafeRelease();
	}
};

/** Encapsulates a GPU read/write ByteAddress buffer with its UAV and SRV. */
struct FRWByteAddressBuffer : public FByteAddressBuffer
{
	FUnorderedAccessViewRHIRef UAV;

	void Initialize(FRHICommandListBase& RHICmdList, const TCHAR* DebugName, uint32 InNumBytes, EBufferUsageFlags AdditionalUsage = BUF_None)
	{
		FByteAddressBuffer::Initialize(RHICmdList, DebugName, InNumBytes, BUF_UnorderedAccess | AdditionalUsage);
		UAV = RHICmdList.CreateUnorderedAccessView(Buffer, FRHIViewDesc::CreateBufferUAV().SetType(FRHIViewDesc::EBufferType::Raw));
	}

	void Release()
	{
		FByteAddressBuffer::Release();
		UAV.SafeRelease();
	}
};

struct FDynamicReadBuffer : public FReadBuffer
{
	/** Pointer to the vertex buffer mapped in main memory. */
	uint8* MappedBuffer;

	/** Default constructor. */
	FDynamicReadBuffer()
		: MappedBuffer(nullptr)
	{
	}

	virtual ~FDynamicReadBuffer()
	{
		Release();
	}

	void Initialize(FRHICommandListBase& RHICmdList, const TCHAR* DebugName, uint32 BytesPerElement, uint32 NumElements, EPixelFormat Format, EBufferUsageFlags AdditionalUsage = BUF_None)
	{
		// Commented out due to the hack made in FGlobalDynamicReadBuffer::AllocateInternal to use static in D3D11.
		//ensure(
		//	EnumHasAnyFlags(AdditionalUsage, BUF_Dynamic | BUF_Volatile | BUF_Static) &&					// buffer should be Dynamic or Volatile or Static
		//	EnumHasAnyFlags(AdditionalUsage, BUF_Dynamic) != EnumHasAnyFlags(AdditionalUsage, BUF_Volatile) // buffer should not be both
		//	);

		FReadBuffer::Initialize(RHICmdList, DebugName, BytesPerElement, NumElements, Format, AdditionalUsage);
	}

	/**
	* Locks the vertex buffer so it may be written to.
	*/
	void Lock(FRHICommandListBase& RHICmdList)
	{
		check(MappedBuffer == nullptr);
		check(IsValidRef(Buffer));
		MappedBuffer = (uint8*)RHICmdList.LockBuffer(Buffer, 0, NumBytes, RLM_WriteOnly);
	}

	/**
	* Unocks the buffer so the GPU may read from it.
	*/
	void Unlock(FRHICommandListBase& RHICmdList)
	{
		check(MappedBuffer);
		check(IsValidRef(Buffer));
		RHICmdList.UnlockBuffer(Buffer);
		MappedBuffer = nullptr;
	}
};

/**
 * Convert the ESimpleRenderTargetMode into usable values 
 * @todo: Can we easily put this into a .cpp somewhere?
 */
RHI_API void DecodeRenderTargetMode(ESimpleRenderTargetMode Mode, ERenderTargetLoadAction& ColorLoadAction, ERenderTargetStoreAction& ColorStoreAction, ERenderTargetLoadAction& DepthLoadAction, ERenderTargetStoreAction& DepthStoreAction, ERenderTargetLoadAction& StencilLoadAction, ERenderTargetStoreAction& StencilStoreAction, FExclusiveDepthStencil DepthStencilUsage);

inline void TransitionRenderPassTargets(FRHICommandList& RHICmdList, const FRHIRenderPassInfo& RPInfo)
{
	FRHITransitionInfo Transitions[MaxSimultaneousRenderTargets];
	int32 TransitionIndex = 0;
	uint32 NumColorRenderTargets = RPInfo.GetNumColorRenderTargets();
	for (uint32 Index = 0; Index < NumColorRenderTargets; Index++)
	{
		const FRHIRenderPassInfo::FColorEntry& ColorRenderTarget = RPInfo.ColorRenderTargets[Index];
		if (ColorRenderTarget.RenderTarget != nullptr)
		{
			Transitions[TransitionIndex] = FRHITransitionInfo(ColorRenderTarget.RenderTarget, ERHIAccess::Unknown, ERHIAccess::RTV);
			TransitionIndex++;
		}
	}

	const FRHIRenderPassInfo::FDepthStencilEntry& DepthStencilTarget = RPInfo.DepthStencilRenderTarget;
	if (DepthStencilTarget.DepthStencilTarget != nullptr && (RPInfo.DepthStencilRenderTarget.ExclusiveDepthStencil.IsAnyWrite()))
	{
		RHICmdList.Transition(FRHITransitionInfo(DepthStencilTarget.DepthStencilTarget, ERHIAccess::Unknown, ERHIAccess::DSVRead | ERHIAccess::DSVWrite));
	}

	RHICmdList.Transition(MakeArrayView(Transitions, TransitionIndex));
}

/** Performs a clear render pass on an RHI texture. The texture is expected to be in the RTV state. */
inline void ClearRenderTarget(FRHICommandList& RHICmdList, FRHITexture* Texture, uint32 MipIndex = 0, uint32 ArraySlice = 0)
{
	check(Texture);
	FRHIRenderPassInfo Info(Texture, ERenderTargetActions::Clear_Store);
	Info.ColorRenderTargets[0].MipIndex = (uint8)MipIndex;
	Info.ColorRenderTargets[0].ArraySlice = (int32)ArraySlice;
	RHICmdList.BeginRenderPass(Info, TEXT("ClearRenderTarget"));
	RHICmdList.EndRenderPass();
}

inline void TransitionAndCopyTexture(FRHICommandList& RHICmdList, FRHITexture* SrcTexture, FRHITexture* DstTexture, const FRHICopyTextureInfo& Info)
{
	check(SrcTexture && DstTexture);
	check(SrcTexture->GetNumSamples() == DstTexture->GetNumSamples());

	if (SrcTexture == DstTexture)
	{
		RHICmdList.Transition({
			FRHITransitionInfo(SrcTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask)
		});
		return;
	}

	RHICmdList.Transition({
		FRHITransitionInfo(SrcTexture, ERHIAccess::Unknown, ERHIAccess::CopySrc),
		FRHITransitionInfo(DstTexture, ERHIAccess::Unknown, ERHIAccess::CopyDest)
	});

	RHICmdList.CopyTexture(SrcTexture, DstTexture, Info);

	RHICmdList.Transition({
		FRHITransitionInfo(SrcTexture, ERHIAccess::CopySrc,  ERHIAccess::SRVMask),
		FRHITransitionInfo(DstTexture, ERHIAccess::CopyDest, ERHIAccess::SRVMask)
	});
}

/**
 * Computes the vertex count for a given number of primitives of the specified type.
 * @param NumPrimitives The number of primitives.
 * @param PrimitiveType The type of primitives.
 * @returns The number of vertices.
 */
inline uint32 GetVertexCountForPrimitiveCount(uint32 NumPrimitives, uint32 PrimitiveType)
{
	static_assert(PT_Num == 6, "This function needs to be updated");
	uint32 Factor = (PrimitiveType == PT_TriangleList) ? 3 : (PrimitiveType == PT_LineList) ? 2 : (PrimitiveType == PT_RectList) ? 3 : 1;
	uint32 Offset = (PrimitiveType == PT_TriangleStrip) ? 2 : 0;

	return NumPrimitives * Factor + Offset;

}

inline uint32 ComputeAnisotropyRT(int32 InitializerMaxAnisotropy)
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MaxAnisotropy"));
	int32 CVarValue = CVar->GetValueOnAnyThread(); // this is sometimes called from main thread during initialization of static RHI states

	return FMath::Clamp(InitializerMaxAnisotropy > 0 ? InitializerMaxAnisotropy : CVarValue, 1, 16);
}

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
#define ENABLE_TRANSITION_DUMP 0
#else
#define ENABLE_TRANSITION_DUMP 1
#endif

class FDumpTransitionsHelper
{
public:
	RHI_API static void DumpResourceTransition(const FName& ResourceName, const ERHIAccess TransitionType);
	
private:
	static void DumpTransitionForResourceHandler();

	static TAutoConsoleVariable<FString> CVarDumpTransitionsForResource;
	static FAutoConsoleVariableSink CVarDumpTransitionsForResourceSink;
	static FName DumpTransitionForResource;
};

#if ENABLE_TRANSITION_DUMP
#define DUMP_TRANSITION(ResourceName, TransitionType) FDumpTransitionsHelper::DumpResourceTransition(ResourceName, TransitionType);
#else
#define DUMP_TRANSITION(ResourceName, TransitionType)
#endif

extern RHI_API void SetDepthBoundsTest(FRHICommandList& RHICmdList, float WorldSpaceDepthNear, float WorldSpaceDepthFar, const FMatrix& ProjectionMatrix);

/** Returns the value of the rhi.SyncInterval CVar. */
extern RHI_API uint32 RHIGetSyncInterval();

/** Returns the value of the rhi.SyncSlackMS CVar or length of a full frame interval if the frame offset system is disabled. */
extern RHI_API float RHIGetSyncSlackMS();

/** Returns the top and bottom vsync present thresholds (the values of rhi.PresentThreshold.Top and rhi.PresentThreshold.Bottom) */
extern RHI_API void RHIGetPresentThresholds(float& OutTopPercent, float& OutBottomPercent);

/** Returns the value of the rhi.SyncAllowVariable CVar. */
extern RHI_API bool RHIGetSyncAllowVariable();

extern RHI_API void RHITriggerTaskEventOnFlip(uint64 PresentIndex, const UE::Tasks::FTaskEvent& TaskEvent);

extern RHI_API void RHIInitializeFlipTracking();
extern RHI_API void RHIShutdownFlipTracking();

/** Sets the FrameIndex and InputTime for the current frame. */
extern RHI_API float RHIGetFrameTime();

extern RHI_API void RHICalculateFrameTime();

/** Returns the VendorID of the preferred vendor or -1 if none were specified. */
extern RHI_API EGpuVendorId RHIGetPreferredAdapterVendor();

class FRHIFrameInfo
{
public:
	static constexpr uint32 FrameInfoHistoryCount = 16;

	struct FRHIFrameInfoMarker
	{
		//Present Counter that correlate the the Game Frame Index
		uint64 PresentCounter = 0;
		//Wall clock cycles when Input was sample on the CPU
		uint64 InputCycle = 0;
		//Wall clock cycles when the frame was flip to the Display composition
		uint64 VSyncCycle = 0;
		//Frame Input latency from InputCycle to Display composition in milliseconds
		double InputLatencyMS = 0.0;
	};


	FRHIFrameInfo()		
	{}

	virtual ~FRHIFrameInfo() {}

	std::atomic<float> AverageInputLatencyMs = 0.0f;
	
public:
	void SetInputSampleMarker(uint64 FrameIndex) ;
	void SetPresentCounter(uint64 FrameIndex, uint64 PresentCounter);
	void SetVsyncDetails(const FRHIFlipDetails& NewFlipFrame);

	float GetAverageInputLatencyInMs()  { return AverageInputLatencyMs; }

private:
	void ComputeHistory(uint64 FrameIndex);
	void Trim();

	TMap<uint64,FRHIFrameInfoMarker> Frames;
	UE::FMutex FramesMutex;
};

#if UE_INPUT_LATENCY_TRACKING
extern RHI_API void RHIFrameInfoSetInputSampleMarker(uint64 FrameNumber);
extern RHI_API void RHIFrameInfoSetPresentCounter(uint64 FrameNumber, uint64 PresentCounter);
extern RHI_API void RHIFrameInfoSetVsyncDetails (const FRHIFlipDetails& NewFlipFrame);
#else
inline RHI_API void RHIFrameInfoSetInputSampleMarker(uint64 FrameNumber) {}
inline RHI_API void RHIFrameInfoSetPresentCounter(uint64 FrameNumber, uint64 PresentCounter) {}
inline RHI_API void RHIFrameInfoSetVsyncDetails(const FRHIFlipDetails& NewFlipFrame) {}
#endif 
