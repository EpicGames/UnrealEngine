// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHIValidation.h: Public Valdation RHI definitions.
=============================================================================*/

#pragma once 

#include "RHI.h"
#include "RHIDescriptorRange.h"
#include "RHIValidationCommon.h"
#include "RHIValidationUtils.h"
#include "DataDrivenShaderPlatformInfo.h"

#if ENABLE_RHI_VALIDATION

// Controls whether BUF_SourceCopy should be validate or not.
extern RHI_API bool GRHIValidateBufferSourceCopy;


// This is a macro because we only want to evaluate the message expression if the checked expression is false.
#define RHI_VALIDATION_CHECK(InExpression, InMessage) \
	do \
	{ \
		if(UNLIKELY(!(InExpression))) \
		{ \
			FValidationRHI::ReportValidationFailure(InMessage); \
		} \
	}while(0)

#define RHI_VALIDATION_CONTEXT_CHECK(InExpression, Context, InMessage) \
	do \
	{ \
		if(UNLIKELY(!(InExpression))) \
		{ \
			FValidationRHI::ReportValidationFailure(InMessage, Context); \
		} \
	}while(0)

class FValidationComputeContext;
class FValidationContext;

class FValidationRHI : public FDynamicRHI
{
public:
	RHI_API FValidationRHI(FDynamicRHI* InRHI);
	RHI_API virtual ~FValidationRHI();

	static inline void ValidateThreadGroupCount(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
	{
		RHI_VALIDATION_CHECK((ThreadGroupCountX <= (uint32)GRHIMaxDispatchThreadGroupsPerDimension.X), *FString::Printf(TEXT("ThreadGroupCountX is invalid: %u. Must be greater than 0 and less than %d"), ThreadGroupCountX, GRHIMaxDispatchThreadGroupsPerDimension.X));
		RHI_VALIDATION_CHECK((ThreadGroupCountY <= (uint32)GRHIMaxDispatchThreadGroupsPerDimension.Y), *FString::Printf(TEXT("ThreadGroupCountY is invalid: %u. Must be greater than 0 and less than %d"), ThreadGroupCountY, GRHIMaxDispatchThreadGroupsPerDimension.Y));
		RHI_VALIDATION_CHECK((ThreadGroupCountZ <= (uint32)GRHIMaxDispatchThreadGroupsPerDimension.Z), *FString::Printf(TEXT("ThreadGroupCountZ is invalid: %u. Must be greater than 0 and less than %d"), ThreadGroupCountZ, GRHIMaxDispatchThreadGroupsPerDimension.Z));
	}

	static inline void ValidateIndirectArgsBuffer(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset, uint32 ArgumentSize, uint32 ArgumentsBoundarySize)
	{
		auto GetBufferDesc = [&]() -> FString { return FString::Printf(TEXT("Buffer: %s, Size: %u, Stride: %u, Offset: %u, ArgSize: %u"), ArgumentBuffer->GetDebugName(), ArgumentBuffer->GetSize(), ArgumentBuffer->GetStride(), ArgumentOffset, ArgumentSize); };
		RHI_VALIDATION_CHECK(EnumHasAnyFlags(ArgumentBuffer->GetUsage(), EBufferUsageFlags::VertexBuffer | EBufferUsageFlags::ByteAddressBuffer), *FString::Printf(TEXT("Indirect argument buffer must be a vertex or byte address buffer to be used as an indirect dispatch parameter. %s"), *GetBufferDesc()));
		RHI_VALIDATION_CHECK(EnumHasAnyFlags(ArgumentBuffer->GetUsage(), EBufferUsageFlags::DrawIndirect), *FString::Printf(TEXT("Indirect dispatch parameter buffer was not flagged with BUF_DrawIndirect. %s"), *GetBufferDesc()));
		RHI_VALIDATION_CHECK((ArgumentOffset % 4) == 0, *FString::Printf(TEXT("Indirect argument offset must be a multiple of 4. %s"), *GetBufferDesc()));
		RHI_VALIDATION_CHECK((ArgumentOffset + ArgumentSize) <= ArgumentBuffer->GetSize(), *FString::Printf(TEXT("Indirect argument doesn't fit in the buffer. %s"), *GetBufferDesc()));
		if (ArgumentsBoundarySize > 0)
		{
			RHI_VALIDATION_CHECK(ArgumentOffset / ArgumentsBoundarySize == (ArgumentOffset + sizeof(FRHIDispatchIndirectParametersNoPadding) - 1) / ArgumentsBoundarySize,
				*FString::Printf(TEXT("Indirect arguments cannot cross %d byte boundary. %s"), ArgumentsBoundarySize, *GetBufferDesc()));
		}
	}

	static inline void ValidateDispatchIndirectArgsBuffer(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset)
	{
		ValidateIndirectArgsBuffer(ArgumentBuffer, ArgumentOffset, sizeof(FRHIDispatchIndirectParametersNoPadding), PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE);
	}

	virtual void Init() override final
	{
		RHI->Init();
		RHIName = RHI->GetName();
		RHIName += TEXT("_Validation");
		RenderThreadFrameID = 0;
		RHIThreadFrameID = 0;
	}

	/** Called after the RHI is initialized; before the render thread is started. */
	virtual void PostInit() override final
	{
		// Need to copy this as each DynamicRHI has an instance
		check(RHI->PixelFormatBlockBytes.Num() <= PixelFormatBlockBytes.Num());
		RHI->PixelFormatBlockBytes = PixelFormatBlockBytes;
		RHI->PostInit();
	}

	/** Shutdown the RHI; handle shutdown and resource destruction before the RHI's actual destructor is called (so that all resources of the RHI are still available for shutdown). */
	virtual void Shutdown() override final
	{
		RHI->Shutdown();
	}

	virtual const TCHAR* GetName() override final
	{
		return *RHIName;
	}

	virtual ERHIInterfaceType GetInterfaceType() const override final
	{
		return RHI->GetInterfaceType();
	}

	virtual FDynamicRHI* GetNonValidationRHI() override final
	{
		return RHI;
	}

	/////// RHI Methods

	virtual void RHIEndFrame_RenderThread(FRHICommandListImmediate& RHICmdList) override final;
	virtual void RHIEndFrame(const FRHIEndFrameArgs& Args) override final;

	// FlushType: Thread safe
	virtual FSamplerStateRHIRef RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer) override final
	{
		return RHI->RHICreateSamplerState(Initializer);
	}

	// FlushType: Thread safe
	virtual FRasterizerStateRHIRef RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer) override final
	{
		return RHI->RHICreateRasterizerState(Initializer);
	}

	// FlushType: Thread safe
	virtual FDepthStencilStateRHIRef RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer) override final
	{
		FDepthStencilStateRHIRef State = RHI->RHICreateDepthStencilState(Initializer);

		// @todo: remove this and use the PSO's dsmode instead?
		// Determine the actual depth stencil mode that applies for this state
		FExclusiveDepthStencil::Type DepthStencilMode = FExclusiveDepthStencil::DepthNop_StencilNop;
		if (Initializer.DepthTest != CF_Always || Initializer.bEnableDepthWrite)
		{
			DepthStencilMode = Initializer.bEnableDepthWrite
				? FExclusiveDepthStencil::DepthWrite
				: FExclusiveDepthStencil::DepthRead;
		}

		// set up stencil testing if it's enabled
		if (Initializer.bEnableFrontFaceStencil || Initializer.bEnableBackFaceStencil)
		{
			bool bBackFaceStencilWriteEnabled = false;

			// bEnableBackFaceStencil means to use separate settings for the Back, not if it's enabled at all
			if (Initializer.bEnableBackFaceStencil)
			{
				bBackFaceStencilWriteEnabled =
					(Initializer.BackFaceStencilFailStencilOp != SO_Keep ||
						Initializer.BackFacePassStencilOp != SO_Keep ||
						Initializer.BackFaceDepthFailStencilOp != SO_Keep);
			}

			if (Initializer.StencilReadMask != 0)
			{
				DepthStencilMode = FExclusiveDepthStencil::Type(DepthStencilMode | FExclusiveDepthStencil::StencilRead);
			}
			if (Initializer.StencilWriteMask != 0)
			{
				bool bFrontFaceStencilWriteEnabled =
					Initializer.FrontFaceStencilFailStencilOp != SO_Keep ||
					Initializer.FrontFacePassStencilOp != SO_Keep ||
					Initializer.FrontFaceDepthFailStencilOp != SO_Keep;

				if (bFrontFaceStencilWriteEnabled || bBackFaceStencilWriteEnabled)
				{
					DepthStencilMode = FExclusiveDepthStencil::Type(DepthStencilMode | FExclusiveDepthStencil::StencilWrite);
				}
			}
		}
		State->ActualDSMode = DepthStencilMode;
		// @todo: remove this and use the PSO's dsmode instead?
		
		DepthStencilStates.FindOrAdd(State.GetReference()) = Initializer;
		return State;
	}

	// FlushType: Thread safe
	virtual FBlendStateRHIRef RHICreateBlendState(const FBlendStateInitializerRHI& Initializer) override final
	{
		return RHI->RHICreateBlendState(Initializer);
	}

	// FlushType: Wait RHI Thread
	virtual FVertexDeclarationRHIRef RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements) override final
	{
		return RHI->RHICreateVertexDeclaration(Elements);
	}

	// FlushType: Wait RHI Thread
	virtual FPixelShaderRHIRef RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc) override final
	{
		return RHI->RHICreatePixelShader(CreateShaderDesc);
	}

	// FlushType: Wait RHI Thread
	virtual FVertexShaderRHIRef RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc) override final
	{
		return RHI->RHICreateVertexShader(CreateShaderDesc);
	}

	// FlushType: Wait RHI Thread
	virtual FGeometryShaderRHIRef RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc) override final
	{
		check(RHISupportsGeometryShaders(GMaxRHIShaderPlatform));
		return RHI->RHICreateGeometryShader(CreateShaderDesc);
	}

	// FlushType: Wait RHI Thread
	virtual FMeshShaderRHIRef RHICreateMeshShader(const FRHICreateShaderDesc& CreateShaderDesc)
	{
		check(RHISupportsMeshShadersTier0(GMaxRHIShaderPlatform));
		return RHI->RHICreateMeshShader(CreateShaderDesc);
	}

	// FlushType: Wait RHI Thread
	virtual FAmplificationShaderRHIRef RHICreateAmplificationShader(const FRHICreateShaderDesc& CreateShaderDesc)
	{
		check(RHISupportsMeshShadersTier0(GMaxRHIShaderPlatform));
		return RHI->RHICreateAmplificationShader(CreateShaderDesc);
	}

	// Some RHIs can have pending messages/logs for error tracking, or debug modes
	virtual void FlushPendingLogs() override final
	{
		RHI->FlushPendingLogs();
	}

	// FlushType: Wait RHI Thread
	virtual FComputeShaderRHIRef RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc) override final
	{
		return RHI->RHICreateComputeShader(CreateShaderDesc);
	}

	/**
	 * Attempts to open a shader library for the given shader platform & name within the provided directory.
	 * @param Platform The shader platform for shaders withing the library.
	 * @param FilePath The directory in which the library should exist.
	 * @param Name The name of the library, i.e. "Global" or "Unreal" without shader-platform or file-extension qualification.
	 * @return The new library if one exists and can be constructed, otherwise nil.
	 */
	 // FlushType: Must be Thread-Safe.
	virtual FRHIShaderLibraryRef RHICreateShaderLibrary(EShaderPlatform Platform, FString const& FilePath, FString const& Name) override final
	{
		return RHI->RHICreateShaderLibrary(Platform, FilePath, Name);
	}

	virtual FGPUFenceRHIRef RHICreateGPUFence(const FName &Name) override final
	{
		return RHI->RHICreateGPUFence(Name);
	}

	virtual void RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI) override final
	{
		RHI->RHIWriteGPUFence_TopOfPipe(RHICmdList, FenceRHI);
	}

	virtual void RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo);

	virtual void RHIReleaseTransition(FRHITransition* Transition)
	{
		RHI->RHIReleaseTransition(Transition);
	}

	virtual IRHITransientResourceAllocator* RHICreateTransientResourceAllocator() override final;

	/**
	* Creates a staging buffer, which is memory visible to the cpu without any locking.
	* @return The new staging-buffer.
	*/
	// FlushType: Thread safe.	
	virtual FStagingBufferRHIRef RHICreateStagingBuffer() override final
	{
		return RHI->RHICreateStagingBuffer();
	}

	/**
	 * Lock a staging buffer to read contents on the CPU that were written by the GPU, without having to stall.
	 * @discussion This function requires that you have issued an CopyToStagingBuffer invocation and verified that the FRHIGPUFence has been signaled before calling.
	 * @param StagingBuffer The buffer to lock.
	 * @param Fence An optional fence synchronized with the last buffer update.
	 * @param Offset The offset in the buffer to return.
	 * @param SizeRHI The length of the region in the buffer to lock.
	 * @returns A pointer to the data starting at 'Offset' and of length 'SizeRHI' from 'StagingBuffer', or nullptr when there is an error.
	 */
	virtual void* RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI) override final
	{
		return RHI->RHILockStagingBuffer(StagingBuffer, Fence, Offset, SizeRHI);
	}

	/**
	 * Unlock a staging buffer previously locked with RHILockStagingBuffer.
	 * @param StagingBuffer The buffer that was previously locked.
	 */
	virtual void RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer) override final
	{
		RHI->RHIUnlockStagingBuffer(StagingBuffer);
	}

	/**
	 * Lock a staging buffer to read contents on the CPU that were written by the GPU, without having to stall.
	 * @discussion This function requires that you have issued an CopyToStagingBuffer invocation and verified that the FRHIGPUFence has been signaled before calling.
	 * @param RHICmdList The command-list to execute on or synchronize with.
	 * @param StagingBuffer The buffer to lock.
	 * @param Fence An optional fence synchronized with the last buffer update.
	 * @param Offset The offset in the buffer to return.
	 * @param SizeRHI The length of the region in the buffer to lock.
	 * @returns A pointer to the data starting at 'Offset' and of length 'SizeRHI' from 'StagingBuffer', or nullptr when there is an error.
	 */
	virtual void* LockStagingBuffer_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI) override final
	{
		return RHI->LockStagingBuffer_RenderThread(RHICmdList, StagingBuffer, Fence, Offset, SizeRHI);
	}

	/**
	 * Unlock a staging buffer previously locked with LockStagingBuffer_RenderThread.
	 * @param RHICmdList The command-list to execute on or synchronize with.
	 * @param StagingBuffer The buffer what was previously locked.
	 */
	virtual void UnlockStagingBuffer_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHIStagingBuffer* StagingBuffer) override final
	{
		RHI->UnlockStagingBuffer_RenderThread(RHICmdList, StagingBuffer);
	}

	virtual void RHIMapStagingSurface_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, uint32 GPUIndex, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight)
	{
		RHI->RHIMapStagingSurface_RenderThread(RHICmdList, Texture, GPUIndex, Fence, OutData, OutWidth, OutHeight);
	}

	virtual void RHIUnmapStagingSurface_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, uint32 GPUIndex)
	{
		RHI->RHIUnmapStagingSurface_RenderThread(RHICmdList, Texture, GPUIndex);
	}

	virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIVertexDeclaration* VertexDeclaration, FRHIVertexShader* VertexShader, FRHIPixelShader* PixelShader, FRHIGeometryShader* GeometryShader) override final
	{
		return RHI->RHICreateBoundShaderState(VertexDeclaration, VertexShader, PixelShader, GeometryShader);
	}

#if PLATFORM_SUPPORTS_MESH_SHADERS && PLATFORM_USE_FALLBACK_PSO
	virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIAmplificationShader* AmplificationShader, FRHIMeshShader* MeshShader, FRHIPixelShader* PixelShader) override final
	{
		return RHI->RHICreateBoundShaderState(AmplificationShader, MeshShader, PixelShader);
	}
#endif

	virtual FGraphicsPipelineStateRHIRef RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer) override final
	{
		ValidatePipeline(Initializer);
		FGraphicsPipelineStateRHIRef PSO = RHI->RHICreateGraphicsPipelineState(Initializer);
		if (PSO.IsValid())
		{
			PSO->DSMode = Initializer.DepthStencilState->ActualDSMode;
		}
		return PSO;
	}

	virtual TRefCountPtr<FRHIComputePipelineState> RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer) override final
	{
		return RHI->RHICreateComputePipelineState(Initializer);
	}

	/**
	* Creates a uniform buffer.  The contents of the uniform buffer are provided in a parameter, and are immutable.
	* CAUTION: Even though this is marked as threadsafe, it is only valid to call from the render thread or the RHI thread. Thus is need not be threadsafe on platforms that do not support or aren't using an RHIThread
	* @param Contents - A pointer to a memory block of size NumBytes that is copied into the new uniform buffer.
	* @param NumBytes - The number of bytes the uniform buffer should contain.
	* @return The new uniform buffer.
	*/
	// FlushType: Thread safe, but varies depending on the RHI override final
	virtual FUniformBufferRHIRef RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation) override final
	{
		check(Layout);
		check(Layout->Resources.Num() > 0 || Layout->ConstantBufferSize > 0);
		FUniformBufferRHIRef UniformBuffer = RHI->RHICreateUniformBuffer(Contents, Layout, Usage, Validation);

		// Use the render thread frame ID for any non RHI thread allocations. TODO: This is actually incorrect as command list recording on the render thread timeline
		// can straddle EndFrame boundaries, causing a uniform buffer allocated in frame N to be recorded as being allocated in frame N+1. The solution here is to introduce
		// a command list to RHICreateUniformBuffer so that the correct render thread frame can be propagated. Unfortunately, in the meantime, this means that the lifetime
		// tracker can miss legitimate cases. For example, if a (single frame) uniform buffer is allocated in frame N (but straddles to frame N+1), and then is incorrectly
		// used in frame N+1, that test will pass because they are equal. However, since the issue is timing dependent, it's still likely to catch legitimate allocation misuses.
		UniformBuffer->InitLifetimeTracking(IsInRHIThread() ? RHIThreadFrameID : RenderThreadFrameID.load(std::memory_order_relaxed), Contents, Usage);
		return UniformBuffer;
	}

	virtual void RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents) override final
	{
		check(UniformBufferRHI);
		check(Contents);
		RHI->RHIUpdateUniformBuffer(RHICmdList, UniformBufferRHI, Contents);

		RHICmdList.EnqueueLambda([this, UniformBufferRHI] (FRHICommandListBase& RHICmdList)
		{
			UniformBufferRHI->UpdateAllocation(RHIThreadFrameID);
		});
	}

	[[nodiscard]] virtual FRHIBufferInitializer RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc) final override
	{
		return RHI->RHICreateBufferInitializer(RHICmdList, CreateDesc);
	}

	// FlushType: Flush RHI Thread
	virtual void* RHILockBuffer(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode) override final;
	virtual void* RHILockBufferMGPU(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 GPUIndex, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode) override final;

	// FlushType: Flush RHI Thread
	virtual void RHIUnlockBuffer(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) override final
	{
		RHI->RHIUnlockBuffer(RHICmdList, Buffer);
	}
	virtual void RHIUnlockBufferMGPU(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 GPUIndex) override final
	{
		RHI->RHIUnlockBufferMGPU(RHICmdList, Buffer, GPUIndex);
	}

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	virtual void RHIUpdateAllocationTags(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) override final
	{
		RHI->RHIUpdateAllocationTags(RHICmdList, Buffer);
	}
#endif

	virtual FTextureReferenceRHIRef RHICreateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* InReferencedTexture)
	{
		return RHI->RHICreateTextureReference(RHICmdList, InReferencedTexture);
	}
	virtual void RHIUpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITextureReference* TextureRef, FRHITexture* NewTexture)
	{
		RHI->RHIUpdateTextureReference(RHICmdList, TextureRef, NewTexture);
	}

	virtual FShaderResourceViewRHIRef RHICreateShaderResourceView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) override final;
	virtual FUnorderedAccessViewRHIRef RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) override final;

	virtual FRHIResourceCollectionRef RHICreateResourceCollection(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers) override final
	{
		return RHI->RHICreateResourceCollection(RHICmdList, InMembers);
	}

	virtual void RHIUpdateResourceCollection(FRHICommandListBase& RHICmdList, FRHIResourceCollection* InResourceCollection, uint32 InStartIndex, TConstArrayView<FRHIResourceCollectionMember> InMemberUpdates) override final
	{
		RHI_VALIDATION_CHECK(InStartIndex < static_cast<uint32>(InResourceCollection->GetMembers().Num()) && InStartIndex + InMemberUpdates.Num() <= static_cast<uint32>(InResourceCollection->GetMembers().Num()),
			*FString::Printf(TEXT("ResourceCollection update range of [%d, %d) is invalid for a ResourceCollection of size '%d'"), InStartIndex, InStartIndex + InMemberUpdates.Num(), InResourceCollection->GetMembers().Num()));

		RHI->RHIUpdateResourceCollection(RHICmdList, InResourceCollection, InStartIndex, InMemberUpdates);
	}

	virtual FRHICalcTextureSizeResult RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex) override final
	{
		ensure(Desc.IsValid());
		ensure(FirstMipIndex < Desc.NumMips);

		return RHI->RHICalcTexturePlatformSize(Desc, FirstMipIndex);
	}

	/**
	* Retrieves texture memory stats.
	* safe to call on the main thread
	*/
	// FlushType: Thread safe
	virtual void RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats) override final
	{
		RHI->RHIGetTextureMemoryStats(OutStats);
	}

	/**
	* Fills a texture with to visualize the texture pool memory.
	*
	* @param	TextureData		Start address
	* @param	SizeX			Number of pixels along X
	* @param	SizeY			Number of pixels along Y
	* @param	Pitch			Number of bytes between each row
	* @param	PixelSize		Number of bytes each pixel represents
	*
	* @return true if successful, false otherwise
	*/
	// FlushType: Flush Immediate
	virtual bool RHIGetTextureMemoryVisualizeData(FColor* TextureData, int32 SizeX, int32 SizeY, int32 Pitch, int32 PixelSize) override final
	{
		return RHI->RHIGetTextureMemoryVisualizeData(TextureData, SizeX, SizeY, Pitch, PixelSize);
	}

	struct FRHIValidationTextureInitializer : public FRHITextureInitializer
	{
		FRHIValidationTextureInitializer(const FRHITextureCreateDesc& CreateDesc, FRHITextureInitializer&& InOther)
			: FRHITextureInitializer(MoveTemp(InOther))
		{
			FFinalizeCallback OriginalFinalizeCallback = MoveTemp(FinalizeCallback);
			FinalizeCallback = [OriginalFinalizeCallback = MoveTemp(OriginalFinalizeCallback)](FRHICommandListBase& RHICmdList)
				{
					FTextureRHIRef Texture = OriginalFinalizeCallback(RHICmdList);
					ensure(Texture->IsBarrierTrackingInitialized());
					return Texture;
				};

			// ue-todo: custom GetSubresourceCallback?
		}
	};

	[[nodiscard]]
	virtual FRHITextureInitializer RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc) override final
	{
		CreateDesc.CheckValidity();
		return FRHIValidationTextureInitializer(CreateDesc, RHI->RHICreateTextureInitializer(RHICmdList, CreateDesc));
	}


	/**
	* Thread-safe function that can be used to create a texture outside of the
	* rendering thread. This function can ONLY be called if GRHISupportsAsyncTextureCreation
	* is true.  Cannot create rendertargets with this method.
	* @param SizeX - width of the texture to create
	* @param SizeY - height of the texture to create
	* @param Format - EPixelFormat texture format
	* @param NumMips - number of mips to generate or 0 for full mip pyramid
	* @param Flags - ETextureCreateFlags creation flags
	* @param InitialMipData - pointers to mip data with which to create the texture
	* @param NumInitialMips - how many mips are provided in InitialMipData
	* @param OutCompletionEvent - An event signaled on operation completion. Can return null. Operation can still be pending after function returns (e.g. initial data upload in-flight)
	* @returns a reference to a 2D texture resource
	*/
	// FlushType: Thread safe
	virtual FTextureRHIRef RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent) override final
	{
		check(GRHISupportsAsyncTextureCreation);
		ensure(FMath::Max(SizeX, SizeY) >= (1u << (FMath::Max(1u, NumMips) - 1)));
		FTextureRHIRef Texture = RHI->RHIAsyncCreateTexture2D(SizeX, SizeY, Format, NumMips, Flags, InResourceState, InitialMipData, NumInitialMips, DebugName, OutCompletionEvent);
		ensure(Texture->IsBarrierTrackingInitialized());
		return Texture;
	}

	virtual void RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos) override final
	{
		RHI->RHIReplaceResources(RHICmdList, MoveTemp(ReplaceInfos));
	}

	/**
	* Computes the size in memory required by a given texture.
	*
	* @param	TextureRHI		- Texture we want to know the size of, 0 is safely ignored
	* @return					- Size in Bytes
	*/
	// FlushType: Thread safe
	virtual uint32 RHIComputeMemorySize(FRHITexture* TextureRHI) override final
	{
		return RHI->RHIComputeMemorySize(TextureRHI);
	}

	virtual FRHILockTextureResult RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;
	virtual void RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;

	/**
	* Updates a region of a 2D texture from system memory
	* @param Texture - the RHI texture resource to update
	* @param MipIndex - mip level index to be modified
	* @param UpdateRegion - The rectangle to copy source image data from
	* @param SourcePitch - size in bytes of each row of the source image
	* @param SourceData - source image data, starting at the upper left corner of the source rectangle (in same pixel format as texture)
	*/
	// FlushType: Flush RHI Thread
	virtual void RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData) override final
	{
		RHI->RHIUpdateTexture2D(RHICmdList, Texture, MipIndex, UpdateRegion, SourcePitch, SourceData);
	}

	virtual void RHIUpdateFromBufferTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, FRHIBuffer* Buffer, uint32 BufferOffset) override final
	{
		RHI->RHIUpdateFromBufferTexture2D(RHICmdList, Texture, MipIndex, UpdateRegion, SourcePitch, Buffer, BufferOffset);
	}

	/**
	* Updates a region of a 3D texture from system memory
	* @param Texture - the RHI texture resource to update
	* @param MipIndex - mip level index to be modified
	* @param UpdateRegion - The rectangle to copy source image data from
	* @param SourceRowPitch - size in bytes of each row of the source image, usually Bpp * SizeX
	* @param SourceDepthPitch - size in bytes of each depth slice of the source image, usually Bpp * SizeX * SizeY
	* @param SourceData - source image data, starting at the upper left corner of the source rectangle (in same pixel format as texture)
	*/
	// FlushType: Flush RHI Thread
	virtual void RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData) override final
	{
		RHI->RHIUpdateTexture3D(RHICmdList, Texture, MipIndex, UpdateRegion, SourceRowPitch, SourceDepthPitch, SourceData);
	}

	// FlushType: Thread safe
	virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const TCHAR* Name) override final;

	virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, const TCHAR* Name) override final;

	virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIUnorderedAccessView* UnorderedAccessViewRHI, const TCHAR* Name) override final;

	/**
	* Reads the contents of a texture to an output buffer (non MSAA and MSAA) and returns it as a FColor array.
	* If the format or texture type is unsupported the OutData array will be size 0
	*/
	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIReadSurfaceData(FRHITexture* Texture, FIntRect Rect, TArray<FColor>& OutData, FReadSurfaceDataFlags InFlags) override final
	{
		RHI->RHIReadSurfaceData(Texture, Rect, OutData, InFlags);
	}

	// Default fallback; will not work for non-8-bit surfaces and it's extremely slow.
	virtual void RHIReadSurfaceData(FRHITexture* Texture, FIntRect Rect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags) override final
	{
		RHI->RHIReadSurfaceData(Texture, Rect, OutData, InFlags);
	}

	/** Watch out for OutData to be 0 (can happen on DXGI_ERROR_DEVICE_REMOVED), don't call RHIUnmapStagingSurface in that case. */
	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIMapStagingSurface(FRHITexture* Texture, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex = 0) final override
	{
		RHI->RHIMapStagingSurface(Texture, Fence, OutData, OutWidth, OutHeight, GPUIndex);
	}

	/** call after a succesful RHIMapStagingSurface() call */
	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIUnmapStagingSurface(FRHITexture* Texture, uint32 GPUIndex = 0) override final
	{
		RHI->RHIUnmapStagingSurface(Texture, GPUIndex);
	}

	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIReadSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, TArray<FFloat16Color>& OutData, ECubeFace CubeFace, int32 ArrayIndex, int32 MipIndex) override final
	{
		RHI->RHIReadSurfaceFloatData(Texture, Rect, OutData, CubeFace, ArrayIndex, MipIndex);
	}

	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData) override final
	{
		RHI->RHIRead3DSurfaceFloatData(Texture, Rect, ZMinMax, OutData);
	}

	virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags InFlags) override final
	{
		RHI->RHIRead3DSurfaceFloatData(Texture, Rect, ZMinMax, OutData, InFlags);
	}

	// FlushType: Wait RHI Thread
	virtual FRenderQueryRHIRef RHICreateRenderQuery(ERenderQueryType QueryType) override final
	{
		return RHI->RHICreateRenderQuery(QueryType);
	}
	// CAUTION: Even though this is marked as threadsafe, it is only valid to call from the render thread. It is need not be threadsafe on platforms that do not support or aren't using an RHIThread
	// FlushType: Thread safe, but varies by RHI
	virtual bool RHIGetRenderQueryResult(FRHIRenderQuery* RenderQuery, uint64& OutResult, bool bWait, uint32 GPUIndex = INDEX_NONE) override final
	{
		return RHI->RHIGetRenderQueryResult(RenderQuery, OutResult, bWait, GPUIndex);
	}

	virtual void RHIBeginRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType) override final
	{
		RHI->RHIBeginRenderQueryBatch_TopOfPipe(RHICmdList, QueryType);
	}

	virtual void RHIEndRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType) override final
	{
		RHI->RHIEndRenderQueryBatch_TopOfPipe(RHICmdList, QueryType);
	}

	virtual void RHIBeginRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery) override final
	{
		RHI->RHIBeginRenderQuery_TopOfPipe(RHICmdList, RenderQuery);
	}

	virtual void RHIEndRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery) override final
	{
		RHI->RHIEndRenderQuery_TopOfPipe(RHICmdList, RenderQuery);
	}

	// FlushType: Thread safe
	virtual uint32 RHIGetViewportNextPresentGPUIndex(FRHIViewport* Viewport) override final
	{
		return RHI->RHIGetViewportNextPresentGPUIndex(Viewport);
	}

	// With RHI thread, this is the current backbuffer from the perspective of the render thread.
	// FlushType: Thread safe
	virtual FTextureRHIRef RHIGetViewportBackBuffer(FRHIViewport* Viewport) override final
	{
		FTextureRHIRef Texture = RHI->RHIGetViewportBackBuffer(Viewport);
		if (!Texture->GetTrackerResource()->IsBarrierTrackingInitialized())
		{
			// Assume present and renderer needs to perform transition to RTV if needed
			ERHIAccess ResourceState = ERHIAccess::Present;
			Texture->InitBarrierTracking(Texture->GetNumMips(), Texture->GetSizeXYZ().Z, Texture->GetFormat(), Texture->GetFlags(), ResourceState, TEXT("ViewportTexture"));
		}
		return Texture;
	}

	virtual uint32 RHIGetHTilePlatformConfig(uint32 DepthWidth, uint32 DepthHeight) const override final
	{
		return RHI->RHIGetHTilePlatformConfig(DepthWidth, DepthHeight);
	}

	virtual uint32 RHIGetHTilePlatformConfig(const FRHITextureDesc& DepthDesc) const override final
	{
		return RHI->RHIGetHTilePlatformConfig(DepthDesc);
	}

	virtual void RHIAliasTextureResources(FTextureRHIRef& DestTexture, FTextureRHIRef& SourceTexture) override final
	{
		// Source and target need to be valid objects.
		check(DestTexture && SourceTexture);
		// Source texture must have been created (i.e. have a native resource backing).
		check(SourceTexture->GetNativeResource() != nullptr);
		RHI->RHIAliasTextureResources(DestTexture, SourceTexture);
	}

	virtual FTextureRHIRef RHICreateAliasedTexture(FTextureRHIRef& SourceTexture) override final
	{
		check(SourceTexture);
		return RHI->RHICreateAliasedTexture(SourceTexture);
	}

	virtual void RHIGetDisplaysInformation(FDisplayInformationArray& OutDisplayInformation) override final
	{
		RHI->RHIGetDisplaysInformation(OutDisplayInformation);
	}

	virtual void RHIAcquireThreadOwnership() override final
	{
		RHI->RHIAcquireThreadOwnership();
	}

	virtual void RHIReleaseThreadOwnership() override final
	{
		RHI->RHIReleaseThreadOwnership();
	}

	virtual FViewportRHIRef RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) override final
	{
		check(PixelFormat != PF_Unknown);
		return RHI->RHICreateViewport(WindowHandle, SizeX, SizeY, bIsFullscreen, PixelFormat);
	}

	virtual void RHIResizeViewport(FRHIViewport* Viewport, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) override final
	{
		check(PixelFormat != PF_Unknown);
		RHI->RHIResizeViewport(Viewport, SizeX, SizeY, bIsFullscreen, PixelFormat);
	}

	virtual void RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* Viewport, const FRHIPresentArgs& PresentArgs) override final
	{
		RHI->RHIEndDrawingViewport(RHICmdList, Viewport, PresentArgs);
	}

	virtual void RHIHandleDisplayChange() override final
	{
		RHI->RHIHandleDisplayChange();
	}

	//  must be called from the main thread.
	// FlushType: Thread safe
	virtual void RHITick(float DeltaTime) override final
	{
		RHI->RHITick(DeltaTime);
	}

	// Blocks the CPU until the GPU catches up and goes idle.
	// FlushType: Flush Immediate (seems wrong)
	virtual void RHIBlockUntilGPUIdle() override final
	{
		RHI->RHIBlockUntilGPUIdle();
	}

	// Tells the RHI we're about to suspend it
	virtual void RHIBeginSuspendRendering() override final
	{
		RHI->RHIBeginSuspendRendering();
	}

	// Operations to suspend title rendering and yield control to the system
	// FlushType: Thread safe
	virtual void RHISuspendRendering() override final
	{
		RHI->RHISuspendRendering();
	}

	// FlushType: Thread safe
	virtual void RHIResumeRendering() override final
	{
		RHI->RHIResumeRendering();
	}

	// FlushType: Flush Immediate
	virtual bool RHIIsRenderingSuspended() override final
	{
		return RHI->RHIIsRenderingSuspended();
	}

	/**
	*	Retrieve available screen resolutions.
	*
	*	@param	Resolutions			TArray<FScreenResolutionRHI> parameter that will be filled in.
	*	@param	bIgnoreRefreshRate	If true, ignore refresh rates.
	*
	*	@return	bool				true if successfully filled the array
	*/
	// FlushType: Thread safe
	virtual bool RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate) override final
	{
		return RHI->RHIGetAvailableResolutions(Resolutions, bIgnoreRefreshRate);
	}

	/**
	* Returns a supported screen resolution that most closely matches input.
	* @param Width - Input: Desired resolution width in pixels. Output: A width that the platform supports.
	* @param Height - Input: Desired resolution height in pixels. Output: A height that the platform supports.
	*/
	// FlushType: Thread safe
	virtual void RHIGetSupportedResolution(uint32& Width, uint32& Height) override final
	{
		RHI->RHIGetSupportedResolution(Width, Height);
	}

	/**
	* Function that is used to allocate / free space used for virtual texture mip levels.
	* Make sure you also update the visible mip levels.
	* @param Texture - the texture to update, must have been created with TexCreate_Virtual
	* @param FirstMip - the first mip that should be in memory
	*/
	// FlushType: Wait RHI Thread
	virtual void RHIVirtualTextureSetFirstMipInMemory(FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, uint32 FirstMip) override final
	{
		RHI->RHIVirtualTextureSetFirstMipInMemory(RHICmdList, Texture, FirstMip);
	}

	/**
	* Function that can be used to update which is the first visible mip to the GPU.
	* @param Texture - the texture to update, must have been created with TexCreate_Virtual
	* @param FirstMip - the first mip that should be visible to the GPU
	*/
	// FlushType: Wait RHI Thread
	virtual void RHIVirtualTextureSetFirstMipVisible(FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, uint32 FirstMip) override final
	{
		RHI->RHIVirtualTextureSetFirstMipVisible(RHICmdList, Texture, FirstMip);
	}

	/**
	* Provides access to the native device. Generally this should be avoided but is useful for third party plugins.
	*/
	// FlushType: Flush RHI Thread
	virtual void* RHIGetNativeDevice() override final
	{
		return RHI->RHIGetNativeDevice();
	}

	/**
	* Provides access to the native physical device. Generally this should be avoided but is useful for third party plugins.
	*/
	// FlushType: Flush RHI Thread
	virtual void* RHIGetNativePhysicalDevice() override final
	{
		return RHI->RHIGetNativePhysicalDevice();
	}

	/**
	* Provides access to the native graphics command queue. Generally this should be avoided but is useful for third party plugins.
	*/
	// FlushType: Flush RHI Thread
	virtual void* RHIGetNativeGraphicsQueue() override final
	{
		return RHI->RHIGetNativeGraphicsQueue();
	}

	/**
	* Provides access to the native compute command queue. Generally this should be avoided but is useful for third party plugins.
	*/
	// FlushType: Flush RHI Thread
	virtual void* RHIGetNativeComputeQueue() override final
	{
		return RHI->RHIGetNativeComputeQueue();
	}

	virtual IRHICommandContext* RHIGetDefaultContext() override final;
	virtual IRHIComputeContext* RHIGetCommandContext(ERHIPipeline Pipeline) override final;
	virtual IRHIUploadContext* RHIGetUploadContext() final override
	{
		return RHI->RHIGetUploadContext();
	}

	virtual FRHIFinalizeContextsResult RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args) override final;
	virtual FRHICloseTranslateChainResult RHICloseTranslateChain(FRHICloseTranslateChainArgs&& Args) override final;

	virtual void RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args) override final;
	
	virtual IRHICommandContext* RHIGetParallelCommandContext(FRHIParallelRenderPassInfo const& ParallelRenderPass) override final;

	virtual void RHIProcessDeleteQueue() override final
	{
		RHI->RHIProcessDeleteQueue();
	}

	virtual uint64 RHIGetMinimumAlignmentForBufferBackedSRV(EPixelFormat Format) override final
	{
		return RHI->RHIGetMinimumAlignmentForBufferBackedSRV(Format);
	}

	virtual void* LockBuffer_BottomOfPipe(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode) override final
	{
		RHI_VALIDATION_CHECK(LockMode != RLM_WriteOnly_NoOverwrite || GRHISupportsMapWriteNoOverwrite, TEXT("Using RLM_WriteOnly_NoOverwrite when the RHI doesn't support it."));
		return RHI->LockBuffer_BottomOfPipe(RHICmdList, Buffer, Offset, SizeRHI, LockMode);
	}
	
	virtual void UnlockBuffer_BottomOfPipe(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) override final
	{
		RHI->UnlockBuffer_BottomOfPipe(RHICmdList, Buffer);
	}

	virtual FUpdateTexture3DData RHIBeginUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion) override final
	{
		return RHI->RHIBeginUpdateTexture3D(RHICmdList, Texture, MipIndex, UpdateRegion);
	}

	virtual void RHIEndUpdateTexture3D(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData) override final
	{
		RHI->RHIEndUpdateTexture3D(RHICmdList, UpdateData);
	}

	virtual void RHIEndMultiUpdateTexture3D(FRHICommandListBase& RHICmdList, TArray<FUpdateTexture3DData>& UpdateDataArray) override final
	{
		RHI->RHIEndMultiUpdateTexture3D(RHICmdList, UpdateDataArray);
	}

	virtual void RHIReadSurfaceFloatData_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, FIntRect Rect, TArray<FFloat16Color>& OutData, ECubeFace CubeFace, int32 ArrayIndex, int32 MipIndex) override final
	{
		RHI->RHIReadSurfaceFloatData_RenderThread(RHICmdList, Texture, Rect, OutData, CubeFace, ArrayIndex, MipIndex);
	}

	virtual FRHIFlipDetails RHIWaitForFlip(double TimeoutInSeconds) override final
	{
		return RHI->RHIWaitForFlip(TimeoutInSeconds);
	}

	virtual void RHISignalFlipEvent() override final
	{
		RHI->RHISignalFlipEvent();
	}

	virtual uint16 RHIGetPlatformTextureMaxSampleCount() override final
	{
		return RHI->RHIGetPlatformTextureMaxSampleCount();
	};


#if RHI_RAYTRACING
	virtual FRayTracingGeometryRHIRef RHICreateRayTracingGeometry(FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& Initializer) override final
	{
		FRayTracingGeometryRHIRef Result = RHI->RHICreateRayTracingGeometry(RHICmdList, Initializer);
		Result->InitBarrierTracking(ERHIAccess::BVHWrite, *Initializer.DebugName.ToString()); // BVHs are always created in BVHWrite state
		return Result;
	}

	virtual FRayTracingSceneRHIRef RHICreateRayTracingScene(FRayTracingSceneInitializer Initializer) override final
	{
		FRayTracingSceneRHIRef Result = RHI->RHICreateRayTracingScene(MoveTemp(Initializer));
		Result->InitBarrierTracking(ERHIAccess::BVHWrite, *Initializer.DebugName.ToString()); // BVHs are always created in BVHWrite state
		return Result;
	}

	virtual FRayTracingShaderRHIRef RHICreateRayTracingShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency) override final
	{
		return RHI->RHICreateRayTracingShader(CreateShaderDesc, ShaderFrequency);
	}

	virtual FRayTracingPipelineStateRHIRef RHICreateRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer) override final
	{
		return RHI->RHICreateRayTracingPipelineState(Initializer);
	}

	virtual FRayTracingAccelerationStructureSize RHICalcRayTracingSceneSize(const FRayTracingSceneInitializer& Initializer) override final
	{
		return RHI->RHICalcRayTracingSceneSize(Initializer);
	}

	virtual FRayTracingAccelerationStructureSize RHICalcRayTracingGeometrySize(const FRayTracingGeometryInitializer& Initializer) override final
	{
		return RHI->RHICalcRayTracingGeometrySize(Initializer);
	}

	virtual FRayTracingClusterOperationSize RHICalcRayTracingClusterOperationSize(const FRayTracingClusterOperationInitializer& Initializer) override final
	{
		return RHI->RHICalcRayTracingClusterOperationSize(Initializer);
	}

	virtual FRayTracingAccelerationStructureOfflineMetadata RHIGetRayTracingGeometryOfflineMetadata(const FRayTracingGeometryOfflineDataHeader& OfflineDataHeader)
	{
		return RHI->RHIGetRayTracingGeometryOfflineMetadata(OfflineDataHeader);
	}

	virtual FShaderBindingTableRHIRef RHICreateShaderBindingTable(FRHICommandListBase& RHICmdList, const FRayTracingShaderBindingTableInitializer& Initializer) override final
	{
		return RHI->RHICreateShaderBindingTable(RHICmdList, Initializer);
	}

#if !UE_BUILD_SHIPPING
	void RHISerializeAccelerationStructure(FRHICommandListImmediate& RHICmdList, FRHIRayTracingScene* Scene, const TCHAR* Path)
	{
		return RHI->RHISerializeAccelerationStructure(RHICmdList, Scene, Path);
	}
#endif
#endif // RHI_RAYTRACING

	virtual FShaderBundleRHIRef RHICreateShaderBundle(const FShaderBundleCreateInfo& CreateInfo) override final
	{
		return RHI->RHICreateShaderBundle(CreateInfo);
	}

	virtual void RHIWriteResourceGPUAddresses(FRHICommandListBase& RHICmdList, uint32 GPUIndex, TConstArrayView<FRHIResource*> Resources, uint64* AddressesPtr)
	{
		return RHI->RHIWriteResourceGPUAddresses(RHICmdList, GPUIndex, Resources, AddressesPtr);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	RHI_API static void ValidateDescriptorRangeResources(const FRHIDescriptorRangeDesc& RangeDesc, TConstArrayView<FRHIDescriptorResource> Resources);

	virtual FDescriptorRangeRHIRef RHICreateDescriptorRange(FRHICommandListBase& RHICmdList, const FRHIDescriptorRangeDesc& Desc, TConstArrayView<FRHIDescriptorResource> InitialResources) override final
	{
		ValidateDescriptorRangeResources(Desc, InitialResources);
		return RHI->RHICreateDescriptorRange(RHICmdList, Desc, InitialResources);
	}
	virtual void RHIUpdateDescriptorRange(FRHICommandListBase& RHICmdList, FRHIDescriptorRange* Range, uint32 StartIndex, TConstArrayView<FRHIDescriptorResource> Resources) override final
	{
		RHI_VALIDATION_CHECK((uint64)StartIndex + Resources.Num() <= Range->GetDesc().Count,
			*FString::Printf(TEXT("UpdateDescriptorRange would write past end of range. StartIndex=%u, ResourceCount=%d, RangeCapacity=%u."),
				StartIndex, Resources.Num(), Range->GetDesc().Count));
		ValidateDescriptorRangeResources(Range->GetDesc(), Resources);
		RHI->RHIUpdateDescriptorRange(RHICmdList, Range, StartIndex, Resources);
	}
#endif

//protected:
	static void ReportValidationFailure(const TCHAR* InMessage, RHIValidation::FTracker* Tracker);
	static void ReportValidationFailure(const TCHAR* InMessage, FValidationComputeContext* Context);
	static void ReportValidationFailure(const TCHAR* InMessage, FValidationContext* Context);
	static void ReportValidationFailure(const TCHAR* InMessage, FRHICommandListBase* CmdList = nullptr);

	FDynamicRHI*				RHI;
	TMap<FRHIDepthStencilState*, FDepthStencilStateInitializerRHI> DepthStencilStates;

	std::atomic_uint64_t		RenderThreadFrameID;
	uint64						RHIThreadFrameID;

private:
	FString						RHIName;
	static TSet<uint32>			SeenFailureHashes;
	static FCriticalSection		SeenFailureHashesMutex;

	void ValidatePipeline(const FGraphicsPipelineStateInitializer& Initializer);

	// Shared validation logic, called from RHILockBuffer / RHILockBufferMGPU
	void LockBufferValidate(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, EResourceLockMode LockMode);
};

#endif	// ENABLE_RHI_VALIDATION
