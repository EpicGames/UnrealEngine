// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ValidationRHICommon.h: Public Validation RHI definitions.
=============================================================================*/

#pragma once

#include "Experimental/ConcurrentLinearAllocator.h"
#include "PixelFormat.h"
#include "RHIPipeline.h"
#include "RHIStrings.h"
#include "RHIAccess.h"
#include "RHIBreadcrumbs.h"
#include "Templates/TypeHash.h"

#if ENABLE_RHI_VALIDATION
extern RHI_API bool GRHIValidationEnabled;
#else
const bool GRHIValidationEnabled = false;
#endif

#if ENABLE_RHI_VALIDATION

class FRHIShader;
class FRHIShaderResourceView;
class FRHIUniformBuffer;
class FRHIViewableResource;
class FRHIUnorderedAccessView;
class FRHITexture;

struct FValidationCommandList;
class FValidationComputeContext;
class FValidationContext;
class FValidationRHI;

struct FRHIBufferCreateDesc;
struct FRHITextureCreateDesc;
struct FRHITransitionInfo;
struct FRHIViewDesc;

class FRayTracingPipelineStateInitializer;
class FRHIRayTracingPipelineState;
struct FRayTracingShaderBindingTableInitializer;
struct FRayTracingLocalShaderBindings;

enum class ERayTracingShaderBindingTableLifetime : uint8;
enum class ERayTracingBindingType : uint8;
enum class ERayTracingShaderBindingMode : uint8;
enum class ERayTracingHitGroupIndexingMode : uint8;

namespace RHIValidation
{
	struct FStaticUniformBuffers
	{
		TArray<FRHIUniformBuffer*> Bindings;
		bool bInSetPipelineStateCall{};

		void Reset();
		void ValidateSetShaderUniformBuffer(FRHIUniformBuffer* UniformBuffer);
	};

	struct FStageBoundUniformBuffers
	{
		FStageBoundUniformBuffers();
		void Reset();
		void Bind(uint32 Index, FRHIUniformBuffer* UniformBuffer);

		TArray<FRHIUniformBuffer*> Buffers;
	};

	struct FBoundUniformBuffers
	{
		void Reset();
		FStageBoundUniformBuffers& Get(EShaderFrequency Stage) { return StageBindings[Stage]; }

		FStageBoundUniformBuffers StageBindings[SF_NumFrequencies];
	};

	class  FTracker;
	class  FResource;
	class  FTextureResource;
	struct FOperation;
	struct FSubresourceState;
	struct FSubresourceRange;
	struct FResourceIdentity;

	enum class ELoggingMode
	{
		None,
		Manual,
		Automatic
	};

	enum class EResourcePlane
	{
		// Common plane index. Used for all resources
		Common = 0,

		// Additional plane indices for depth stencil resources
		Stencil = 1,
		Htile = 0, // @todo: do we need to track htile resources?

		// Additional plane indices for color render targets
		Cmask = 0, // @todo: do we need to track cmask resources?
		Fmask = 0, // @todo: do we need to track fmask resources?

		Max = 2
	};

	struct FSubresourceIndex
	{
		static constexpr int32 kWholeResource = -1;

		int32 MipIndex;
		int32 ArraySlice;
		int32 PlaneIndex;

		constexpr FSubresourceIndex()
			: MipIndex(kWholeResource)
			, ArraySlice(kWholeResource)
			, PlaneIndex(kWholeResource)
		{}

		constexpr FSubresourceIndex(int32 InMipIndex, int32 InArraySlice, int32 InPlaneIndex)
			: MipIndex(InMipIndex)
			, ArraySlice(InArraySlice)
			, PlaneIndex(InPlaneIndex)
		{}

		inline bool IsWholeResource() const
		{
			return MipIndex == kWholeResource
				&& ArraySlice == kWholeResource
				&& PlaneIndex == kWholeResource;
		}
	};

	struct FState
	{
		ERHIAccess Access;
		ERHIPipeline Pipelines;

		FState() = default;

		FState(ERHIAccess InAccess, ERHIPipeline InPipelines)
			: Access(InAccess)
			, Pipelines(InPipelines)
		{}

		inline bool operator == (const FState& RHS) const
		{
			return Access == RHS.Access &&
				Pipelines == RHS.Pipelines;
		}

		inline bool operator != (const FState& RHS) const
		{
			return !(*this == RHS);
		}

		inline FString ToString() const
		{
			return FString::Printf(TEXT("Access: %s, Pipelines: %s"),
				*GetRHIAccessName(Access),
				*GetRHIPipelineName(Pipelines));
		}
	};

	struct FSubresourceState
	{
		struct FPipelineState
		{
			FPipelineState()
			{
				Current.Access = ERHIAccess::Unknown;
				Current.Pipelines = ERHIPipeline::Graphics;
				Previous = Current;
			}

			FState Previous;
			FState Current;
			EResourceTransitionFlags Flags = EResourceTransitionFlags::None;

			// True when a BeginTransition has been issued, and false when the transition has been ended.
			bool bTransitioning = false;

			// True when a transition with EResourceTransitionFlags::IgnoreAfterState happened, another transition with EResourceTransitionFlags::IgnoreAfterState needs to happen before a regular one
			bool bIgnoringAfterState = false;

			// True when the resource has been used within a Begin/EndUAVOverlap region.
			bool bUsedWithAllUAVsOverlap = false;

			// True if the calling code explicitly enabled overlapping on this UAV.
			bool bExplicitAllowUAVOverlap = false;
			bool bUsedWithExplicitUAVsOverlap = false;

			// Pointer to the previous create/begin transition backtraces if logging is enabled for this resource.
			void* CreateTransitionBacktrace = nullptr;
			void* BeginTransitionBacktrace = nullptr;
		};

		TRHIPipelineArray<uint64> LastTransitionFences{InPlace, 0};
		TRHIPipelineArray<FPipelineState> States;

		void BeginTransition   (FResource* Resource, FSubresourceIndex const& SubresourceIndex, const FState& CurrentStateFromRHI, const FState& TargetState, EResourceTransitionFlags NewFlags, ERHITransitionCreateFlags CreateFlags, ERHIPipeline Pipeline, const TRHIPipelineArray<uint64>& PipelineMaxAwaitedFenceValues, void* CreateTrace);
		void EndTransition     (FResource* Resource, FSubresourceIndex const& SubresourceIndex, const FState& CurrentStateFromRHI, const FState& TargetState, EResourceTransitionFlags NewFlags, ERHIPipeline Pipeline, uint64 PipelineFenceValue, void* CreateTrace);
		void Assert            (FResource* Resource, FSubresourceIndex const& SubresourceIndex, const FState& RequiredState, bool bAllowAllUAVsOverlap);
		void AssertTracked     (FResource* Resource, FSubresourceIndex const& SubresourceIndex, const FState& RequiredState, ERHIPipeline ExecutingPipeline);
		void SpecificUAVOverlap(FResource* Resource, FSubresourceIndex const& SubresourceIndex, ERHIPipeline Pipeline, bool bAllow);
	};

	struct FSubresourceRange
	{
		uint32 MipIndex, NumMips;
		uint32 ArraySlice, NumArraySlices;
		uint32 PlaneIndex, NumPlanes;

		FSubresourceRange() = default;

		FSubresourceRange(uint32 InMipIndex, uint32 InNumMips, uint32 InArraySlice, uint32 InNumArraySlices, uint32 InPlaneIndex, uint32 InNumPlanes)
			: MipIndex(InMipIndex)
			, NumMips(InNumMips)
			, ArraySlice(InArraySlice)
			, NumArraySlices(InNumArraySlices)
			, PlaneIndex(InPlaneIndex)
			, NumPlanes(InNumPlanes)
		{}

		inline bool operator == (FSubresourceRange const& RHS) const
		{
			return MipIndex == RHS.MipIndex
				&& NumMips == RHS.NumMips
				&& ArraySlice == RHS.ArraySlice
				&& NumArraySlices == RHS.NumArraySlices
				&& PlaneIndex == RHS.PlaneIndex
				&& NumPlanes == RHS.NumPlanes;
		}

		inline bool operator != (FSubresourceRange const& RHS) const
		{
			return !(*this == RHS);
		}

		inline bool IsWholeResource(FResource& Resource) const;
	};

	inline uint32 GetTypeHash(const FSubresourceRange& Range)
	{
		uint32 Hash = HashCombineFast(Range.MipIndex, Range.NumMips);
		Hash = HashCombineFast(Hash, Range.ArraySlice);
		Hash = HashCombineFast(Hash, Range.NumArraySlices);
		Hash = HashCombineFast(Hash, Range.PlaneIndex);
		Hash = HashCombineFast(Hash, Range.NumPlanes);
		return Hash;
	}

	struct FResourceIdentity
	{
		FResource* Resource;
		FSubresourceRange SubresourceRange;

		FResourceIdentity() = default;

		inline bool operator == (FResourceIdentity const& RHS) const
		{
			return Resource == RHS.Resource
				&& SubresourceRange == RHS.SubresourceRange;
		}

		inline bool operator != (FResourceIdentity const& RHS) const
		{
			return !(*this == RHS);
		}
	};
	
	inline uint32 GetTypeHash(const FResourceIdentity& ResourceIdentiy)
	{
		uint32 Hash = PointerHash(ResourceIdentiy.Resource);
		Hash = HashCombineFast(Hash, GetTypeHash(ResourceIdentiy.SubresourceRange));
		return Hash;
	}

	struct FViewIdentity : public FResourceIdentity
	{
		uint32 Stride = 0;

		RHI_API FViewIdentity(FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc);
	};

	struct FTransientState
	{
		FTransientState() = default;

		enum class EStatus : uint8
		{
			None,
			Acquired,
			Discarded
		};

		FTransientState(ERHIAccess InitialAccess)
			: bTransient(InitialAccess == ERHIAccess::Discard)
		{}

		void* AcquireBacktrace = nullptr;
		int32 NumAcquiredSubresources = 0;

		bool bTransient = false;
		EStatus Status = EStatus::None;

		inline bool IsAcquired() const { return Status == EStatus::Acquired; }
		inline bool IsDiscarded() const { return Status == EStatus::Discarded; }

		void Acquire(FResource* Resource, void* CreateTrace, ERHIPipeline ExecutingPipeline);
		void Discard(FResource* Resource, void* CreateTrace, ERHIPipeline DiscardPipelines, ERHIPipeline ExecutingPipeline);

		static void AliasingOverlap(FResource* ResourceBefore, FResource* ResourceAfter, void* CreateTrace);
	};

	class FResource
	{
		friend FTracker;
		friend FTextureResource;
		friend FOperation;
		friend FSubresourceState;
		friend FSubresourceRange;
		friend FTransientState;
		friend FValidationRHI;

	protected:
		uint32 NumMips = 0;
		uint32 NumArraySlices = 0;
		uint32 NumPlanes = 0;
		FTransientState TransientState;
		FState TrackedState{ ERHIAccess::Unknown, ERHIPipeline::None };

	private:
		FString DebugName;

		FSubresourceState WholeResourceState;
		TArray<FSubresourceState> SubresourceStates;

		mutable FThreadSafeCounter NumOpRefs;

		inline void EnumerateSubresources(FSubresourceRange const& SubresourceRange, TFunctionRef<void(FSubresourceState&, FSubresourceIndex const&)> Callback, bool bBeginTransition = false);

	public:
		~FResource()
		{
			checkf(NumOpRefs.GetValue() == 0, TEXT("RHI validation resource '%s' is being deleted, but it is still queued in the replay command stream!"), *DebugName);
		}

		ELoggingMode LoggingMode = ELoggingMode::None;

		RHI_API void SetDebugName(const TCHAR* Name, const TCHAR* Suffix = nullptr);
		inline const TCHAR* GetDebugName() const { return DebugName.Len() ? *DebugName : nullptr; }

		inline bool IsBarrierTrackingInitialized() const { return NumMips > 0 && NumArraySlices > 0; }

		inline void AddOpRef() const
		{
			NumOpRefs.Increment();
		}

		inline void ReleaseOpRef() const
		{
			const int32 RefCount = NumOpRefs.Decrement();
			check(RefCount >= 0);
		}

		inline FState GetTrackedState() const
		{
			return TrackedState;
		}

		inline uint32 GetNumSubresources() const
		{
			return NumMips * NumArraySlices * NumPlanes;
		}

		inline FSubresourceRange GetWholeResourceRange()
		{
			checkSlow(NumMips > 0 && NumArraySlices > 0 && NumPlanes > 0);

			FSubresourceRange SubresourceRange;
			SubresourceRange.MipIndex = 0;
			SubresourceRange.ArraySlice = 0;
			SubresourceRange.PlaneIndex = 0;
			SubresourceRange.NumMips = NumMips;
			SubresourceRange.NumArraySlices = NumArraySlices;
			SubresourceRange.NumPlanes = NumPlanes;
			return SubresourceRange;
		}

		inline FResourceIdentity GetWholeResourceIdentity()
		{
			FResourceIdentity Identity;
			Identity.Resource = this;
			Identity.SubresourceRange = GetWholeResourceRange();
			return Identity;
		}

		void InitTransient(const TCHAR* InDebugName);

	protected:
		void InitBarrierTracking(int32 InNumMips, int32 InNumArraySlices, int32 InNumPlanes, ERHIAccess InResourceState, const TCHAR* InDebugName);
	};

	inline bool FSubresourceRange::IsWholeResource(FResource& Resource) const
	{
		return MipIndex == 0
			&& ArraySlice == 0
			&& PlaneIndex == 0
			&& NumMips == Resource.NumMips
			&& NumArraySlices == Resource.NumArraySlices
			&& NumPlanes == Resource.NumPlanes;
	}

	class FBufferResource : public FResource
	{
	public:
		FBufferResource() = default;
		RHI_API FBufferResource(const FRHIBufferCreateDesc& CreateDesc);

		RHI_API void InitBarrierTracking(const FRHIBufferCreateDesc& CreateDesc);
	};

	class FAccelerationStructureResource : public FBufferResource
	{
	public:
		void InitBarrierTracking(ERHIAccess InResourceState, const TCHAR* InDebugName)
		{
			FResource::InitBarrierTracking(1, 1, 1, InResourceState, InDebugName);
		}
	};
	   
	class FTextureResource
	{
	private:
		// Don't use inheritance here. Because FRHITextureReferences exist, we have to
		// call through a virtual to get the real underlying tracker resource from an FRHITexture*.
		FResource PRIVATE_TrackerResource;

		int32 GetNumPlanesFromFormat(EPixelFormat Format);

	public:
		FTextureResource() = default;
		RHI_API FTextureResource(FRHITextureCreateDesc const& CreateDesc);

		virtual ~FTextureResource() = default;

		virtual FResource* GetTrackerResource() { return &PRIVATE_TrackerResource; }

		RHI_API void InitBarrierTracking(FRHITextureCreateDesc const& CreateDesc);

		inline bool IsBarrierTrackingInitialized() const
		{
			// @todo: clean up const_cast once FRHITextureReference is removed and
			// we don't need to keep a separate PRIVATE_TrackerResource object.
			return const_cast<FTextureResource*>(this)->GetTrackerResource()->IsBarrierTrackingInitialized();
		}

		RHI_API void InitBarrierTracking  (int32 InNumMips, int32 InNumArraySlices, EPixelFormat PixelFormat, ETextureCreateFlags Flags, ERHIAccess InResourceState, const TCHAR* InDebugName);
		RHI_API void CheckValidationLayout(int32 InNumMips, int32 InNumArraySlices, EPixelFormat PixelFormat);

		RHI_API FResourceIdentity GetViewIdentity(uint32 InMipIndex, uint32 InNumMips, uint32 InArraySlice, uint32 InNumArraySlices, uint32 InPlaneIndex, uint32 InNumPlanes);
		RHI_API FResourceIdentity GetTransitionIdentity(const FRHITransitionInfo& Info);

		inline FResourceIdentity GetWholeResourceIdentity()
		{
			return GetTrackerResource()->GetWholeResourceIdentity();
		}

		inline FResourceIdentity GetWholeResourceIdentitySRV()
		{
			FResourceIdentity Identity = GetWholeResourceIdentity();

			// When binding a whole texture for shader read (SRV), we only use the first plane.
			// Other planes like stencil require a separate view to access for read in the shader.
			Identity.SubresourceRange.NumPlanes = 1;

			return Identity;
		}
	};

	class FRayTracingPipelineState
	{
	public:

		RHI_API FRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer);
		RHI_API FRHIRayTracingShader* GetShader(ERayTracingBindingType BindingType, uint32 Index) const;

	private:
		
		// Cache the RHIShaders per binding type so they can be retrieved during SetBindingsOnShaderBindingTable to find all the used resources for a certain RHIShader
		TArray<FRHIRayTracingShader*> MissShaders;
		TArray<FRHIRayTracingShader*> HitGroupShaders;
		TArray<FRHIRayTracingShader*> CallableShaders;
	};
		
	struct FUAVBinding
	{
		FRHIUnorderedAccessView* UAV = nullptr;
		uint32 Slot = 0;

		inline bool operator == (FUAVBinding const& RHS) const
		{
			return UAV == RHS.UAV
			&& Slot == RHS.Slot;
		}

		inline bool operator != (FUAVBinding const& RHS) const
		{
			return !(*this == RHS);
		}
	};
		
	inline uint32 GetTypeHash(const FUAVBinding& UAVBinding)
	{
		uint32 Hash = PointerHash(UAVBinding.UAV);
		Hash = HashCombineFast(Hash, UAVBinding.Slot);
		return Hash;
	}

	class FShaderBindingTable
	{
	public:

		RHI_API FShaderBindingTable(const FRayTracingShaderBindingTableInitializer& InInitializer);

		RHI_API void SetBindingsOnShaderBindingTable(FRayTracingPipelineState* RayTracingPipelineState, uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings, ERayTracingBindingType BindingType);
		RHI_API void Commit();

		RHI_API void ValidateStateForDispatch(RHIValidation::FTracker* Tracker) const;
		
		void AddSRV(const FResourceIdentity& ResourceIdentity, uint32 WorkerIndex)
		{
			WorkerData[WorkerIndex].SRVs.Add(ResourceIdentity);
		}

		void AddUAV(FRHIUnorderedAccessView* UAV, uint32 InSlot, uint32 WorkerIndex)
		{
			WorkerData[WorkerIndex].UAVs.Add({ UAV, InSlot });
		}

	private:
		
		ERayTracingShaderBindingTableLifetime LifeTime;
		ERayTracingShaderBindingMode ShaderBindingMode;
		ERayTracingHitGroupIndexingMode HitGroupIndexingMode;
		bool bIsDirty = true;
		
		struct FWorkerThreadData
		{
			TSet<FResourceIdentity> SRVs;
			TSet<FUAVBinding> UAVs;
		};

		static constexpr uint32 MaxBindingWorkers = 5; // RHI thread + 4 parallel workers.
		FWorkerThreadData WorkerData[MaxBindingWorkers];
	};

	struct FFence
	{
		bool bSignaled = false;
		ERHIPipeline SrcPipe = ERHIPipeline::None;
		ERHIPipeline DstPipe = ERHIPipeline::None;
		uint64 FenceValue = 0;
	};

	enum class EOpType
	{
		  BeginTransition
		, EndTransition
		, SetTrackedAccess
		, AliasingOverlap
		, AcquireTransient
		, DiscardTransient
		, InitTransient
		, Assert
		, Rename
		, Signal
		, Wait
		, AllUAVsOverlap
		, SpecificUAVOverlap
#if WITH_RHI_BREADCRUMBS
		, BeginBreadcrumbGPU
		, EndBreadcrumbGPU
		, SetBreadcrumbRange
#endif 
	};

	struct FUniformBufferResource
	{
		uint64 AllocatedFrameID = 0;
		bool bContainsNullContents = false;
		EUniformBufferUsage UniformBufferUsage;
		void* AllocatedCallstack;

		void InitLifetimeTracking(uint64 FrameID, const void* Contents, EUniformBufferUsage Usage);
		void UpdateAllocation(uint64 FrameID);
		void ValidateLifeTime();
	};

	struct FOpQueueState;

	struct FOperation
	{
		EOpType Type;

		union
		{
			struct
			{
				FResourceIdentity Identity;
				FState PreviousState;
				FState NextState;
				EResourceTransitionFlags Flags;
				ERHITransitionCreateFlags CreateFlags;
				void* CreateBacktrace;
			} Data_BeginTransition;

			struct
			{
				FResourceIdentity Identity;
				FState PreviousState;
				FState NextState;
				EResourceTransitionFlags Flags;
				void* CreateBacktrace;
			} Data_EndTransition;

			struct
			{
				FResource* Resource;
				FState State;
			} Data_SetTrackedAccess;

			struct
			{
				FResource* ResourceBefore;
				FResource* ResourceAfter;
				void* CreateBacktrace;
			} Data_AliasingOverlap;

			struct
			{
				FResource* Resource;
				void* CreateBacktrace;
			} Data_AcquireTransient;

			struct
			{
				FResource* Resource;
				TCHAR* DebugName;
			} Data_InitTransient;

			struct
			{
				FResourceIdentity Identity;
				FState RequiredState;
			} Data_Assert;

			struct
			{
				FResource* Resource;
				TCHAR* DebugName;
				const TCHAR* Suffix;
			} Data_Rename;

			struct
			{
				FFence* Fence;
			} Data_Signal;

			struct
			{
				FFence* Fence;
			} Data_Wait;

			struct
			{
				bool bAllow;
			} Data_AllUAVsOverlap;

			struct
			{
				FResourceIdentity Identity;
				bool bAllow;
			} Data_SpecificUAVOverlap;

#if WITH_RHI_BREADCRUMBS
			struct
			{
				FRHIBreadcrumbNode* Breadcrumb;
			} Data_Breadcrumb;

			struct
			{
				FRHIBreadcrumbRange Range;
			} Data_BreadcrumbRange;
#endif // WITH_RHI_BREADCRUMBS
		};

		// Returns true if the operation is complete
		RHI_API bool Replay(FOpQueueState& Queue) const;

		static inline FOperation BeginTransitionResource(FResourceIdentity Identity, FState PreviousState, FState NextState, EResourceTransitionFlags Flags, ERHITransitionCreateFlags CreateFlags, void* CreateBacktrace)
		{
			for (ERHIPipeline Pipeline : MakeFlagsRange(PreviousState.Pipelines))
			{
				Identity.Resource->AddOpRef();
			}

			FOperation Op;
			Op.Type = EOpType::BeginTransition;
			Op.Data_BeginTransition.Identity = Identity;
			Op.Data_BeginTransition.PreviousState = PreviousState;
			Op.Data_BeginTransition.NextState = NextState;
			Op.Data_BeginTransition.Flags = Flags;
			Op.Data_BeginTransition.CreateFlags = CreateFlags;
			Op.Data_BeginTransition.CreateBacktrace = CreateBacktrace;
			return MoveTemp(Op);
		}

		static inline FOperation EndTransitionResource(FResourceIdentity Identity, FState PreviousState, FState NextState, EResourceTransitionFlags Flags, void* CreateBacktrace)
		{
			for (ERHIPipeline Pipeline : MakeFlagsRange(NextState.Pipelines))
			{
				Identity.Resource->AddOpRef();
			}

			FOperation Op;
			Op.Type = EOpType::EndTransition;
			Op.Data_EndTransition.Identity = Identity;
			Op.Data_EndTransition.PreviousState = PreviousState;
			Op.Data_EndTransition.NextState = NextState;
			Op.Data_EndTransition.Flags = Flags;
			Op.Data_EndTransition.CreateBacktrace = CreateBacktrace;
			return MoveTemp(Op);
		}

		static inline FOperation SetTrackedAccess(FResource* Resource, FState State)
		{
			Resource->AddOpRef();

			FOperation Op;
			Op.Type = EOpType::SetTrackedAccess;
			Op.Data_SetTrackedAccess.Resource = Resource;
			Op.Data_SetTrackedAccess.State = State;
			return MoveTemp(Op);
		}

		static inline FOperation AliasingOverlap(FResource* ResourceBefore, FResource* ResourceAfter, void* CreateBacktrace)
		{
			ResourceBefore->AddOpRef();
			ResourceAfter->AddOpRef();

			FOperation Op;
			Op.Type = EOpType::AliasingOverlap;
			Op.Data_AliasingOverlap.ResourceBefore = ResourceBefore;
			Op.Data_AliasingOverlap.ResourceAfter = ResourceAfter;
			Op.Data_AliasingOverlap.CreateBacktrace = CreateBacktrace;
			return Op;
		}

		static inline FOperation AcquireTransientResource(FResource* Resource, void* CreateBacktrace)
		{
			Resource->AddOpRef();

			FOperation Op;
			Op.Type = EOpType::AcquireTransient;
			Op.Data_AcquireTransient.Resource = Resource;
			Op.Data_AcquireTransient.CreateBacktrace = CreateBacktrace;
			return MoveTemp(Op);
		}

		static inline FOperation InitTransient(FResource* Resource, const TCHAR* DebugName)
		{
			Resource->AddOpRef();

			FOperation Op;
			Op.Type = EOpType::InitTransient;
			Op.Data_InitTransient.Resource = Resource;
			AllocStringCopy(Op.Data_InitTransient.DebugName, DebugName);
			
			return MoveTemp(Op);
		}

		static inline FOperation Assert(FResourceIdentity Identity, FState RequiredState)
		{
			Identity.Resource->AddOpRef();

			FOperation Op;
			Op.Type = EOpType::Assert;
			Op.Data_Assert.Identity = Identity;
			Op.Data_Assert.RequiredState = RequiredState;
			return MoveTemp(Op);
		}

		static inline FOperation Rename(FResource* Resource, const TCHAR* NewName, const TCHAR* Suffix = nullptr)
		{
			Resource->AddOpRef();

			FOperation Op;
			Op.Type = EOpType::Rename;
			Op.Data_Rename.Resource = Resource;
			AllocStringCopy(Op.Data_Rename.DebugName, NewName);
			Op.Data_Rename.Suffix = Suffix;
			return MoveTemp(Op);
		}

		static inline FOperation Signal(FFence* Fence)
		{
			FOperation Op;
			Op.Type = EOpType::Signal;
			Op.Data_Signal.Fence = Fence;
			return MoveTemp(Op);
		}

		static inline FOperation Wait(FFence* Fence)
		{
			FOperation Op;
			Op.Type = EOpType::Wait;
			Op.Data_Wait.Fence = Fence;
			return MoveTemp(Op);
		}

		static inline FOperation AllUAVsOverlap(bool bAllow)
		{
			FOperation Op;
			Op.Type = EOpType::AllUAVsOverlap;
			Op.Data_AllUAVsOverlap.bAllow = bAllow;
			return MoveTemp(Op);
		}

		static inline FOperation SpecificUAVOverlap(FResourceIdentity Identity, bool bAllow)
		{
			Identity.Resource->AddOpRef();

			FOperation Op;
			Op.Type = EOpType::SpecificUAVOverlap;
			Op.Data_SpecificUAVOverlap.Identity = Identity;
			Op.Data_SpecificUAVOverlap.bAllow = bAllow;
			return MoveTemp(Op);
		}

#if WITH_RHI_BREADCRUMBS
		static inline FOperation BeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
		{
			check(Breadcrumb && Breadcrumb != FRHIBreadcrumbNode::Sentinel);

			FOperation Op;
			Op.Type = EOpType::BeginBreadcrumbGPU;
			Op.Data_Breadcrumb.Breadcrumb = Breadcrumb;
			return Op;
		}

		static inline FOperation EndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
		{
			check(Breadcrumb && Breadcrumb != FRHIBreadcrumbNode::Sentinel);

			FOperation Op;
			Op.Type = EOpType::EndBreadcrumbGPU;
			Op.Data_Breadcrumb.Breadcrumb = Breadcrumb;
			return Op;
		}

		static FOperation SetBreadcrumbRange(FRHIBreadcrumbRange const& BreadcrumbRange)
		{
			check(BreadcrumbRange.First != FRHIBreadcrumbNode::Sentinel);
			check(BreadcrumbRange.Last != FRHIBreadcrumbNode::Sentinel);

			FOperation Op;
			Op.Type = EOpType::SetBreadcrumbRange;
			Op.Data_BreadcrumbRange.Range = BreadcrumbRange;

			return Op;
		}
#endif // WITH_RHI_BREADCRUMBS

	private:
		static inline void AllocStringCopy(TCHAR*& OutString, const TCHAR* InString)
		{
			int32 Len = FCString::Strlen(InString);
			OutString = new TCHAR[Len + 1];
			FMemory::Memcpy(OutString, InString, Len * sizeof(TCHAR));
			OutString[Len] = 0;
		}
	};

	struct FTransitionResource
	{
		TRHIPipelineArray<TArray<FOperation>> PendingSignals;
		TRHIPipelineArray<TArray<FOperation>> PendingWaits;

		TArray<FOperation> PendingAliases;
		TArray<FOperation> PendingAliasingOverlaps;
		TArray<FOperation> PendingOperationsBegin;
		TArray<FOperation> PendingOperationsEnd;
	};

	enum class EUAVMode
	{
		Graphics,
		Compute,
		Num
	};

	struct FOpQueueState
	{
		ERHIPipeline const Pipeline;
		uint64 FenceValue = 0;
		TRHIPipelineArray<uint64> MaxAwaitedFenceValues{InPlace, 0};

#if WITH_RHI_BREADCRUMBS
		struct
		{
			FRHIBreadcrumbRange Range {};
			FRHIBreadcrumbNode* Current = nullptr;
		} Breadcrumbs;
#endif

		bool bAllowAllUAVsOverlap = false;

		struct FOpsList : public TArray<FOperation>
		{
			int32 ReplayPos = 0;

			FOpsList(FOpsList&&) = default;
			FOpsList(TArray<FOperation>&& Other)
				: TArray(MoveTemp(Other))
			{}
		};

		TArray<FOpsList> Ops;

		FOpQueueState(ERHIPipeline Pipeline)
			: Pipeline(Pipeline)
		{}

		void AppendOps(FValidationCommandList* CommandList);

		// Returns true if progress was made
		bool Execute();
	};

	class FTracker
	{
		struct FUAVTracker
		{
		private:
			TArray<FRHIUnorderedAccessView*> UAVs;

		public:
			FUAVTracker()
			{
				UAVs.Reserve(GRHIGlobals.MinGuaranteedSimultaneousUAVs);
			}

			inline FRHIUnorderedAccessView*& operator[](int32 Slot)
			{
				if (Slot >= UAVs.Num())
				{
					UAVs.SetNumZeroed(Slot + 1);
				}
				return UAVs[Slot];
			}

			inline void Reset()
			{
				UAVs.SetNum(0, EAllowShrinking::No);
			}

			void DrawOrDispatch(FTracker* BarrierTracker, const FState& RequiredState);
		};

	public:
		FTracker(ERHIPipeline InPipeline)
			: Pipeline(InPipeline)
		{}

		RHI_API void AddOp(const FOperation& Op);

		void AddOps(TArray<FOperation> const& List)
		{
			for (const FOperation& Op : List)
			{
				AddOp(Op);
			}
		}

		TArray<FOperation> Finalize()
		{
			return MoveTemp(CurrentList);
		}

#if WITH_RHI_BREADCRUMBS
		void BeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
		{
			AddOp(FOperation::BeginBreadcrumbGPU(Breadcrumb));
		}

		void EndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
		{
			AddOp(FOperation::EndBreadcrumbGPU(Breadcrumb));
		}
#endif

		void SetTrackedAccess(FResource* Resource, ERHIAccess Access, ERHIPipeline Pipelines)
		{
			AddOp(FOperation::SetTrackedAccess(Resource, FState(Access, Pipelines)));
		}

		void Rename(FResource* Resource, const TCHAR* NewName, const TCHAR* Suffix = nullptr)
		{
			AddOp(FOperation::Rename(Resource, NewName, Suffix));
		}

		void Assert(FResourceIdentity Identity, ERHIAccess RequiredAccess)
		{
			AddOp(FOperation::Assert(Identity, FState(RequiredAccess, Pipeline)));
		}

		void AssertUAV(FRHIUnorderedAccessView* UAV, EUAVMode Mode, int32 Slot)
		{
			checkSlow(Mode == EUAVMode::Compute || Pipeline == ERHIPipeline::Graphics);
			UAVTrackers[int32(Mode)][Slot] = UAV;
		}

		void AssertUAV(FRHIUnorderedAccessView* UAV, ERHIAccess Access, int32 Slot)
		{
			checkSlow(!(Access & ~ERHIAccess::UAVMask));
			AssertUAV(UAV, Access == ERHIAccess::UAVGraphics ? EUAVMode::Graphics : EUAVMode::Compute, Slot);
		}

		void TransitionResource(FResourceIdentity Identity, FState PreviousState, FState NextState, EResourceTransitionFlags Flags)
		{
			// This function exists due to the implicit transitions that RHI functions make (e.g. RHICopyToResolveTarget).
			// It should be removed when we eventually remove all implicit transitions from the RHI.
			AddOp(FOperation::BeginTransitionResource(Identity, PreviousState, NextState, Flags, ERHITransitionCreateFlags::None, nullptr));
			AddOp(FOperation::EndTransitionResource(Identity, PreviousState, NextState, Flags, nullptr));
		}

		void AllUAVsOverlap(bool bAllow)
		{
			AddOp(FOperation::AllUAVsOverlap(bAllow));
		}

		void SpecificUAVOverlap(FResourceIdentity Identity, bool bAllow)
		{
			AddOp(FOperation::SpecificUAVOverlap(Identity, bAllow));
		}

		void Dispatch()
		{
			UAVTrackers[int32(EUAVMode::Compute)].DrawOrDispatch(this, FState(ERHIAccess::UAVCompute, Pipeline));
		}

		void Draw()
		{
			checkSlow(Pipeline == ERHIPipeline::Graphics);
			UAVTrackers[int32(EUAVMode::Graphics)].DrawOrDispatch(this, FState(ERHIAccess::UAVGraphics, Pipeline));
		}

		void ResetUAVState(EUAVMode Mode)
		{
			UAVTrackers[int32(Mode)].Reset();
		}

		void ResetAllUAVState()
		{
			for (int32 Index = 0; Index < UE_ARRAY_COUNT(UAVTrackers); ++Index)
			{
				ResetUAVState(EUAVMode(Index));
			}
		}
		
		void SetExecutingCmdList(FRHICommandListBase* InCmdList)
		{
			ExecutingCmdList = InCmdList;
		}
		
		FRHICommandListBase* GetExecutingCmdList()
		{
			return ExecutingCmdList;
		}

		static FOpQueueState& GetQueue(ERHIPipeline Pipeline);

		static void SubmitValidationOps(ERHIPipeline Pipeline, TArray<RHIValidation::FOperation>&& Ops);

	private:
		const ERHIPipeline Pipeline;
		TArray<FOperation> CurrentList;
		FUAVTracker UAVTrackers[int32(EUAVMode::Num)];
		FRHICommandListBase* ExecutingCmdList = nullptr;
		
		friend FOperation;
		static RHI_API FOpQueueState OpQueues[int32(ERHIPipeline::Num)];
	};

	extern RHI_API void* CaptureBacktrace();

	/** Validates that the SRV is conform to what the shader expects */
	extern RHI_API void ValidateShaderResourceView(const FRHIShader* RHIShaderBase, uint32 BindIndex, const FRHIShaderResourceView* SRV);
	extern RHI_API void ValidateShaderResourceView(const FRHIShader* RHIShaderBase, uint32 BindIndex, const FRHITexture* Texture);

	/** Validates that the UAV conforms to what the shader expects */
	extern RHI_API void ValidateUnorderedAccessView(const FRHIShader* RHIShaderBase, uint32 BindIndex, const FRHIUnorderedAccessView* SRV);

	/** Validates that the UB conforms to what the shader expects */
	extern RHI_API void ValidateUniformBuffer(const FRHIShader* RHIShaderBase, uint32 BindIndex, FRHIUniformBuffer* SRV);

}

#endif // ENABLE_RHI_VALIDATION
