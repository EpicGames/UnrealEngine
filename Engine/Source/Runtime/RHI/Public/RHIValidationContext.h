// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHIValidationContext.h: Public RHI Validation Context definitions.
=============================================================================*/

#pragma once

#include "RHIValidationCommon.h"
#include "RHIValidationUtils.h"
#include "RHIValidation.h"

#if ENABLE_RHI_VALIDATION

#include "RHI.h"

class FValidationRHI;

void ValidateShaderParameters(FRHIShader* RHIShader, RHIValidation::FTracker* Tracker, RHIValidation::FStaticUniformBuffers& StaticUniformBuffers, RHIValidation::FStageBoundUniformBuffers& BoundUniformBuffers, TConstArrayView<FRHIShaderParameterResource> InParameters, ERHIAccess InRequiredAccess, RHIValidation::EUAVMode InRequiredUAVMode);

class FValidationComputeContext final : public IRHIComputeContext
{
public:
	enum EType
	{
		Default,
		Parallel
	} const Type;

	FValidationComputeContext(EType Type);

	void ValidateDispatch();

	virtual ~FValidationComputeContext() = default;

	virtual IRHIComputeContext& GetLowestLevelContext() override final
	{
		checkSlow(RHIContext);
		return *RHIContext;
	}

	virtual void SetExecutingCommandList(FRHICommandListBase* InCmdList) override final
	{
		IRHIComputeContext::SetExecutingCommandList(InCmdList);
		RHIContext->SetExecutingCommandList(InCmdList);
		
		Tracker->SetExecutingCmdList(InCmdList);
	}

	virtual void RHISetComputePipelineState(FRHIComputePipelineState* ComputePipelineState) override final
	{
		State.BoundShader = ComputePipelineState->GetComputeShader();

		// Reset the compute UAV tracker since the renderer must re-bind all resources after changing a shader.
		Tracker->ResetUAVState(RHIValidation::EUAVMode::Compute);

		State.StaticUniformBuffers.bInSetPipelineStateCall = true;
		RHIContext->RHISetComputePipelineState(ComputePipelineState);
		State.StaticUniformBuffers.bInSetPipelineStateCall = false;
	}

	virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) override final
	{
		ValidateDispatch();
		FValidationRHI::ValidateThreadGroupCount(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
		RHIContext->RHIDispatchComputeShader(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
		Tracker->Dispatch();
	}

	virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) override final
	{
		ValidateDispatch();
		FValidationRHI::ValidateDispatchIndirectArgsBuffer(ArgumentBuffer, ArgumentOffset);
		Tracker->Assert(ArgumentBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
		RHIContext->RHIDispatchIndirectComputeShader(ArgumentBuffer, ArgumentOffset);
		Tracker->Dispatch();
	}

	virtual void RHISetAsyncComputeBudget(EAsyncComputeBudget Budget) override final
	{
		RHIContext->RHISetAsyncComputeBudget(Budget);
	}

	virtual void RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions) override final
	{
		for (const FRHITransition* Transition : Transitions)
		{
			Tracker->AddOps(Transition->PendingAliases);
		}

		for (const FRHITransition* Transition : Transitions)
		{
			Tracker->AddOps(Transition->PendingOperationsBegin);
		}

		for (const FRHITransition* Transition : Transitions)
		{
			Tracker->AddOps(Transition->PendingSignals[GetPipeline()]);
		}

		RHIContext->RHIBeginTransitions(Transitions);
	}

	virtual void RHIEndTransitions(TArrayView<const FRHITransition*> Transitions) override final
	{
		for (const FRHITransition* Transition : Transitions)
		{
			Tracker->AddOps(Transition->PendingWaits[GetPipeline()]);
		}

		for (const FRHITransition* Transition : Transitions)
		{
			Tracker->AddOps(Transition->PendingOperationsEnd);
		}

		for (const FRHITransition* Transition : Transitions)
		{
			Tracker->AddOps(Transition->PendingAliasingOverlaps);
		}

		RHIContext->RHIEndTransitions(Transitions);
	}

	virtual void SetTrackedAccess(const FRHITrackedAccessInfo& Info) override final
	{
		check(Info.Resource != nullptr);
		check(Info.Access != ERHIAccess::Unknown);
		check(Info.Pipelines != ERHIPipeline::None);

		Tracker->SetTrackedAccess(Info.Resource->GetValidationTrackerResource(), Info.Access, Info.Pipelines);

		RHIContext->SetTrackedAccess(Info);
	}

	virtual void RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values) override final
	{
		Tracker->Assert(UnorderedAccessViewRHI->GetViewIdentity(), ERHIAccess::UAVCompute);
		RHIContext->RHIClearUAVFloat(UnorderedAccessViewRHI, Values);
	}

	virtual void RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values) override final
	{
		// @todo should we assert here? If the base RHI uses a compute shader via
		// FRHICommandList_RecursiveHazardous then we might double-assert which breaks the tracking
		RHIContext->RHIClearUAVUint(UnorderedAccessViewRHI, Values);
	}

	virtual void RHISetShaderRootConstants(const FUint32Vector4& Constants) final override
	{
		RHIContext->RHISetShaderRootConstants(Constants);
	}

	virtual void RHIRayTraceDispatch(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		uint32 Width, uint32 Height) override final
	{
		SBT->ValidateStateForDispatch(Tracker);
		RHIContext->RHIRayTraceDispatch(RayTracingPipelineState, RayGenShader, SBT, GlobalResourceBindings, Width, Height);
	}

	virtual void RHIRayTraceDispatchIndirect(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) override final
	{
		FValidationRHI::ValidateDispatchIndirectArgsBuffer(ArgumentBuffer, ArgumentOffset);
		Tracker->Assert(ArgumentBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
		Tracker->Assert(ArgumentBuffer->GetWholeResourceIdentity(), ERHIAccess::SRVCompute);

		SBT->ValidateStateForDispatch(Tracker);
		RHIContext->RHIRayTraceDispatchIndirect(RayTracingPipelineState, RayGenShader, SBT, GlobalResourceBindings, ArgumentBuffer, ArgumentOffset);
	}

	virtual void RHIDispatchComputeShaderBundle(
		FRHIShaderBundle* ShaderBundleRHI,
		FRHIBuffer* RecordArgBuffer,
		TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches,
		ERHIShaderBundleMethod Method) final override
	{
		if (!GRHIGlobals.ShaderBundles.RequiresSharedBindlessParameters)
		{
			RHI_VALIDATION_CONTEXT_CHECK(SharedBindlessParameters.Num() == 0, this, TEXT("SharedBindlessParameters should not be set on this platform and configuration"));
		}
		
		RHI_VALIDATION_CONTEXT_CHECK(Dispatches.Num() > 0, this, TEXT("A shader bundle must be dispatched with at least one record."));
		
		for (const FRHIShaderBundleComputeDispatch& Dispatch : Dispatches)
		{
			if (!Dispatch.IsValid())
			{
				continue;
			}

			State.BoundShader = Dispatch.Shader;

			// Reset the compute UAV tracker since the renderer must re-bind all resources after changing a shader.
			Tracker->ResetUAVState(RHIValidation::EUAVMode::Compute);

			ValidateShaderParameters(Dispatch.Shader, Tracker, State.StaticUniformBuffers, State.BoundUniformBuffers, Dispatch.Parameters->ResourceParameters, ERHIAccess::SRVCompute, RHIValidation::EUAVMode::Compute);
			ValidateShaderParameters(Dispatch.Shader, Tracker, State.StaticUniformBuffers, State.BoundUniformBuffers, Dispatch.Parameters->BindlessParameters, ERHIAccess::SRVCompute, RHIValidation::EUAVMode::Compute);

			if (Method == ERHIShaderBundleMethod::Emulation)
			{
				const uint32 ArgumentOffset = (Dispatch.RecordIndex * ShaderBundleRHI->ArgStride) + ShaderBundleRHI->ArgOffset;
				FValidationRHI::ValidateDispatchIndirectArgsBuffer(RecordArgBuffer, ArgumentOffset);
			}
		}

		Tracker->Assert(RecordArgBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);

		RHIContext->RHIDispatchComputeShaderBundle(ShaderBundleRHI, RecordArgBuffer, SharedBindlessParameters, Dispatches, Method);
	}
	
	virtual void RHIDispatchGraphicsShaderBundle(
		FRHIShaderBundle* ShaderBundleRHI,
		FRHIBuffer* RecordArgBuffer,
		const FRHIShaderBundleGraphicsState& BundleState,
		TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleGraphicsDispatch> Dispatches,
		ERHIShaderBundleMethod Method) final override
	{
		if (!GRHIGlobals.ShaderBundles.RequiresSharedBindlessParameters)
		{
			RHI_VALIDATION_CONTEXT_CHECK(SharedBindlessParameters.Num() == 0, this, TEXT("SharedBindlessParameters should not be set on this platform and configuration"));
		}

		// TODO:
#if 0
		RHI_VALIDATION_CONTEXT_CHECK(Dispatches.Num() > 0, this, TEXT("A shader bundle must be dispatched with at least one record."));
		
		for (const FRHIShaderBundleGraphicsDispatch& Dispatch : Dispatches)
		{
			if (!Dispatch.IsValid())
			{
				continue;
			}

			// Reset the graphics UAV tracker since the renderer must re-bind all resources after changing a shader.
			Tracker->ResetUAVState(RHIValidation::EUAVMode::Graphics);

			ValidateShaderParameters(Dispatch.Shader, Tracker, State.StaticUniformBuffers, Dispatch.Parameters.ResourceParameters, ERHIAccess::SRVGraphics, RHIValidation::EUAVMode::Graphics);
			ValidateShaderParameters(Dispatch.Shader, Tracker, State.StaticUniformBuffers, Dispatch.Parameters.BindlessParameters, ERHIAccess::SRVGraphics, RHIValidation::EUAVMode::Graphics);

			if (bEmulated)
			{
				const uint32 ArgumentOffset = (Dispatch.RecordIndex * ShaderBundleRHI->ArgStride) + ShaderBundleRHI->ArgOffset;
				//ValidateIndirectArgsBuffer
				//FValidationRHI::ValidateDispatchIndirectArgsBuffer(RecordArgBuffer, ArgumentOffset);
			}
		}

		Tracker->Assert(RecordArgBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
#endif
		RHIContext->RHIDispatchGraphicsShaderBundle(ShaderBundleRHI, RecordArgBuffer, BundleState, SharedBindlessParameters, Dispatches, Method);
	}

	virtual void RHIBeginUAVOverlap() final override
	{
		Tracker->AllUAVsOverlap(true);
		RHIContext->RHIBeginUAVOverlap();
	}

	virtual void RHIEndUAVOverlap() final override
	{
		Tracker->AllUAVsOverlap(false);
		RHIContext->RHIEndUAVOverlap();
	}

	virtual void RHIBeginUAVOverlap(TConstArrayView<FRHIUnorderedAccessView*> UAVs) final override
	{
		for (FRHIUnorderedAccessView* UAV : UAVs)
		{
			Tracker->SpecificUAVOverlap(UAV->GetViewIdentity(), true);
		}
		RHIContext->RHIBeginUAVOverlap(UAVs);
	}

	virtual void RHIEndUAVOverlap(TConstArrayView<FRHIUnorderedAccessView*> UAVs) final override
	{
		for (FRHIUnorderedAccessView* UAV : UAVs)
		{
			Tracker->SpecificUAVOverlap(UAV->GetViewIdentity(), false);
		}
		RHIContext->RHIEndUAVOverlap(UAVs);
	}

	virtual void RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override
	{
		if (State.BoundShader == nullptr)
		{
			RHI_VALIDATION_CONTEXT_CHECK(false, this, TEXT("A compute PSO has to be set to set resources into a shader!"));
			return;
		}

		if (Shader != State.BoundShader)
		{
			RHI_VALIDATION_CONTEXT_CHECK(false, this, *FString::Printf(TEXT("Invalid attempt to set parameters for compute shader '%s' while the currently bound shader is '%s'"), Shader->GetShaderName(), State.BoundShader->GetShaderName()));
			return;
		}

		ValidateShaderParameters(Shader, Tracker, State.StaticUniformBuffers, State.BoundUniformBuffers, InResourceParameters, ERHIAccess::SRVCompute, RHIValidation::EUAVMode::Compute);
		ValidateShaderParameters(Shader, Tracker, State.StaticUniformBuffers, State.BoundUniformBuffers, InBindlessParameters, ERHIAccess::SRVCompute, RHIValidation::EUAVMode::Compute);

		RHIContext->RHISetShaderParameters(Shader, InParametersData, InParameters, InResourceParameters, InBindlessParameters);
	}

	virtual void RHISetShaderUnbinds(FRHIComputeShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) final override
	{
		if (State.BoundShader == nullptr)
		{
			RHI_VALIDATION_CONTEXT_CHECK(false, this, TEXT("A compute PSO has to be set to set resources into a shader!"));
			return;
		}

		RHIContext->RHISetShaderUnbinds(Shader, InUnbinds);
	}

	virtual void RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers) override final
	{
		InUniformBuffers.Bind(State.StaticUniformBuffers.Bindings);
		RHIContext->RHISetStaticUniformBuffers(InUniformBuffers);
	}

	virtual void RHISetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* UniformBuffer) override final
	{
		RHIContext->RHISetStaticUniformBuffer(Slot, UniformBuffer);
	}

#if WITH_RHI_BREADCRUMBS
	virtual void RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override
	{
		Tracker->BeginBreadcrumbGPU(Breadcrumb);
		RHIContext->RHIBeginBreadcrumbGPU(Breadcrumb);
	}
	virtual void RHIEndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override
	{
		Tracker->EndBreadcrumbGPU(Breadcrumb);
		RHIContext->RHIEndBreadcrumbGPU(Breadcrumb);
	}
#endif // WITH_RHI_BREADCRUMBS

	virtual void RHIWriteGPUFence(FRHIGPUFence* FenceRHI) override final
	{
		RHIContext->RHIWriteGPUFence(FenceRHI);
	}

	virtual void RHISetGPUMask(FRHIGPUMask GPUMask) override final
	{
		RHIContext->RHISetGPUMask(GPUMask);
	}

	virtual FRHIGPUMask RHIGetGPUMask() const override final
	{
		return RHIContext->RHIGetGPUMask();
	}

	virtual void RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* DestinationStagingBufferRHI, uint32 InOffset, uint32 InNumBytes) override final;

#if WITH_MGPU
	virtual void RHITransferResources(TConstArrayView<FTransferResourceParams> Params)
	{
		RHIContext->RHITransferResources(Params);
	}

	virtual void RHITransferResourceSignal(TConstArrayView<FTransferResourceFenceData*> FenceDatas, FRHIGPUMask SrcGPUMask)
	{
		RHIContext->RHITransferResourceSignal(FenceDatas, SrcGPUMask);
	}

	virtual void RHITransferResourceWait(TConstArrayView<FTransferResourceFenceData*> FenceDatas)
	{
		RHIContext->RHITransferResourceWait(FenceDatas);
	}

	virtual void RHICrossGPUTransfer(TConstArrayView<FTransferResourceParams> Params, TConstArrayView<FCrossGPUTransferFence*> PreTransfer, TConstArrayView<FCrossGPUTransferFence*> PostTransfer)
	{
		RHIContext->RHICrossGPUTransfer(Params, PreTransfer, PostTransfer);
	}

	virtual void RHICrossGPUTransferSignal(TConstArrayView<FTransferResourceParams> Params, TConstArrayView<FCrossGPUTransferFence*> PreTransfer)
	{
		RHIContext->RHICrossGPUTransferSignal(Params, PreTransfer);
	}

	virtual void RHICrossGPUTransferWait(TConstArrayView<FCrossGPUTransferFence*> SyncPoints)
	{
		RHIContext->RHICrossGPUTransferWait(SyncPoints);
	}
#endif // WITH_MGPU

	virtual void RHIExecuteMultiIndirectClusterOperation(const FRayTracingClusterOperationParams& Params) override final
	{
		RHIContext->RHIExecuteMultiIndirectClusterOperation(Params);
	}

	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange) override final
	{
		// #hwrt_todo: explicit transitions and state validation for BLAS
		for (const FRayTracingGeometryBuildParams& P : Params)
		{
			const FRayTracingGeometryInitializer& Initializer = P.Geometry->GetInitializer();

			if (Initializer.IndexBuffer)
			{
				Tracker->Assert(Initializer.IndexBuffer->GetWholeResourceIdentity(), ERHIAccess::SRVCompute);
			}

			for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
			{
				const FBufferRHIRef& RHIVertexBuffer = Segment.VertexBuffer;
				Tracker->Assert(Segment.VertexBuffer->GetWholeResourceIdentity(), ERHIAccess::SRVCompute);
			}
		}

		RHIContext->RHIBuildAccelerationStructures(Params, ScratchBufferRange);
	}

	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params) override final
	{
		// #hwrt_todo: validate all referenced BLAS states
		for (const FRayTracingSceneBuildParams& P : Params)
		{
			if (P.Scene)
			{
				Tracker->Assert(P.Scene->GetWholeResourceIdentity(), ERHIAccess::BVHWrite);
			}

			if (P.InstanceBuffer)
			{
				Tracker->Assert(P.InstanceBuffer->GetWholeResourceIdentity(), ERHIAccess::SRVCompute);
			}

			if (P.ScratchBuffer)
			{
				Tracker->Assert(P.ScratchBuffer->GetWholeResourceIdentity(), ERHIAccess::UAVCompute);
			}
		}
		
		RHIContext->RHIBuildAccelerationStructures(Params);
	}

	virtual void RHIBindAccelerationStructureMemory(FRHIRayTracingScene* Scene, FRHIBuffer* Buffer, uint32 BufferOffset) override final
	{
		RHIContext->RHIBindAccelerationStructureMemory(Scene, Buffer, BufferOffset);
	}

	inline void LinkToContext(IRHIComputeContext* PlatformContext)
	{
		RHIContext = PlatformContext;
		PlatformContext->WrappingContext = this;
		PlatformContext->Tracker = &State.TrackerInstance;
	}

	IRHIComputeContext* RHIContext = nullptr;

protected:
	struct FState
	{
		RHIValidation::FTracker TrackerInstance{ ERHIPipeline::AsyncCompute };
		RHIValidation::FStaticUniformBuffers StaticUniformBuffers;
		RHIValidation::FStageBoundUniformBuffers BoundUniformBuffers;

		FString ComputePassName;
		FRHIComputeShader* BoundShader = nullptr;

		void Reset();
	} State;

	friend class FValidationRHI;
};

class FValidationContext final : public IRHICommandContext
{
public:
	enum EType
	{
		Default,
		Parallel
	} const Type;

	FValidationContext(EType InType);

	virtual IRHIComputeContext& GetLowestLevelContext() override final
	{
		checkSlow(RHIContext);
		return *RHIContext;
	}

	virtual void SetExecutingCommandList(FRHICommandListBase* InCmdList) override final
	{
		IRHICommandContext::SetExecutingCommandList(InCmdList);
		RHIContext->SetExecutingCommandList(InCmdList);
		
		Tracker->SetExecutingCmdList(InCmdList);
	}

	virtual void RHISetComputePipelineState(FRHIComputePipelineState* ComputePipelineState) override final
	{
		State.bGfxPSOSet = false;

		FMemory::Memset(State.BoundShaders, 0);
		State.BoundShaders[SF_Compute] = ComputePipelineState->GetComputeShader();

		// Reset the compute UAV tracker since the renderer must re-bind all resources after changing a shader.
		Tracker->ResetUAVState(RHIValidation::EUAVMode::Compute);

		State.StaticUniformBuffers.bInSetPipelineStateCall = true;
		RHIContext->RHISetComputePipelineState(ComputePipelineState);
		State.StaticUniformBuffers.bInSetPipelineStateCall = false;
	}

	virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) override final
	{
		ValidateDispatch();
		FValidationRHI::ValidateThreadGroupCount(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
		RHIContext->RHIDispatchComputeShader(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
		Tracker->Dispatch();
	}

	virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) override final
	{
		ValidateDispatch();
		FValidationRHI::ValidateDispatchIndirectArgsBuffer(ArgumentBuffer, ArgumentOffset);
		Tracker->Assert(ArgumentBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
		RHIContext->RHIDispatchIndirectComputeShader(ArgumentBuffer, ArgumentOffset);
		Tracker->Dispatch();
	}

	virtual void RHISetAsyncComputeBudget(EAsyncComputeBudget Budget) override final
	{
		RHIContext->RHISetAsyncComputeBudget(Budget);
	}

	// Useful when used with geometry shader (emit polygons to different viewports), otherwise SetViewPort() is simpler
	// @param Count >0
	// @param Data must not be 0
	virtual void RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data) override final
	{
		RHIContext->RHISetMultipleViewports(Count, Data);
	}

	virtual void RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values) final override
	{
		// @todo should we assert here? If the base RHI uses a compute shader via
		// FRHICommandList_RecursiveHazardous then we might double-assert which breaks the tracking
		RHIContext->RHIClearUAVFloat(UnorderedAccessViewRHI, Values);
	}

	virtual void RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values) final override
	{
		// @todo should we assert here? If the base RHI uses a compute shader via
		// FRHICommandList_RecursiveHazardous then we might double-assert which breaks the tracking
		RHIContext->RHIClearUAVUint(UnorderedAccessViewRHI, Values);
	}

	virtual void RHISetShaderRootConstants(const FUint32Vector4& Constants) final override
	{
		RHIContext->RHISetShaderRootConstants(Constants);
	}

	virtual void RHIDispatchComputeShaderBundle(
		FRHIShaderBundle* ShaderBundleRHI,
		FRHIBuffer* RecordArgBuffer,
		TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches,
		ERHIShaderBundleMethod Method) final override
	{
		if (!GRHIGlobals.ShaderBundles.RequiresSharedBindlessParameters)
		{
			RHI_VALIDATION_CONTEXT_CHECK(SharedBindlessParameters.Num() == 0, this, TEXT("SharedBindlessParameters should not be set on this platform and configuration"));
		}

		RHI_VALIDATION_CONTEXT_CHECK(Dispatches.Num() > 0, this, TEXT("A shader bundle must be dispatched with at least one record."));
		
		RHIValidation::FStageBoundUniformBuffers& BoundUniformBuffers = State.BoundUniformBuffers.Get(SF_Compute);

		for (const FRHIShaderBundleComputeDispatch& Dispatch : Dispatches)
		{
			if (!Dispatch.IsValid())
			{
				continue;
			}

			State.BoundShaders[SF_Compute] = Dispatch.Shader;
			
			// Reset the compute UAV tracker since the renderer must re-bind all resources after changing a shader.
			Tracker->ResetUAVState(RHIValidation::EUAVMode::Compute);

			ValidateShaderParameters(Dispatch.Shader, Tracker, State.StaticUniformBuffers, BoundUniformBuffers, Dispatch.Parameters->ResourceParameters, ERHIAccess::SRVCompute, RHIValidation::EUAVMode::Compute);
			ValidateShaderParameters(Dispatch.Shader, Tracker, State.StaticUniformBuffers, BoundUniformBuffers, Dispatch.Parameters->BindlessParameters, ERHIAccess::SRVCompute, RHIValidation::EUAVMode::Compute);

			if (Method == ERHIShaderBundleMethod::Emulation)
			{
				const uint32 ArgumentOffset = (Dispatch.RecordIndex * ShaderBundleRHI->ArgStride) + ShaderBundleRHI->ArgOffset;
				FValidationRHI::ValidateDispatchIndirectArgsBuffer(RecordArgBuffer, ArgumentOffset);
			}
		}

		Tracker->Assert(RecordArgBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);

		RHIContext->RHIDispatchComputeShaderBundle(ShaderBundleRHI, RecordArgBuffer, SharedBindlessParameters, Dispatches, Method);
	}

	virtual void RHIDispatchGraphicsShaderBundle(
		FRHIShaderBundle* ShaderBundleRHI,
		FRHIBuffer* RecordArgBuffer,
		const FRHIShaderBundleGraphicsState& BundleState,
		TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleGraphicsDispatch> Dispatches,
		ERHIShaderBundleMethod Method) final override
	{
		if (!GRHIGlobals.ShaderBundles.RequiresSharedBindlessParameters)
		{
			RHI_VALIDATION_CONTEXT_CHECK(SharedBindlessParameters.Num() == 0, this, TEXT("SharedBindlessParameters should not be set on this platform and configuration"));
		}

		// TODO
#if 0
		RHI_VALIDATION_CONTEXT_CHECK(Dispatches.Num() > 0, this, TEXT("SharedBindlessParameters should not be set on this platform and configuration"));

		for (const FRHIShaderBundleGraphicsDispatch& Dispatch : Dispatches)
		{
			if (!Dispatch.IsValid())
			{
				continue;
			}

			//State.bComputePSOSet = true;

			// Reset the compute UAV tracker since the renderer must re-bind all resources after changing a shader.
			Tracker->ResetUAVState(RHIValidation::EUAVMode::Graphics);

			ValidateShaderParameters(Dispatch.Shader, Tracker, State.StaticUniformBuffers, Dispatch.Parameters.ResourceParameters, ERHIAccess::SRVGraphics, RHIValidation::EUAVMode::Graphics);
			ValidateShaderParameters(Dispatch.Shader, Tracker, State.StaticUniformBuffers, Dispatch.Parameters.BindlessParameters, ERHIAccess::SRVGraphics, RHIValidation::EUAVMode::Graphics);

			if (bEmulated)
			{
				const uint32 ArgumentOffset = (Dispatch.RecordIndex * ShaderBundleRHI->ArgStride) + ShaderBundleRHI->ArgOffset;
				//FValidationRHI::ValidateDispatchIndirectArgsBuffer(RecordArgBuffer, ArgumentOffset);
			}
		}

		Tracker->Assert(RecordArgBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
#endif
		RHIContext->RHIDispatchGraphicsShaderBundle(ShaderBundleRHI, RecordArgBuffer, BundleState, SharedBindlessParameters, Dispatches, Method);
	}

	virtual void RHIBeginUAVOverlap() final override
	{
		Tracker->AllUAVsOverlap(true);
		RHIContext->RHIBeginUAVOverlap();
	}

	virtual void RHIEndUAVOverlap() final override
	{
		Tracker->AllUAVsOverlap(false);
		RHIContext->RHIEndUAVOverlap();
	}

	virtual void RHIBeginUAVOverlap(TConstArrayView<FRHIUnorderedAccessView*> UAVs) final override
	{
		for (FRHIUnorderedAccessView* UAV : UAVs)
		{
			Tracker->SpecificUAVOverlap(UAV->GetViewIdentity(), true);
		}
		RHIContext->RHIBeginUAVOverlap(UAVs);
	}

	virtual void RHIEndUAVOverlap(TConstArrayView<FRHIUnorderedAccessView*> UAVs) final override
	{
		for (FRHIUnorderedAccessView* UAV : UAVs)
		{
			Tracker->SpecificUAVOverlap(UAV->GetViewIdentity(), false);
		}
		RHIContext->RHIEndUAVOverlap(UAVs);
	}

	virtual void RHIResummarizeHTile(FRHITexture* DepthTexture) override final
	{
		Tracker->Assert(DepthTexture->GetWholeResourceIdentity(), ERHIAccess::DSVWrite);
		RHIContext->RHIResummarizeHTile(DepthTexture);
	}

	virtual void* RHIGetNativeCommandBuffer() override final
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RHIContext->RHIGetNativeCommandBuffer();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	virtual void RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions) override final
	{
		for (const FRHITransition* Transition : Transitions)
		{
			if (Transition->AllowInRenderingPass() == false)
			{
				ensureMsgf(!State.bInsideBeginRenderPass, TEXT("Ending a transition within a renderpass is not supported!"));
			}

			Tracker->AddOps(Transition->PendingAliases);
		}

		for (const FRHITransition* Transition : Transitions)
		{
			Tracker->AddOps(Transition->PendingOperationsBegin);
		}

		for (const FRHITransition* Transition : Transitions)
		{
			Tracker->AddOps(Transition->PendingSignals[GetPipeline()]);
		}

		RHIContext->RHIBeginTransitions(Transitions);
	}

	virtual void RHIEndTransitions(TArrayView<const FRHITransition*> Transitions) override final
	{
		for (const FRHITransition* Transition : Transitions)
		{
			if (Transition->AllowInRenderingPass() == false)
			{
				ensureMsgf(!State.bInsideBeginRenderPass, TEXT("Ending a transition within a renderpass is not supported!"));
			}

			Tracker->AddOps(Transition->PendingWaits[GetPipeline()]);
		}

		for (const FRHITransition* Transition : Transitions)
		{
			Tracker->AddOps(Transition->PendingOperationsEnd);
		}

		for (const FRHITransition* Transition : Transitions)
		{
			Tracker->AddOps(Transition->PendingAliasingOverlaps);
		}

		RHIContext->RHIEndTransitions(Transitions);
	}

	virtual void SetTrackedAccess(const FRHITrackedAccessInfo& Info) override final
	{
		check(Info.Resource != nullptr);
		check(Info.Access != ERHIAccess::Unknown);

		Tracker->SetTrackedAccess(Info.Resource->GetValidationTrackerResource(), Info.Access, Info.Pipelines);

		RHIContext->SetTrackedAccess(Info);
	}

	virtual void RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery) override final
	{
		RHIContext->RHIBeginRenderQuery(RenderQuery);
	}

	virtual void RHIEndRenderQuery(FRHIRenderQuery* RenderQuery) override final
	{
		RHIContext->RHIEndRenderQuery(RenderQuery);
	}

	virtual void RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBuffer, uint32 Offset) override final
	{
		//#todo-rco: Decide if this is needed or not...
		//checkf(State.bGfxPSOSet, TEXT("A Graphics PSO has to be set to set-up the vertex streams!"));

		// @todo: do we really need to allow setting nullptr stream sources anymore?
		if (VertexBuffer)
		{
		checkf(State.bInsideBeginRenderPass, TEXT("A RenderPass has to be set to set-up the vertex streams!"));
			Tracker->Assert(VertexBuffer->GetWholeResourceIdentity(), ERHIAccess::VertexOrIndexBuffer);
		}

		RHIContext->RHISetStreamSource(StreamIndex, VertexBuffer, Offset);
	}

	// @param MinX including like Win32 RECT
	// @param MinY including like Win32 RECT
	// @param MaxX excluding like Win32 RECT
	// @param MaxY excluding like Win32 RECT
	virtual void RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ) override final
	{
		RHIContext->RHISetViewport(MinX, MinY, MinZ, MaxX, MaxY, MaxZ);
	}

	virtual void RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ) override final
	{
		RHIContext->RHISetStereoViewport(LeftMinX, RightMinX, LeftMinY, RightMinY, MinZ, LeftMaxX, RightMaxX, LeftMaxY, RightMaxY, MaxZ);
	}

	// @param MinX including like Win32 RECT
	// @param MinY including like Win32 RECT
	// @param MaxX excluding like Win32 RECT
	// @param MaxY excluding like Win32 RECT
	virtual void RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY) override final
	{
		RHIContext->RHISetScissorRect(bEnable, MinX, MinY, MaxX, MaxY);
	}

	virtual void RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState) override final
	{
		checkf(State.bInsideBeginRenderPass, TEXT("Graphics PSOs can only be set inside a RenderPass!"));
		State.bGfxPSOSet = true;

		for (int32 FrequencyIndex = 0; FrequencyIndex < SF_NumFrequencies; ++FrequencyIndex)
		{
			EShaderFrequency Frequency = (EShaderFrequency)FrequencyIndex;
			State.BoundShaders[FrequencyIndex] = IsValidGraphicsFrequency(Frequency) ? GraphicsState->GetShader(Frequency) : nullptr;
		}

		ValidateDepthStencilForSetGraphicsPipelineState(GraphicsState->DSMode);

		// Setting a new PSO unbinds all previous bound resources
		Tracker->ResetUAVState(RHIValidation::EUAVMode::Graphics);

		State.StaticUniformBuffers.bInSetPipelineStateCall = true;
		RHIContext->RHISetGraphicsPipelineState(GraphicsState, StencilRef, bApplyAdditionalState);
		State.StaticUniformBuffers.bInSetPipelineStateCall = false;
	}

#if PLATFORM_USE_FALLBACK_PSO
	virtual void RHISetGraphicsPipelineState(const FGraphicsPipelineStateInitializer& PsoInit, uint32 StencilRef, bool bApplyAdditionalState) override final
	{
		checkf(State.bInsideBeginRenderPass, TEXT("Graphics PSOs can only be set inside a RenderPass!"));
		State.bGfxPSOSet = true;

		FMemory::Memset(State.BoundShaders, 0);
		State.BoundShaders[SF_Vertex] = PsoInit.BoundShaderState.GetVertexShader();
		State.BoundShaders[SF_Pixel] = PsoInit.BoundShaderState.GetPixelShader();
		State.BoundShaders[SF_Geometry] = PsoInit.BoundShaderState.GetGeometryShader();
		State.BoundShaders[SF_Amplification] = PsoInit.BoundShaderState.GetAmplificationShader();
		State.BoundShaders[SF_Mesh] = PsoInit.BoundShaderState.GetMeshShader();

		ValidateDepthStencilForSetGraphicsPipelineState(PsoInit.DepthStencilState->ActualDSMode);

		// Setting a new PSO unbinds all previous bound resources
		Tracker->ResetUAVState(RHIValidation::EUAVMode::Graphics);

		State.StaticUniformBuffers.bInSetPipelineStateCall = true;
		RHIContext->RHISetGraphicsPipelineState(PsoInit, StencilRef, bApplyAdditionalState);
		State.StaticUniformBuffers.bInSetPipelineStateCall = false;
	}
#endif

	virtual void RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override
	{
		if (!State.bGfxPSOSet)
		{
			RHI_VALIDATION_CONTEXT_CHECK(false, this, TEXT("A graphics PSO has to be set to set resources into a shader!"));
			return;
		}

		RHIValidation::FStageBoundUniformBuffers& BoundUniformBuffers = State.BoundUniformBuffers.Get(Shader->GetFrequency());

		ERHIAccess RequiredAccess = Shader->GetFrequency() == SF_Pixel ? ERHIAccess::SRVGraphicsPixel : ERHIAccess::SRVGraphicsNonPixel;

		ValidateShaderParameters(Shader, Tracker, State.StaticUniformBuffers, BoundUniformBuffers, InResourceParameters, RequiredAccess, RHIValidation::EUAVMode::Graphics);
		ValidateShaderParameters(Shader, Tracker, State.StaticUniformBuffers, BoundUniformBuffers, InBindlessParameters, RequiredAccess, RHIValidation::EUAVMode::Graphics);

		RHIContext->RHISetShaderParameters(Shader, InParametersData, InParameters, InResourceParameters, InBindlessParameters);
	}

	virtual void RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override
	{
		if (State.BoundShaders[SF_Compute] == nullptr)
		{
			RHI_VALIDATION_CONTEXT_CHECK(false, this, TEXT("A compute PSO has to be set to set resources into a shader!"));
			return;
		}

		if (Shader != State.BoundShaders[SF_Compute])
		{
			RHI_VALIDATION_CONTEXT_CHECK(false, this, *FString::Printf(TEXT("Invalid attempt to set parameters for compute shader '%s' while the currently bound shader is '%s'"), Shader->GetShaderName(), State.BoundShaders[SF_Compute]->GetShaderName()));
			return;
		}

		RHIValidation::FStageBoundUniformBuffers& BoundUniformBuffers = State.BoundUniformBuffers.Get(SF_Compute);

		ValidateShaderParameters(Shader, Tracker, State.StaticUniformBuffers, BoundUniformBuffers, InResourceParameters, ERHIAccess::SRVCompute, RHIValidation::EUAVMode::Compute);
		ValidateShaderParameters(Shader, Tracker, State.StaticUniformBuffers, BoundUniformBuffers, InBindlessParameters, ERHIAccess::SRVCompute, RHIValidation::EUAVMode::Compute);

		RHIContext->RHISetShaderParameters(Shader, InParametersData, InParameters, InResourceParameters, InBindlessParameters);
	}

	virtual void RHISetShaderUnbinds(FRHIGraphicsShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) override final
	{
		if (!State.bGfxPSOSet)
		{
			RHI_VALIDATION_CONTEXT_CHECK(false, this, TEXT("A graphics PSO has to be set to set resources into a shader!"));
			return;
		}

		RHIContext->RHISetShaderUnbinds(Shader, InUnbinds);
	}

	virtual void RHISetShaderUnbinds(FRHIComputeShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) override final
	{
		if (State.BoundShaders[SF_Compute] == nullptr)
		{
			RHI_VALIDATION_CONTEXT_CHECK(false, this, TEXT("A compute PSO has to be set to set resources into a shader!"));
			return;
		}

		RHIContext->RHISetShaderUnbinds(Shader, InUnbinds);
	}

	virtual void RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers) override final
	{
		InUniformBuffers.Bind(State.StaticUniformBuffers.Bindings);
		RHIContext->RHISetStaticUniformBuffers(InUniformBuffers);
	}

	virtual void RHISetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* UniformBuffer) override final
	{
		RHIContext->RHISetStaticUniformBuffer(Slot, UniformBuffer);
	}

	virtual void RHISetUniformBufferDynamicOffset(FUniformBufferStaticSlot Slot, uint32 Offset) override final
	{
		RHIContext->RHISetUniformBufferDynamicOffset(Slot, Offset);
	}

	virtual void RHISetStencilRef(uint32 StencilRef) override final
	{
		//checkf(State.bGfxPSOSet, TEXT("A Graphics PSO has to be set to change stencil ref!"));
		RHIContext->RHISetStencilRef(StencilRef);
	}

	virtual void RHISetBlendFactor(const FLinearColor& BlendFactor) override final
	{
		//checkf(State.bGfxPSOSet, TEXT("A Graphics PSO has to be set to change blend factor!"));
		RHIContext->RHISetBlendFactor(BlendFactor);
	}

	virtual void RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances) override final
	{
		ValidateDrawing();
		RHIContext->RHIDrawPrimitive(BaseVertexIndex, NumPrimitives, NumInstances);
		Tracker->Draw();
	}

	virtual void RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) override final
	{
		ValidateDrawing();
		FValidationRHI::ValidateIndirectArgsBuffer(ArgumentBuffer, ArgumentOffset, sizeof(FRHIDrawIndirectParameters), 0);
		Tracker->Assert(ArgumentBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
		RHIContext->RHIDrawPrimitiveIndirect(ArgumentBuffer, ArgumentOffset);
		Tracker->Draw();
	}

	// @param NumPrimitives need to be >0 
	virtual void RHIDrawIndexedPrimitive(FRHIBuffer* IndexBuffer, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances) override final
	{
		ValidateDrawing();
		checkf(EnumHasAnyFlags(IndexBuffer->GetUsage(), EBufferUsageFlags::IndexBuffer), TEXT("The buffer '%s' is used as an index buffer, but was not created with the IndexBuffer flag."), *IndexBuffer->GetName().ToString());
		Tracker->Assert(IndexBuffer->GetWholeResourceIdentity(), ERHIAccess::VertexOrIndexBuffer);
		RHIContext->RHIDrawIndexedPrimitive(IndexBuffer, BaseVertexIndex, FirstInstance, NumVertices, StartIndex, NumPrimitives, NumInstances);
		Tracker->Draw();
	}

	virtual void RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) override final
	{
		ValidateDrawing();
		checkf(EnumHasAnyFlags(IndexBuffer->GetUsage(), EBufferUsageFlags::IndexBuffer), TEXT("The buffer '%s' is used as an index buffer, but was not created with the IndexBuffer flag."), *IndexBuffer->GetName().ToString());
		FValidationRHI::ValidateIndirectArgsBuffer(ArgumentBuffer, ArgumentOffset, sizeof(FRHIDrawIndexedIndirectParameters), 0);
		Tracker->Assert(ArgumentBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
		Tracker->Assert(IndexBuffer->GetWholeResourceIdentity(), ERHIAccess::VertexOrIndexBuffer);
		RHIContext->RHIDrawIndexedPrimitiveIndirect(IndexBuffer, ArgumentBuffer, ArgumentOffset);
		Tracker->Draw();
	}

	virtual void RHIMultiDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset, FRHIBuffer* CountBuffer, uint32 CountBufferOffset, uint32 MaxDrawArguments) override final
	{
		ValidateDrawing();
		checkf(EnumHasAnyFlags(IndexBuffer->GetUsage(), EBufferUsageFlags::IndexBuffer), TEXT("The buffer '%s' is used as an index buffer, but was not created with the IndexBuffer flag."), *IndexBuffer->GetName().ToString());
		FValidationRHI::ValidateIndirectArgsBuffer(ArgumentBuffer, ArgumentOffset, sizeof(FRHIDrawIndexedIndirectParameters), 0);
		Tracker->Assert(ArgumentBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
		if (CountBuffer)
		{
			Tracker->Assert(CountBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
		}
		Tracker->Assert(IndexBuffer->GetWholeResourceIdentity(), ERHIAccess::VertexOrIndexBuffer);
		RHIContext->RHIMultiDrawIndexedPrimitiveIndirect(IndexBuffer, ArgumentBuffer, ArgumentOffset, CountBuffer, CountBufferOffset, MaxDrawArguments);
		Tracker->Draw();
	}

	virtual void RHIDispatchMeshShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override
	{
		ValidateDrawing();
		FValidationRHI::ValidateThreadGroupCount(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
		RHIContext->RHIDispatchMeshShader(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
		Tracker->Draw();
	}

	virtual void RHIDispatchIndirectMeshShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override
	{
		ValidateDrawing();
		FValidationRHI::ValidateDispatchIndirectArgsBuffer(ArgumentBuffer, ArgumentOffset);
		Tracker->Assert(ArgumentBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
		RHIContext->RHIDispatchIndirectMeshShader(ArgumentBuffer, ArgumentOffset);
		Tracker->Draw();
	}

	/**
	* Sets Depth Bounds range with the given min/max depth.
	* @param MinDepth	The minimum depth for depth bounds test
	* @param MaxDepth	The maximum depth for depth bounds test.
	*					The valid values for fMinDepth and fMaxDepth are such that 0 <= fMinDepth <= fMaxDepth <= 1
	*/
	virtual void RHISetDepthBounds(float MinDepth, float MaxDepth) override final
	{
		checkf(MinDepth >= 0.f && MinDepth <= 1.f, TEXT("Depth bounds min of %f is outside allowed range of [0, 1]"), MinDepth);
		checkf(MaxDepth >= 0.f && MaxDepth <= 1.f, TEXT("Depth bounds max of %f is outside allowed range of [0, 1]"), MaxDepth);
		RHIContext->RHISetDepthBounds(MinDepth, MaxDepth);
	}

	virtual void RHISetShadingRate(EVRSShadingRate ShadingRate, EVRSRateCombiner Combiner) override final
	{
		RHIContext->RHISetShadingRate(ShadingRate, Combiner);
	}

#if WITH_RHI_BREADCRUMBS
	virtual void RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) override final
	{
		Tracker->BeginBreadcrumbGPU(Breadcrumb);
		RHIContext->RHIBeginBreadcrumbGPU(Breadcrumb);
	}
	virtual void RHIEndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) override final
	{
		Tracker->EndBreadcrumbGPU(Breadcrumb);
		RHIContext->RHIEndBreadcrumbGPU(Breadcrumb);
	}
#endif // WITH_RHI_BREADCRUMBS

	void ValidateBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName)
	{
		checkf(!State.bInsideBeginRenderPass, TEXT("Trying to begin RenderPass '%s', but already inside '%s'!"), *State.RenderPassName, InName);
		checkf(InName != nullptr, TEXT("RenderPass should have a name!"));

		State.bInsideBeginRenderPass = true;
		State.RenderPassInfo = InInfo;
		State.RenderPassName = InName;

		FIntVector ViewDimensions = FIntVector(0);

		// assert that render targets are writable
		for (int32 RTVIndex = 0; RTVIndex < MaxSimultaneousRenderTargets; ++RTVIndex)
		{
			FRHIRenderPassInfo::FColorEntry& RTV = State.RenderPassInfo.ColorRenderTargets[RTVIndex];
			if (RTV.RenderTarget == nullptr)
			{
				checkf(RTV.ResolveTarget == nullptr, TEXT("Render target is null, but resolve target is not."));
				continue;
			}

			// Check all bound textures have the same dimensions
			FIntVector MipDimensions = RTV.RenderTarget->GetMipDimensions(RTV.MipIndex);
			checkf(ViewDimensions.IsZero() || ViewDimensions == MipDimensions, TEXT("Render target size mismatch (RT%d: %dx%d vs. Expected: %dx%d). All render and depth target views must have the same effective dimensions."), RTVIndex, MipDimensions.X, MipDimensions.Y, ViewDimensions.X, ViewDimensions.Y);
			ViewDimensions = MipDimensions;

			uint32 ArraySlice = RTV.ArraySlice;
			uint32 NumArraySlices = 1;
			if (RTV.ArraySlice < 0)
			{
				ArraySlice = 0;
				NumArraySlices = 0;
			}

			Tracker->Assert(RTV.RenderTarget->GetViewIdentity(RTV.MipIndex, 1, ArraySlice, NumArraySlices, 0, 0), ERHIAccess::RTV);

			if (RTV.ResolveTarget)
			{
				const FRHITextureDesc& RenderTargetDesc = RTV.RenderTarget->GetDesc();
				const FRHITextureDesc& ResolveTargetDesc = RTV.ResolveTarget->GetDesc();

				checkf(RenderTargetDesc.Extent == ResolveTargetDesc.Extent, TEXT("Render target extent must match resolve target extent."));
				checkf(RenderTargetDesc.Format == ResolveTargetDesc.Format, TEXT("Render target format must match resolve target format."));

				Tracker->Assert(RTV.ResolveTarget->GetViewIdentity(RTV.MipIndex, 1, ArraySlice, NumArraySlices, 0, 0), ERHIAccess::ResolveDst);
			}
		}

		FRHIRenderPassInfo::FDepthStencilEntry& DSV = State.RenderPassInfo.DepthStencilRenderTarget;

		if (DSV.DepthStencilTarget)
		{
			// Check all bound textures have the same dimensions
			FIntVector MipDimensions = DSV.DepthStencilTarget->GetMipDimensions(0);
			checkf(ViewDimensions.IsZero() || ViewDimensions == MipDimensions, TEXT("Depth target size mismatch (Depth: %dx%d vs. Expected: %dx%d). All render and depth target views must have the same effective dimensions."), MipDimensions.X, MipDimensions.Y, ViewDimensions.X, ViewDimensions.Y);
			ViewDimensions = MipDimensions;

			if (DSV.ResolveTarget)
			{
				const FRHITextureDesc& DepthStencilTargetDesc = DSV.DepthStencilTarget->GetDesc();
				const FRHITextureDesc& ResolveTargetDesc = DSV.ResolveTarget->GetDesc();

				checkf(DepthStencilTargetDesc.Extent == ResolveTargetDesc.Extent, TEXT("Depth stencil target extent must match resolve target extent."));
				checkf(DepthStencilTargetDesc.IsTexture2D() && ResolveTargetDesc.IsTexture2D(), TEXT("Only 2D depth stencil resolves are supported."));
			}
		}

		// @todo: additional checks for matching array slice counts on RTVs/DSVs

		// assert depth is in the correct mode
		if (DSV.ExclusiveDepthStencil.IsUsingDepth())
		{
			ERHIAccess DepthAccess = DSV.ExclusiveDepthStencil.IsDepthWrite()
				? ERHIAccess::DSVWrite
				: ERHIAccess::DSVRead;

			checkf(DSV.DepthStencilTarget, TEXT("Depth read/write is enabled but no depth stencil texture is bound."));
			Tracker->Assert(DSV.DepthStencilTarget->GetViewIdentity(0, 0, 0, 0, uint32(RHIValidation::EResourcePlane::Common), 1), DepthAccess);

			if (DSV.ResolveTarget)
			{
				Tracker->Assert(DSV.ResolveTarget->GetViewIdentity(0, 0, 0, 0, uint32(RHIValidation::EResourcePlane::Common), 1), ERHIAccess::ResolveDst);
			}
		}

		// assert stencil is in the correct mode
		if (DSV.ExclusiveDepthStencil.IsUsingStencil())
		{
			ERHIAccess StencilAccess = DSV.ExclusiveDepthStencil.IsStencilWrite()
				? ERHIAccess::DSVWrite
				: ERHIAccess::DSVRead;

			checkf(DSV.DepthStencilTarget, TEXT("Stencil read/write is enabled but no depth stencil texture is bound."));

			bool bIsStencilFormat = IsStencilFormat(DSV.DepthStencilTarget->GetFormat());
			checkf(bIsStencilFormat, TEXT("Stencil read/write is enabled but depth stencil texture doesn't have a stencil plane."));
			if (bIsStencilFormat)
			{
				Tracker->Assert(DSV.DepthStencilTarget->GetViewIdentity(0, 0, 0, 0, uint32(RHIValidation::EResourcePlane::Stencil), 1), StencilAccess);

				if (DSV.ResolveTarget)
				{
					Tracker->Assert(DSV.ResolveTarget->GetViewIdentity(0, 0, 0, 0, uint32(RHIValidation::EResourcePlane::Stencil), 1), ERHIAccess::ResolveDst);
				}
			}
		}

		// assert shading-rate attachment is in the correct mode and format.
		if (State.RenderPassInfo.ShadingRateTexture.IsValid())
		{
			FTextureRHIRef ShadingRateTexture = State.RenderPassInfo.ShadingRateTexture;
			checkf(ShadingRateTexture->GetFormat() == GRHIVariableRateShadingImageFormat, TEXT("Shading rate texture is bound, but is not the correct format for this RHI."));
			Tracker->Assert(ShadingRateTexture->GetViewIdentity(0, 0, 0, 0, 0, 0), ERHIAccess::ShadingRateSource);
		}
	}

	virtual void RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName) override final
	{
		ValidateBeginRenderPass(InInfo, InName);
		RHIContext->RHIBeginRenderPass(InInfo, InName);
	}

	virtual void RHIEndRenderPass() override final
	{
		checkf(State.bInsideBeginRenderPass, TEXT("Trying to end a RenderPass but not inside one!"));
		RHIContext->RHIEndRenderPass();
		State.bInsideBeginRenderPass = false;
		State.PreviousRenderPassName = State.RenderPassName;
	}

	virtual void RHINextSubpass() override final
	{
		RHIContext->RHINextSubpass();
	}

	virtual void RHIBeginParallelRenderPass(TSharedPtr<FRHIParallelRenderPassInfo> InInfo, const TCHAR* InName) override final
	{
		RHIContext->RHIBeginParallelRenderPass(InInfo, InName);
	}

	virtual void RHIEndParallelRenderPass() override final
	{
		RHIContext->RHIEndParallelRenderPass();
	}

	virtual void RHIWriteGPUFence(FRHIGPUFence* FenceRHI) override final
	{
		RHIContext->RHIWriteGPUFence(FenceRHI);
	}

	virtual void RHISetGPUMask(FRHIGPUMask GPUMask) override final
	{
		RHIContext->RHISetGPUMask(GPUMask);
	}

	virtual FRHIGPUMask RHIGetGPUMask() const override final
	{
		return RHIContext->RHIGetGPUMask();
	}

	virtual void RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* DestinationStagingBufferRHI, uint32 InOffset, uint32 InNumBytes) override final;

	virtual void RHICopyTexture(FRHITexture* SourceTexture, FRHITexture* DestTexture, const FRHICopyTextureInfo& CopyInfo) override final
	{
		ensureMsgf(!State.bInsideBeginRenderPass, TEXT("Copying inside a RenderPass is not efficient!"));

		// @todo: do we need to pick subresource, not just whole resource identity here.
		// also, is CopySrc / CopyDest correct?
		Tracker->Assert(SourceTexture->GetWholeResourceIdentity(), ERHIAccess::CopySrc);
		Tracker->Assert(DestTexture->GetWholeResourceIdentity(), ERHIAccess::CopyDest);

		FValidationRHIUtils::ValidateCopyTexture(SourceTexture, DestTexture, CopyInfo);
		RHIContext->RHICopyTexture(SourceTexture, DestTexture, CopyInfo);
	}

	virtual void RHICopyBufferRegion(FRHIBuffer* DestBuffer, uint64 DstOffset, FRHIBuffer* SourceBuffer, uint64 SrcOffset, uint64 NumBytes)
	{
		Tracker->Assert(SourceBuffer->GetWholeResourceIdentity(), ERHIAccess::CopySrc);
		Tracker->Assert(DestBuffer->GetWholeResourceIdentity(), ERHIAccess::CopyDest);
		RHIContext->RHICopyBufferRegion(DestBuffer, DstOffset, SourceBuffer, SrcOffset, NumBytes);
	}

	void RHICommitShaderBindingTable(FRHIShaderBindingTable* SBT, FRHIBuffer* InlineBindingDataBuffer)
	{
		SBT->Commit();
		if (InlineBindingDataBuffer)
		{
			Tracker->Assert(InlineBindingDataBuffer->GetWholeResourceIdentity(), ERHIAccess::CopyDest);
		}
		RHIContext->RHICommitShaderBindingTable(SBT, InlineBindingDataBuffer);
	}

	virtual void RHIExecuteMultiIndirectClusterOperation(const FRayTracingClusterOperationParams& Params) override final
	{
		RHIContext->RHIExecuteMultiIndirectClusterOperation(Params);
	}

	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange) override final
	{
		for (const FRayTracingGeometryBuildParams& P : Params)
		{
			const FRayTracingGeometryInitializer& Initializer = P.Geometry->GetInitializer();

			if (Initializer.IndexBuffer)
			{
				Tracker->Assert(Initializer.IndexBuffer->GetWholeResourceIdentity(), ERHIAccess::SRVCompute);
			}

			for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
			{
				Tracker->Assert(Segment.VertexBuffer->GetWholeResourceIdentity(), ERHIAccess::SRVCompute);
			}
		}

		RHIContext->RHIBuildAccelerationStructures(Params, ScratchBufferRange);
	}

	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params) override final
	{
		for (const FRayTracingSceneBuildParams& P : Params)
		{
			if (P.Scene)
			{
				Tracker->Assert(P.Scene->GetWholeResourceIdentity(), ERHIAccess::BVHWrite);
			}

			if (P.InstanceBuffer)
			{
				Tracker->Assert(P.InstanceBuffer->GetWholeResourceIdentity(), ERHIAccess::SRVCompute);
			}

			if (P.ScratchBuffer)
			{
				Tracker->Assert(P.ScratchBuffer->GetWholeResourceIdentity(), ERHIAccess::UAVCompute);
			}
		}

		RHIContext->RHIBuildAccelerationStructures(Params);
	}

	virtual void RHIBindAccelerationStructureMemory(FRHIRayTracingScene* Scene, FRHIBuffer* Buffer, uint32 BufferOffset) override final
	{
		RHIContext->RHIBindAccelerationStructureMemory(Scene, Buffer, BufferOffset);
	}

	virtual void RHIRayTraceDispatch(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		uint32 Width, uint32 Height) override final
	{
		SBT->ValidateStateForDispatch(Tracker);
		RHIContext->RHIRayTraceDispatch(RayTracingPipelineState, RayGenShader, SBT, GlobalResourceBindings, Width, Height);
	}

	virtual void RHIRayTraceDispatchIndirect(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) override final
	{
		FValidationRHI::ValidateDispatchIndirectArgsBuffer(ArgumentBuffer, ArgumentOffset);
		Tracker->Assert(ArgumentBuffer->GetWholeResourceIdentity(), ERHIAccess::IndirectArgs);
		Tracker->Assert(ArgumentBuffer->GetWholeResourceIdentity(), ERHIAccess::SRVCompute);

		SBT->ValidateStateForDispatch(Tracker);
		RHIContext->RHIRayTraceDispatchIndirect(RayTracingPipelineState, RayGenShader, SBT, GlobalResourceBindings, ArgumentBuffer, ArgumentOffset);
	}

	virtual void RHISetBindingsOnShaderBindingTable(FRHIShaderBindingTable* SBT, FRHIRayTracingPipelineState* Pipeline, uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings, ERayTracingBindingType BindingType) override final
	{
		SBT->SetBindingsOnShaderBindingTable(Pipeline, NumBindings, Bindings, BindingType);
		RHIContext->RHISetBindingsOnShaderBindingTable(SBT, Pipeline, NumBindings, Bindings, BindingType);
	}

	void ValidateDispatch();
	void ValidateDrawing();

	IRHICommandContext* RHIContext = nullptr;

	inline void LinkToContext(IRHICommandContext* PlatformContext)
	{
		RHIContext = PlatformContext;
		PlatformContext->WrappingContext = this;
		PlatformContext->Tracker = &State.TrackerInstance;
	}

protected:
	struct FState
	{
		RHIValidation::FTracker TrackerInstance{ ERHIPipeline::Graphics };
		RHIValidation::FStaticUniformBuffers StaticUniformBuffers;
		RHIValidation::FBoundUniformBuffers BoundUniformBuffers;

		FRHIRenderPassInfo RenderPassInfo;
		FString RenderPassName;
		FString PreviousRenderPassName;
		FString ComputePassName;

		FRHIShader* BoundShaders[SF_NumFrequencies] = {};

		bool bGfxPSOSet{};
		bool bInsideBeginRenderPass{};

		void Reset();
	} State;

	friend class FValidationRHI;

private:
	void ValidateDepthStencilForSetGraphicsPipelineState(const FExclusiveDepthStencil& DSMode)
	{
		FRHIRenderPassInfo::FDepthStencilEntry& DSV = State.RenderPassInfo.DepthStencilRenderTarget;

		// assert depth is in the correct mode
		if (DSMode.IsUsingDepth())
		{
			checkf(DSV.ExclusiveDepthStencil.IsUsingDepth(), TEXT("Graphics PSO is using depth but it's not enabled on the RenderPass."));
			checkf(DSMode.IsDepthRead() || DSV.ExclusiveDepthStencil.IsDepthWrite(), TEXT("Graphics PSO is writing to depth but RenderPass depth is ReadOnly."));
		}

		// assert stencil is in the correct mode
		if (DSMode.IsUsingStencil())
		{
			checkf(DSV.ExclusiveDepthStencil.IsUsingStencil(), TEXT("Graphics PSO is using stencil but it's not enabled on the RenderPass."));
			checkf(DSMode.IsStencilRead() || DSV.ExclusiveDepthStencil.IsStencilWrite(), TEXT("Graphics PSO is writing to stencil but RenderPass stencil is ReadOnly."));
		}
	}
};

#endif	// ENABLE_RHI_VALIDATION
