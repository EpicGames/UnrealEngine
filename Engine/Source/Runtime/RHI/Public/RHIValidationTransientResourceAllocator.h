// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "RHITransientResourceAllocator.h"

#if ENABLE_RHI_VALIDATION

class FValidationTransientResourceAllocator : public IRHITransientResourceAllocator
{
public:
	FValidationTransientResourceAllocator(IRHITransientResourceAllocator* InRHIAllocator)
		: RHIAllocator(InRHIAllocator)
	{}

	virtual ~FValidationTransientResourceAllocator();

	// Implementation of FRHITransientResourceAllocator interface
	virtual void SetCreateMode(ERHITransientResourceCreateMode InCreateMode) override final;
	virtual bool SupportsResourceType(ERHITransientResourceType InType) const override final { return RHIAllocator->SupportsResourceType(InType); }
	virtual FRHITransientTexture* CreateTexture(const FRHITextureCreateInfo& InCreateInfo, const TCHAR* InDebugName, const FRHITransientAllocationFences& Fences) override final;
	virtual FRHITransientBuffer* CreateBuffer(const FRHIBufferCreateInfo& InCreateInfo, const TCHAR* InDebugName, const FRHITransientAllocationFences& Fences) override final;
	virtual void DeallocateMemory(FRHITransientTexture* InTexture, const FRHITransientAllocationFences& Fences) override final;
	virtual void DeallocateMemory(FRHITransientBuffer* InBuffer, const FRHITransientAllocationFences& Fences) override final;
	virtual void Flush(FRHICommandListImmediate&, FRHITransientAllocationStats*) override final;
	virtual void Release(FRHICommandListImmediate&) override final;

private:
	// Actual RHI transient allocator which will get all functions forwarded
	IRHITransientResourceAllocator* RHIAllocator = nullptr;

	// All the allocated resources on the transient allocator
	struct FAllocatedResourceData
	{
		enum class EType
		{
			Texture,
			Buffer,
		};

		FString DebugName;
		EType ResourceType = EType::Texture;
	};

	TMap<FRHIResource*, FAllocatedResourceData> AllocatedResourceMap;
	TRHIPipelineArray<TArray<RHIValidation::FOperation>> PendingPipelineOps;
};

#endif	// ENABLE_RHI_VALIDATION
