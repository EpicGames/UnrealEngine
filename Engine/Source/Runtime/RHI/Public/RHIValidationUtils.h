// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#if ENABLE_RHI_VALIDATION

#include "RHIResources.h"
#include "RHICommandList.h"

class FValidationRHIUtils
{
public:
	static void ValidateCopyTexture(FRHITexture* SourceTexture, FRHITexture* DestTexture, const FRHICopyTextureInfo& CopyInfo)
	{
		check(SourceTexture);
		check(DestTexture);

		const EPixelFormat SrcFormat = SourceTexture->GetFormat();
		const EPixelFormat DstFormat = DestTexture->GetFormat();
		const bool bIsSrcBlockCompressed = GPixelFormats[SrcFormat].BlockSizeX > 1;
		const bool bIsDstBlockCompressed = GPixelFormats[DstFormat].BlockSizeX > 1;
		const int32 SrcBlockBytes = GPixelFormats[SrcFormat].BlockBytes;
		const int32 DstBlockBytes = GPixelFormats[DstFormat].BlockBytes;
		const bool bValidCopyFormats = (SrcFormat == DstFormat) || (!bIsSrcBlockCompressed && bIsDstBlockCompressed && SrcBlockBytes == DstBlockBytes);

		checkf(bValidCopyFormats, TEXT("Some RHIs do not support this format conversion by the GPU for transfer operations!"));

		FIntVector SrcSize = SourceTexture->GetMipDimensions(CopyInfo.SourceMipIndex);
		FIntVector DestSize = DestTexture->GetMipDimensions(CopyInfo.DestMipIndex);
		FIntVector CopySize = CopyInfo.Size;
		if (CopySize == FIntVector::ZeroValue)
		{
			CopySize = SrcSize;
		}

		checkf(CopySize.X <= DestSize.X && CopySize.Y <= DestSize.Y, TEXT("Some RHIs can't perform scaling operations [%dx%d to %dx%d] during copies!"), SrcSize.X, SrcSize.Y, DestSize.X, DestSize.Y);

		check(CopyInfo.SourcePosition.X >= 0 && CopyInfo.SourcePosition.Y >= 0 && CopyInfo.SourcePosition.Z >= 0);
		check(CopyInfo.SourcePosition.X + CopySize.X <= SrcSize.X && CopyInfo.SourcePosition.Y + CopySize.Y <= SrcSize.Y);

		check(CopyInfo.DestPosition.X >= 0 && CopyInfo.DestPosition.Y >= 0 && CopyInfo.DestPosition.Z >= 0);
		check(CopyInfo.DestPosition.X + CopySize.X <= DestSize.X && CopyInfo.DestPosition.Y + CopySize.Y <= DestSize.Y);

		if (SourceTexture->GetTexture3D() && DestTexture->GetTexture3D())
		{
			check(CopyInfo.SourcePosition.Z + CopySize.Z <= SrcSize.Z);
			check(CopyInfo.DestPosition.Z + CopySize.Z <= DestSize.Z);
		}
	}
};

#endif // ENABLE_RHI_VALIDATION
