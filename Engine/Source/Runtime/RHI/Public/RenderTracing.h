// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Trace.h"
#include "Containers/ArrayView.h"
#include "Misc/EnumClassFlags.h"

#define UE_RENDER_TRACING_ENABLED UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING

#if UE_RENDER_TRACING_ENABLED
	#define IF_RENDER_TRACING_ENABLED(Op) Op
#else
	#define IF_RENDER_TRACING_ENABLED(Op)
#endif

UE_TRACE_CHANNEL_EXTERN(RenderTraceChannel, RHI_API);

#if UE_RENDER_TRACING_ENABLED

#define RENDER_TRACE_EVENT_BEGIN(EventName, ...) UE_TRACE_EVENT_BEGIN(RenderTrace, EventName##Message, ##__VA_ARGS__) \
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)

#define RENDER_TRACE_LOG(EventName) UE_TRACE_LOG(RenderTrace, EventName##Message, RenderTraceChannel) \
	<< EventName##Message.Timestamp(FPlatformTime::Cycles64())

#define RENDER_TRACE_VALUE(EventName, ValueName, Value) EventName##Message.ValueName(Value)
#define RENDER_TRACE_ARRAY_VALUE(EventName, ValueName, Value, ElemType) EventName##Message.ValueName(reinterpret_cast<const ElemType*>(Value.GetData()), Value.Num())

enum class ERenderTracingChannels : uint64
{
	None = 0,
	RDGPasses = 1 << 0,
	RHICmdLists = 1 << 1,
	RHITranslation = 1 << 2,
	RHISubmission = 1 << 3,

	Submission = RDGPasses | RHICmdLists | RHITranslation | RHISubmission,

	All = 0xffffffffffffffffull
};
ENUM_CLASS_FLAGS(ERenderTracingChannels);


namespace RenderTracing
{

void Initialize();

RHI_API bool IsEnabled(ERenderTracingChannels ChannelMask);

}

#endif
