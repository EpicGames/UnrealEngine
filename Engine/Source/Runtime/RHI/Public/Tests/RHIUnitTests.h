// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class IRHITestModule : public IModuleInterface
{
public:
	virtual void RunAllTests() = 0;
};
