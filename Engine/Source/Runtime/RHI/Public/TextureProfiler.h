// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.7, "TextureProfiler.h is no longer used.")

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Misc/Build.h"
#include "Templates/Function.h"
#include "Misc/CoreMisc.h"
#include "Containers/Map.h"
#include "UObject/NameTypes.h"
#endif

#define TEXTURE_PROFILER_ENABLED 0
