// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Features/IModularFeature.h"
#include "RHIDefinitions.h"
#include "UObject/NameTypes.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include <dxgi1_6.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

class IDXGISwapchainProvider : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName("DXGISwapchainProvider");
		return FeatureName;
	}

	virtual bool SupportsRHI(ERHIInterfaceType RHIType) const = 0;

	virtual const TCHAR* GetProviderName() const = 0;

	virtual HRESULT CreateSwapChainForHwnd(IDXGIFactory2* pFactory, IUnknown* pDevice, HWND hWnd, const DXGI_SWAP_CHAIN_DESC1* pDesc, const DXGI_SWAP_CHAIN_FULLSCREEN_DESC* pFulScreenDesc, IDXGIOutput* pRestrictToOutput, IDXGISwapChain1** ppSwapChain) = 0;
	virtual HRESULT CreateSwapChain(IDXGIFactory* pFactory, IUnknown* pDevice, DXGI_SWAP_CHAIN_DESC* pDesc, IDXGISwapChain** ppSwapChain) = 0;
};
