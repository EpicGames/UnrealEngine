// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using EpicGames.Core;

public class RHI : ModuleRules
{
	public RHI(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("TraceLog");
		PrivateDependencyModuleNames.Add("ApplicationCore");
		PrivateDependencyModuleNames.Add("Cbor");
		PrivateDependencyModuleNames.Add("BuildSettings");
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PrivateDependencyModuleNames.Add("WindowsD3D");
		}

		PublicDefinitions.AddDefinition("WITH_MGPU", Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop));

		if (Target.bCompileAgainstEngine)
		{
			DynamicallyLoadedModuleNames.Add("NullDrv");

			if (Target.Type != TargetRules.TargetType.Server)   // Dedicated servers should skip loading everything but NullDrv
			{
				// Always disable for Shipping builds. Disable by default in Test builds but allow the target to force enable it.
				if (Target.Configuration != UnrealTargetConfiguration.Shipping && (Target.Configuration != UnrealTargetConfiguration.Test || Target.bTrackRHIResourceInfoForTest))
				{
					PublicDefinitions.Add("RHI_WANT_RESOURCE_INFO=1");
				}

				if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
				{
					DynamicallyLoadedModuleNames.Add("D3D11RHI");
					DynamicallyLoadedModuleNames.Add("D3D12RHI");
				}

				// UEBuildAndroid.cs adds VulkanRHI for Android builds if it is enabled
				if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) || Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
				{
					DynamicallyLoadedModuleNames.Add("VulkanRHI");
				}

				if ((Target.Platform.IsInGroup(UnrealPlatformGroup.Windows)) ||
					(Target.IsInPlatformGroup(UnrealPlatformGroup.Linux) && Target.Type != TargetRules.TargetType.Server))  // @todo should servers on all platforms skip this?
				{
					DynamicallyLoadedModuleNames.Add("OpenGLDrv");
				}
			}
		}

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateIncludePathModuleNames.AddRange(new string[] { "ProfileVisualizer" });
		}

		if (Target.bBuildEditor == true)
		{
			PrivateIncludePathModuleNames.Add("TargetPlatform");
			DynamicallyLoadedModuleNames.Add("TargetPlatform");
		}
	}
}
